{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Animation Insights",
	"Description": "Allows debugging of animation systems via Unreal Insights",
	"Category": "Insights",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedPrograms": [ "UnrealInsights" ],
	"Modules": [
		{
			"Name": "GameplayInsights",
			"Type": "EditorAndProgram",
			"LoadingPhase": "Default",
			"ProgramAllowList": [
				"UnrealInsights"
			]
		},
		{
			"Name": "GameplayInsightsEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "RewindDebugger",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		},
		{
			"Name": "RewindDebuggerVLog",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		},
		{
			"Name": "RewindDebuggerRuntime",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "RewindDebuggerVLogRuntime",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}
