// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GameplayInsights : ModuleRules
	{
		public GameplayInsights(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"InputCore",
				"SlateCore",
				"Slate",
				"TraceLog",
				"TraceAnalysis",
				"TraceServices",
				"TraceInsights",
				"TraceInsightsCore",
				"AssetRegistry",
				"ApplicationCore",
				"RewindDebuggerInterface",
				"ToolWidgets"
			});

			PublicIncludePathModuleNames.AddRange(
				new string[]
				{
					"TraceServices"
				}
			);

			if (Target.bCompileAgainstEngine)
			{
				PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"CoreUObject",
				});
			}

			if (Target.Type == TargetType.Editor)
			{
				PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AnimationBlueprintEditor",
					"Persona",
					"EditorFramework",
					"UnrealEd",
  					"GameplayInsightsEditor",
					"EditorWidgets",
					"ToolMenus",
					"Kismet",
					"SubobjectEditor",
					"AnimGraph",
					"AnimGraphRuntime"
				});
			}
		}
	}
}

