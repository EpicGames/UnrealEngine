// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimCurvesTrack.h"
#include "Common/ProviderLock.h"
#include "SCurveTimelineView.h"
#include "IRewindDebugger.h"
#include "GameplayProvider.h"
#include "AnimationProvider.h"
#include "SAnimationCurvesView.h"
#if WITH_ENGINE
#include "Components/SkeletalMeshComponent.h"
#endif // WITH_ENGINE

#define LOCTEXT_NAMESPACE "AnimCurvesTrack"

namespace RewindDebugger
{

FAnimCurvesTrack::FAnimCurvesTrack(uint64 InObjectId) : ObjectId(InObjectId)
{
	ChildPlaceholder = MakeShared<FRewindDebuggerPlaceholderTrack>( "Child", LOCTEXT("No Curves", "No Curves to Display"));
	SetIsExpanded(false);
	Icon = FSlateIcon("EditorStyle", "AnimGraph.Attribute.Curves.Icon", "AnimGraph.Attribute.Curves.Icon");
}

TSharedPtr<SWidget> FAnimCurvesTrack::GetDetailsViewInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	return SNew(SAnimationCurvesView, ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession())
				.CurrentTime_Lambda([RewindDebugger]{ return RewindDebugger->CurrentTraceTime(); });
	
}

TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> FAnimCurvesTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	if (Children.Num() == 0)
	{
		return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(&ChildPlaceholder), 1);
	}

	return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(Children.GetData()), Children.Num());
}

bool FAnimCurvesTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimCurvesTrack::UpdateInternal);
	if (!GetIsExpanded())
	{
		return false;
	}
	
	TArray<uint32> UniqueTrackIds;

	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
	if (AnimationProvider == nullptr)
	{
		return false;
	}

	bool bChanged = false;
	
	// convert time range to from rewind debugger times to profiler times
	TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	double StartTime = TraceTimeRange.GetLowerBoundValue();
	double EndTime = TraceTimeRange.GetUpperBoundValue();
	
	// count number of unique animations in the current time range
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		UniqueTrackIds.SetNum(0, EAllowShrinking::No);

		AnimationProvider->ReadSkeletalMeshPoseTimeline(ObjectId, [&UniqueTrackIds,AnimationProvider, StartTime, EndTime](const FAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bHasCurves)
		{
			// this isn't very efficient, and it gets called every frame.  will need optimizing
			InTimeline.EnumerateEvents(StartTime, EndTime, [&UniqueTrackIds, StartTime, EndTime, AnimationProvider](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InMessage)
			{
				if (InEndTime > StartTime && InStartTime < EndTime)
				{
					AnimationProvider->EnumerateSkeletalMeshCurves(InMessage, [&UniqueTrackIds](const FSkeletalMeshNamedCurve& InCurve)
					{
						UniqueTrackIds.AddUnique(InCurve.Id);
					});
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}

	UniqueTrackIds.StableSort();

	const int32 CurveCount = UniqueTrackIds.Num();

	if (Children.Num() != UniqueTrackIds.Num())
		bChanged = true;
	
	Children.SetNum(CurveCount);
	for (int i = 0; i < CurveCount; i++)
	{
		if (!Children[i].IsValid() || Children[i].Get()->GetCurveId() != UniqueTrackIds[i])
		{
			Children[i] = MakeShared<FAnimCurveTrack>(ObjectId, UniqueTrackIds[i]);
			bChanged = true;
		}

		if (Children[i]->Update())
		{
			bChanged = true;
		}
	}

	return bChanged;
}


FAnimCurveTrack::FAnimCurveTrack(uint64 InObjectId, uint32 InCurveId) :
	CurveId(InCurveId),
	ObjectId(InObjectId)
{
	Icon = FSlateIcon("EditorStyle", "AnimGraph.Attribute.Curves.Icon", "AnimGraph.Attribute.Curves.Icon");
}

TSharedPtr<SCurveTimelineView::FTimelineCurveData> FAnimCurveTrack::GetCurveData() const
{
	if (!CurveData.IsValid())
	{
		CurveData = MakeShared<SCurveTimelineView::FTimelineCurveData>();
	}
	
	CurvesUpdateRequested++;
	
	return CurveData;
}

bool FAnimCurveTrack::UpdateInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if (CurvesUpdateRequested > 10 && AnimationProvider)
	{
		CurvesUpdateRequested = 0;
		UpdateCurvePointsInternal();
	}

	bool bChanged = false;
	if (CurveName.IsEmpty() && AnimationProvider)
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		CurveName = FText::FromString(AnimationProvider->GetName(CurveId));
		bChanged = true;
	}

	return bChanged;
}

void FAnimCurveTrack::UpdateCurvePointsInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimCurveTrack::UpdateCurvePointsInternal);

	auto& CurvePoints = CurveData->Points;
	CurvePoints.SetNum(0, EAllowShrinking::No);

	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
	if (AnimationProvider == nullptr)
	{
		return;
	}

	// convert time range to from rewind debugger times to profiler times
	const TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	double StartTime = TraceTimeRange.GetLowerBoundValue();
	double EndTime = TraceTimeRange.GetUpperBoundValue();

	TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
	AnimationProvider->ReadSkeletalMeshPoseTimeline(ObjectId, [this, AnimationProvider, StartTime, EndTime, AnalysisSession, &CurvePoints](const FAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bHasCurves)
	{
		// this isn't very efficient, and it gets called every frame.  will need optimizing
		InTimeline.EnumerateEvents(StartTime, EndTime, [this, StartTime, EndTime, AnimationProvider, AnalysisSession, &CurvePoints](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InMessage)
		{
			if (InEndTime > StartTime && InStartTime < EndTime)
			{
				double Time = InMessage.RecordingTime;
				AnimationProvider->EnumerateSkeletalMeshCurves(InMessage, [this, Time, &CurvePoints](const FSkeletalMeshNamedCurve& InCurve)
				{
					if (InCurve.Id == CurveId)
					{
						CurvePoints.Add({Time,InCurve.Value});
					}
				});
			}
			return TraceServices::EEventEnumerate::Continue;
		});
	});
}

TSharedPtr<SWidget> FAnimCurveTrack::GetDetailsViewInternal() 
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	TSharedPtr<SAnimationCurvesView> AnimationCurvesView = SNew(SAnimationCurvesView, ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession())
				.CurrentTime_Lambda([RewindDebugger]{ return RewindDebugger->CurrentTraceTime(); });
	AnimationCurvesView->SetCurveFilter(CurveId);
	return AnimationCurvesView;
}

TSharedPtr<SWidget> FAnimCurveTrack::GetTimelineViewInternal()
{
	FLinearColor CurveColor(0.5,0.5,0.5);
	
	return SNew(SCurveTimelineView)
		.TrackName(GetDisplayNameInternal())
		.CurveColor(CurveColor)
		.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
		.RenderFill(false)
		.CurveData_Raw(this, &FAnimCurveTrack::GetCurveData);
}

#if WITH_ENGINE
static const FName AnimationCurvesName("AnimationCurves");
FName FAnimationCurvesTrackCreator::GetTargetTypeNameInternal() const
{
	return USkeletalMeshComponent::StaticClass()->GetFName();
}
	
FName FAnimationCurvesTrackCreator::GetNameInternal() const
{
	return AnimationCurvesName;
}
		
void FAnimationCurvesTrackCreator::GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const
{
	Types.Add({AnimationCurvesName, LOCTEXT("Curves", "Curves")});
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FAnimationCurvesTrackCreator::CreateTrackInternal(const FObjectId& InObjectId) const
{
	return MakeShared<RewindDebugger::FAnimCurvesTrack>(InObjectId.GetMainId());
}

bool FAnimationCurvesTrackCreator::HasDebugInfoInternal(const FObjectId& InObjectId) const
{
	const TraceServices::IAnalysisSession* AnalysisSession = IRewindDebugger::Instance()->GetAnalysisSession();
	
	bool bHasData = false;
	if (const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName))
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadSkeletalMeshPoseTimeline(InObjectId.GetMainId(), [&bHasData](const FAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bInHasCurves)
		{
			bHasData = bInHasCurves;
		});
	}
	return bHasData;
}
#endif // WITH_ENGINE

}

#undef LOCTEXT_NAMESPACE