// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <RewindDebuggerPlaceholderTrack.h>

#include "IAnimationProvider.h"
#include "IRewindDebuggerTrackCreator.h"
#include "RewindDebuggerTrack.h"
#include "RewindDebuggerPlaceholderTrack.h"
#include "SCurveTimelineView.h"
#include "Textures/SlateIcon.h"

namespace RewindDebugger
{

// Sub Track for a single curve
class FAnimCurveTrack : public FRewindDebuggerTrack
{
public:
	FAnimCurveTrack(uint64 InObjectId, uint32 InCurveId);
	
	uint32 GetCurveId() { return CurveId; };

private:
	virtual bool UpdateInternal() override;
	TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FName GetNameInternal() const override { return "AnimationCurve"; }
	virtual FText GetDisplayNameInternal() const override { return CurveName; }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }
	virtual void UpdateCurvePointsInternal();
	
	TSharedPtr<SCurveTimelineView::FTimelineCurveData> GetCurveData() const;
protected:
	uint32 CurveId;

	FSlateIcon Icon;
	FText CurveName;
	uint64 ObjectId;

	mutable TSharedPtr<SCurveTimelineView::FTimelineCurveData> CurveData;
	mutable int CurvesUpdateRequested = 0;
};

// Parent track of all the curves
class FAnimCurvesTrack : public FRewindDebuggerTrack
{
public:
	FAnimCurvesTrack(uint64 InObjectId);


private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	
	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FName GetNameInternal() const override { return "AnimationCurves"; }
	virtual FText GetDisplayNameInternal() const override { return NSLOCTEXT("RewindDebugger", "AnimationCurvesTrackName", "Animation Curves"); }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;
	
	FSlateIcon Icon;
	uint64 ObjectId;

	TSharedPtr<FRewindDebuggerPlaceholderTrack> ChildPlaceholder;
	TArray<TSharedPtr<FAnimCurveTrack>> Children;
};

#if WITH_ENGINE
class FAnimationCurvesTrackCreator : public IRewindDebuggerTrackCreator
{
private:
	virtual FName GetTargetTypeNameInternal() const override;
	virtual FName GetNameInternal() const override;
	virtual void GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<FRewindDebuggerTrack> CreateTrackInternal(const FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const FObjectId& InObjectId) const override;
};
#endif // WITH_ENGINE

}