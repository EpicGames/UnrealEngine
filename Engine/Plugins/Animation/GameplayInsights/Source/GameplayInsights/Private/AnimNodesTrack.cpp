// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodesTrack.h"

#include "AnimationProvider.h"
#include "AnimationSharedData.h"
#include "Common/ProviderLock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameplayProvider.h"
#include "GameplayTimingViewExtender.h"
#include "SkeletalMeshPoseTrack.h"

// TraceServices
#include "TraceServices/Model/Frames.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/ViewModels/ITimingViewDrawHelper.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"

#if WITH_ENGINE
#include "Animation/AnimTrace.h"
#include "Components/SkeletalMeshComponent.h"
#endif

#if WITH_EDITOR
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "IAnimationBlueprintEditor.h"
#include "IPersonaToolkit.h"
#include "Animation/AnimBlueprint.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Animation/AnimInstance.h"
#include "Animation/BlendSpace.h"
#endif

#define LOCTEXT_NAMESPACE "AnimNodesTrack"

INSIGHTS_IMPLEMENT_RTTI(FAnimNodesTrack)

FAnimNodesTrack::FAnimNodesTrack(const FAnimationSharedData& InSharedData, uint64 InObjectID, const TCHAR* InName)
	: FGameplayTimingEventsTrack(InSharedData.GetGameplaySharedData(), InObjectID, FText::Format(LOCTEXT("TrackNameFormat", "Graph - {0}"), FText::FromString(FString(InName))))
	, SharedData(InSharedData)
#if WITH_EDITOR
	, InstanceClass(nullptr)
	, AnimInstance(nullptr)
#endif
{
#if WITH_EDITOR
	const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);

	if(GameplayProvider)
	{
		const FObjectInfo* AnimInstanceInfo = GameplayProvider->FindObjectInfo(GetGameplayTrack().GetObjectId());
		if(AnimInstanceInfo)
		{
			const FClassInfo* AnimInstanceClassInfo = GameplayProvider->FindClassInfo(AnimInstanceInfo->ClassId);
			if(AnimInstanceClassInfo)
			{
				InstanceClass = FSoftObjectPath(AnimInstanceClassInfo->PathName);
			}
		}
	}
#endif

#if WITH_ENGINE
	OnWorldCleanupHandle = FWorldDelegates::OnWorldCleanup.AddRaw(this, &FAnimNodesTrack::OnWorldCleanup);
	OnWorldBeginTearDownHandle = FWorldDelegates::OnWorldBeginTearDown.AddRaw(this, &FAnimNodesTrack::RemoveWorld);
	OnPreWorldFinishDestroyHandle = FWorldDelegates::OnPreWorldFinishDestroy.AddRaw(this, &FAnimNodesTrack::RemoveWorld);
#endif
}

FAnimNodesTrack::~FAnimNodesTrack()
{
#if WITH_ENGINE
	FWorldDelegates::OnPreWorldFinishDestroy.Remove(OnPreWorldFinishDestroyHandle);
	FWorldDelegates::OnWorldBeginTearDown.Remove(OnWorldBeginTearDownHandle);
	FWorldDelegates::OnWorldCleanup.Remove(OnWorldCleanupHandle);
#endif
}

static const TCHAR* GetPhaseName(EAnimGraphPhase InPhase)
{
#if WITH_ENGINE && ANIM_TRACE_ENABLED
	static_assert((__underlying_type(EAnimGraphPhase))EAnimGraphPhase::Initialize == (__underlying_type(FAnimTrace::EPhase))FAnimTrace::EPhase::Initialize, "EAnimGraphPhase and FAnimTrace::EPhase must be kept in sync");
	static_assert((__underlying_type(EAnimGraphPhase))EAnimGraphPhase::PreUpdate == (__underlying_type(FAnimTrace::EPhase))FAnimTrace::EPhase::PreUpdate, "EAnimGraphPhase and FAnimTrace::EPhase must be kept in sync");
	static_assert((__underlying_type(EAnimGraphPhase))EAnimGraphPhase::Update == (__underlying_type(FAnimTrace::EPhase))FAnimTrace::EPhase::Update, "EAnimGraphPhase and FAnimTrace::EPhase must be kept in sync");
	static_assert((__underlying_type(EAnimGraphPhase))EAnimGraphPhase::CacheBones == (__underlying_type(FAnimTrace::EPhase))FAnimTrace::EPhase::CacheBones, "EAnimGraphPhase and FAnimTrace::EPhase must be kept in sync");
	static_assert((__underlying_type(EAnimGraphPhase))EAnimGraphPhase::Evaluate == (__underlying_type(FAnimTrace::EPhase))FAnimTrace::EPhase::Evaluate, "EAnimGraphPhase and FAnimTrace::EPhase must be kept in sync");
#endif

	switch(InPhase)
	{
	case EAnimGraphPhase::Initialize:
		return TEXT("Initialize");
	case EAnimGraphPhase::PreUpdate:
		return TEXT("PreUpdate");
	case EAnimGraphPhase::Update:
		return TEXT("Update");
	case EAnimGraphPhase::CacheBones:
		return TEXT("CacheBones");
	case EAnimGraphPhase::Evaluate:
		return TEXT("Evaluate");
	}

	return TEXT("Unknown");
}

void FAnimNodesTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if(AnimationProvider)
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);

		AnimationProvider->ReadAnimGraphTimeline(GetGameplayTrack().GetObjectId(), [&Context, &Builder](const FAnimationProvider::AnimGraphTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(Context.GetViewport().GetStartTime(), Context.GetViewport().GetEndTime(), [&Builder](double InStartTime, double InEndTime, uint32 InDepth, const FAnimGraphMessage& InMessage)
			{
				Builder.AddEvent(InStartTime, InEndTime, 0, GetPhaseName(InMessage.Phase));
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}

void FAnimNodesTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	DrawEvents(Context);
	GetGameplayTrack().DrawHeaderForTimingTrack(Context, *this, false);
}

void FAnimNodesTrack::InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const
{
	FTimingEventSearchParameters SearchParameters(HoveredTimingEvent.GetStartTime(), HoveredTimingEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch);

	FindAnimGraphMessage(SearchParameters, [this, &Tooltip](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FAnimGraphMessage& InMessage)
	{
		Tooltip.ResetContent();

		Tooltip.AddTitle(FText::Format(LOCTEXT("GraphPhaseFormat", "{0} Anim Graph"), FText::FromString(GetPhaseName(InMessage.Phase))).ToString());

		Tooltip.AddNameValueTextLine(LOCTEXT("EventDuration", "Duration").ToString(), UE::Insights::FormatTimeAuto(InFoundEndTime - InFoundStartTime));
		Tooltip.AddNameValueTextLine(LOCTEXT("EventTime", "Time").ToString(), UE::Insights::FormatTimeAuto(InFoundStartTime));
		Tooltip.AddNameValueTextLine(LOCTEXT("EventWorld", "World").ToString(), GetGameplayTrack().GetWorldName(SharedData.GetAnalysisSession()).ToString());

		Tooltip.UpdateLayout();
	});
}

const TSharedPtr<const ITimingEvent> FAnimNodesTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	FindAnimGraphMessage(InSearchParameters, [this, &FoundEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FAnimGraphMessage& InFoundMessage)
	{
		FoundEvent = MakeShared<const FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth);
	});

	return FoundEvent;
}

void FAnimNodesTrack::FindAnimGraphMessage(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FAnimGraphMessage&)> InFoundPredicate) const
{
	TTimingEventSearch<FAnimGraphMessage>::Search(
		InParameters,

		[this](TTimingEventSearch<FAnimGraphMessage>::FContext& InContext)
		{
			const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

			if(AnimationProvider)
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);

				AnimationProvider->ReadAnimGraphTimeline(GetGameplayTrack().GetObjectId(), [&InContext](const FAnimationProvider::AnimGraphTimeline& InTimeline)
				{
					InTimeline.EnumerateEvents(InContext.GetParameters().StartTime, InContext.GetParameters().EndTime, [&InContext](double InEventStartTime, double InEventEndTime, uint32 InDepth, const FAnimGraphMessage& InMessage)
					{
						InContext.Check(InEventStartTime, InEventEndTime, 0, InMessage);
						return TraceServices::EEventEnumerate::Continue;
					});
				});
			}
		},

		[&InFoundPredicate](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FAnimGraphMessage& InEvent)
		{
			InFoundPredicate(InFoundStartTime, InFoundEndTime, InFoundDepth, InEvent);
		});
}

void FAnimNodesTrack::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection(TEXT("DebugSection"), LOCTEXT("Debug", "Debug"));
	{
#if WITH_EDITOR
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleDebug", "Debug this graph"),
			LOCTEXT("ToggleDebug_Tooltip", "Debug this graph in the animation blueprint editor, opens editor for asset if it exists"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{  
					if(InstanceClass.LoadSynchronous())
					{
						if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(InstanceClass->ClassGeneratedBy))
						{
							GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AnimBlueprint);

							const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
							if (GameplayProvider)
							{
								uint64 OuterUObjectId = 0;
								{
									TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
									if(const FObjectInfo* AnimInstanceObjectInfo = GameplayProvider->FindObjectInfo(GetGameplayTrack().GetObjectId()))
									{
										OuterUObjectId = AnimInstanceObjectInfo->GetOuterUObjectId();
									}
								}

								if (OuterUObjectId != 0)
								{
									// find a counterpart skeletal mesh pose track
									TSharedPtr<FSkeletalMeshPoseTrack> SkeletalMeshPoseTrack = SharedData.FindSkeletalMeshPoseTrack(OuterUObjectId);
									if(SkeletalMeshPoseTrack.IsValid())
									{
										SkeletalMeshPoseTrack->MarkPotentiallyDebugged();

										if(USkeletalMeshComponent* SkeletalMeshComponent = SkeletalMeshPoseTrack->GetComponent(FGameplayTimingViewExtender::GetWorldToVisualize()))
										{
											UAnimInstance* Instance = LazyCreateAnimInstance(SkeletalMeshComponent);
											if(Instance)
											{
												AnimBlueprint->SetObjectBeingDebugged(Instance);
											}
										}
									}
								}
							}
						}
					}
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]()
				{
					if(InstanceClass.Get())
					{
						if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(InstanceClass.Get()->ClassGeneratedBy))
						{
							const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
							if (GameplayProvider)
							{
								uint64 OuterUObjectId = 0;
								{
									TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
									if(const FObjectInfo* AnimInstanceObjectInfo = GameplayProvider->FindObjectInfo(GetGameplayTrack().GetAssociatedObjectId()))
									{
										OuterUObjectId = AnimInstanceObjectInfo->GetOuterUObjectId();
									}
								}

								if (OuterUObjectId != 0)
								{
									// find a counterpart skeletal mesh pose track
									TSharedPtr<FSkeletalMeshPoseTrack> SkeletalMeshPoseTrack = SharedData.FindSkeletalMeshPoseTrack(OuterUObjectId);
									if(SkeletalMeshPoseTrack.IsValid())
									{
										if(USkeletalMeshComponent* SkeletalMeshComponent = SkeletalMeshPoseTrack->GetComponent(FGameplayTimingViewExtender::GetWorldToVisualize()))
										{
											UAnimInstance* Instance = LazyCreateAnimInstance(SkeletalMeshComponent);
											if(Instance)
											{
												AnimBlueprint->IsObjectBeingDebugged(Instance);
											}
										}
									}
								}
							}
						}
					}
					return false;
				})),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
#endif
		MenuBuilder.AddMenuEntry(
			LOCTEXT("OpenAnimGraph", "View this graph"),
			LOCTEXT("OpenAnimGraph_Tooltip", "Open this graph in the schematic anim graph viewer"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{ 
					SharedData.OpenAnimGraphTab(GetGameplayTrack().GetObjectId());
				})
			)
		);
	}
	MenuBuilder.EndSection();
}

#if WITH_EDITOR

UAnimInstance* FAnimNodesTrack::LazyCreateAnimInstance(USkeletalMeshComponent* InComponent)
{
	if(InstanceClass.LoadSynchronous() != nullptr)
	{
		if(AnimInstance == nullptr)
		{
			AnimInstance = NewObject<UAnimInstance>(InComponent, InstanceClass.Get());
		}
		return AnimInstance;
	}

	return nullptr;
}

void FAnimNodesTrack::UpdateDebugData(const TraceServices::FFrame& InFrame)
{
	if(InstanceClass.LoadSynchronous())
	{
		if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(InstanceClass.Get()->ClassGeneratedBy))
		{
			if(AnimInstance && AnimBlueprint->IsObjectBeingDebugged(AnimInstance))
			{
				const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
				const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);

				if(AnimationProvider && GameplayProvider)
				{
					TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);

					const int32 NodeCount = InstanceClass.Get()->GetAnimNodeProperties().Num();

					AnimationProvider->ReadAnimGraphTimeline(GetGameplayTrack().GetObjectId(), [this, AnimationProvider, GameplayProvider, InFrame, NodeCount](const FAnimationProvider::AnimGraphTimeline& InGraphTimeline)
					{
						FAnimBlueprintDebugData& DebugData = InstanceClass.Get()->GetAnimBlueprintDebugData();
						DebugData.ResetNodeVisitSites();

						InGraphTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, [this, AnimationProvider, GameplayProvider, &DebugData, NodeCount](double InGraphStartTime, double InGraphEndTime, uint32 InDepth, const FAnimGraphMessage& InMessage)
						{
							// Basic verification - check node count is the same
							// @TODO: could add some form of node hash/CRC to the class to improve this
							if(InMessage.NodeCount == NodeCount)
							{
								// Check for an update phase (which contains weights)
								if(InMessage.Phase == EAnimGraphPhase::Update)
								{
									AnimationProvider->ReadAnimNodesTimeline(GetGameplayTrack().GetObjectId(), [InGraphStartTime, InGraphEndTime, &DebugData](const FAnimationProvider::AnimNodesTimeline& InNodesTimeline)
									{
										InNodesTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [&DebugData](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNodeMessage& InMessage)
										{
											DebugData.RecordNodeVisit(InMessage.NodeId, InMessage.PreviousNodeId, InMessage.Weight);
											return TraceServices::EEventEnumerate::Continue;
										});
									});

									AnimationProvider->ReadStateMachinesTimeline(GetGameplayTrack().GetObjectId(), [InGraphStartTime, InGraphEndTime, &DebugData](const FAnimationProvider::StateMachinesTimeline& InStateMachinesTimeline)
									{
										InStateMachinesTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [&DebugData](double InStartTime, double InEndTime, uint32 InDepth, const FAnimStateMachineMessage& InMessage)
										{
											DebugData.RecordStateData(InMessage.StateMachineIndex, InMessage.StateIndex, InMessage.StateWeight, InMessage.ElapsedTime);
											return TraceServices::EEventEnumerate::Continue;
										});
									});

									AnimationProvider->ReadAnimSequencePlayersTimeline(GetGameplayTrack().GetObjectId(), [InGraphStartTime, InGraphEndTime, &DebugData](const FAnimationProvider::AnimSequencePlayersTimeline& InSequencePlayersTimeline)
									{
										InSequencePlayersTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [&DebugData](double InStartTime, double InEndTime, uint32 InDepth, const FAnimSequencePlayerMessage& InMessage)
										{
											DebugData.RecordSequencePlayer(InMessage.NodeId, InMessage.Position, InMessage.Length, InMessage.FrameCounter);
											return TraceServices::EEventEnumerate::Continue;
										});
									});

									AnimationProvider->ReadAnimBlendSpacePlayersTimeline(GetGameplayTrack().GetObjectId(), [InGraphStartTime, InGraphEndTime, GameplayProvider, &DebugData](const FAnimationProvider::BlendSpacePlayersTimeline& InBlendSpacePlayersTimeline)
									{
										TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
										InBlendSpacePlayersTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [GameplayProvider, &DebugData](double InStartTime, double InEndTime, uint32 InDepth, const FBlendSpacePlayerMessage& InMessage)
										{
											UBlendSpace* BlendSpace = nullptr;
											const FObjectInfo* BlendSpaceInfo = GameplayProvider->FindObjectInfo(InMessage.BlendSpaceId);
											if(BlendSpaceInfo)
											{
												BlendSpace = TSoftObjectPtr<UBlendSpace>(FSoftObjectPath(BlendSpaceInfo->PathName)).LoadSynchronous();
											}

											DebugData.RecordBlendSpacePlayer(InMessage.NodeId, BlendSpace, FVector(InMessage.PositionX, InMessage.PositionY, InMessage.PositionZ), FVector(InMessage.FilteredPositionX, InMessage.FilteredPositionY, InMessage.FilteredPositionZ));
											return TraceServices::EEventEnumerate::Continue;
										});
									});

									AnimationProvider->ReadAnimSyncTimeline(GetGameplayTrack().GetObjectId(), [InGraphStartTime, InGraphEndTime, AnimationProvider, &DebugData](const FAnimationProvider::AnimSyncTimeline& InAnimSyncTimeline)
									{
										InAnimSyncTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [AnimationProvider, &DebugData](double InStartTime, double InEndTime, uint32 InDepth, const FAnimSyncMessage& InMessage)
										{
											const TCHAR* GroupName = AnimationProvider->GetName(InMessage.GroupNameId);
											if(GroupName)
											{
												DebugData.RecordNodeSync(InMessage.SourceNodeId, FName(GroupName));
											}
									
											return TraceServices::EEventEnumerate::Continue;
										});
									});
								}

								// Some traces come from both update and evaluate phases
								if(InMessage.Phase == EAnimGraphPhase::Update || InMessage.Phase == EAnimGraphPhase::Evaluate)
								{
									AnimationProvider->ReadAnimAttributesTimeline(GetGameplayTrack().GetObjectId(), [InGraphStartTime, InGraphEndTime, AnimationProvider, &DebugData](const FAnimationProvider::AnimAttributeTimeline& InAnimAttributeTimeline)
									{
										InAnimAttributeTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [AnimationProvider, &DebugData](double InStartTime, double InEndTime, uint32 InDepth, const FAnimAttributeMessage& InMessage)
										{
											const TCHAR* AttributeName = AnimationProvider->GetName(InMessage.AttributeNameId);
											if(AttributeName)
											{
												DebugData.RecordNodeAttribute(InMessage.TargetNodeId, InMessage.SourceNodeId, FName(AttributeName));
											}
									
											return TraceServices::EEventEnumerate::Continue;
										});
									});
								}

								// Anim node values can come from all phases
								AnimationProvider->ReadAnimNodeValuesTimeline(GetGameplayTrack().GetObjectId(), [InGraphStartTime, InGraphEndTime, AnimationProvider, &DebugData](const FAnimationProvider::AnimNodeValuesTimeline& InNodeValuesTimeline)
								{
									InNodeValuesTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [AnimationProvider, &DebugData](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNodeValueMessage& InMessage)
									{
										FText Text = AnimationProvider->FormatNodeKeyValue(InMessage);
										DebugData.RecordNodeValue(InMessage.NodeId, Text.ToString());
										return TraceServices::EEventEnumerate::Continue;
									});
								});
							}
							return TraceServices::EEventEnumerate::Continue;
						});
					});
				}
			}
		}	
	}
}

void FAnimNodesTrack::GetCustomDebugObjects(const IAnimationBlueprintEditor& InAnimationBlueprintEditor, TArray<FCustomDebugObject>& OutDebugList)
{
	if(InstanceClass.LoadSynchronous())
	{
		if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(InstanceClass.Get()->ClassGeneratedBy))
		{
			const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
			if (GameplayProvider)
			{
				uint64 OuterUObjectId = 0;
				{
					TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
					if(const FObjectInfo* AnimInstanceObjectInfo = GameplayProvider->FindObjectInfo(GetGameplayTrack().GetObjectId()))
					{
						OuterUObjectId = AnimInstanceObjectInfo->GetOuterUObjectId();
					}
				}

				if (OuterUObjectId != 0)
				{
					// find a counterpart skeletal mesh pose track
					TSharedPtr<FSkeletalMeshPoseTrack> SkeletalMeshPoseTrack = SharedData.FindSkeletalMeshPoseTrack(OuterUObjectId);
					if(SkeletalMeshPoseTrack.IsValid())
					{
						SkeletalMeshPoseTrack->MarkPotentiallyDebugged();

						if(USkeletalMeshComponent* SkeletalMeshComponent = SkeletalMeshPoseTrack->GetComponent(FGameplayTimingViewExtender::GetWorldToVisualize()))
						{
							UAnimInstance* Instance = LazyCreateAnimInstance(SkeletalMeshComponent);
							if(Instance)
							{
								OutDebugList.Emplace(Instance, FText::Format(LOCTEXT("PreviewObjectLabel", "Insights - {0}"), FText::FromString(GetName())).ToString());
							}
						}
					}
				}
			}
		}
	}
}

#endif

#if WITH_ENGINE

void FAnimNodesTrack::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(AnimInstance);
}

void FAnimNodesTrack::OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources)
{
	RemoveWorld(InWorld);
}

void FAnimNodesTrack::RemoveWorld(UWorld* InWorld)
{
	if(AnimInstance && AnimInstance->GetWorld() == InWorld)
	{
		AnimInstance = nullptr;
	}
}

#endif

#undef LOCTEXT_NAMESPACE
