// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTrack.h"
#include "Insights/ViewModels/TimingEventsTrack.h"

#if WITH_EDITOR
#include "Animation/AnimBlueprintGeneratedClass.h"
#endif

class FAnimationSharedData;
class FTimingEventSearchParameters;
struct FAnimGraphMessage;
class IAnimationBlueprintEditor;
struct FCustomDebugObject;
class AActor;
class USkeletalMeshComponent;
class UAnimInstance;

class FAnimNodesTrack : public FGameplayTimingEventsTrack
#if WITH_ENGINE
	, public FGCObject
#endif
{
	INSIGHTS_DECLARE_RTTI(FAnimNodesTrack, FGameplayTimingEventsTrack)

public:
	FAnimNodesTrack(const FAnimationSharedData& InSharedData, uint64 InObjectID, const TCHAR* InName);
	~FAnimNodesTrack();

	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void Draw(const ITimingTrackDrawContext& Context) const override;
	virtual void InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const override;
	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;

#if WITH_EDITOR
	// Update the debug data for any classes that are running on this track
	void UpdateDebugData(const TraceServices::FFrame& InFrame);

	// Get custom debug objects for integration with anim blueprint debugging
	void GetCustomDebugObjects(const IAnimationBlueprintEditor& InAnimationBlueprintEditor, TArray<FCustomDebugObject>& OutDebugList);
#endif

#if WITH_ENGINE
	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("InsightsAnimNodesTrack"); }

	// Handle worlds being torn down
	void OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources);
	void RemoveWorld(UWorld* InWorld);
#endif

private:
	// Helper function used to find an anim graph message
	void FindAnimGraphMessage(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FAnimGraphMessage&)> InFoundPredicate) const;

#if WITH_EDITOR
	UAnimInstance* LazyCreateAnimInstance(USkeletalMeshComponent* InComponent);
#endif

private:
	/** The shared data */
	const FAnimationSharedData& SharedData;

#if WITH_EDITOR
	/** Instance class, if any, used for instantiating debug info */
	TSoftObjectPtr<UAnimBlueprintGeneratedClass> InstanceClass;

	/** Data used for anim BP debugging */
	TObjectPtr<UAnimInstance> AnimInstance;
#endif

#if WITH_ENGINE
	/** Handles used to deal with world switching */
	FDelegateHandle OnWorldCleanupHandle;
	FDelegateHandle OnWorldBeginTearDownHandle;
	FDelegateHandle OnPreWorldFinishDestroyHandle;
#endif
};
