// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNotifiesTrack.h"

#include "AnimationProvider.h"
#include "AnimationSharedData.h"
#include "Common/ProviderLock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameplayProvider.h"
#include "TraceServices/Model/Frames.h"
#include "VariantTreeNode.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/ViewModels/ITimingViewDrawHelper.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"

#define LOCTEXT_NAMESPACE "AnimNotifiesTrack"

INSIGHTS_IMPLEMENT_RTTI(FAnimNotifiesTrack)

FAnimNotifiesTrack::FAnimNotifiesTrack(const FAnimationSharedData& InSharedData, uint64 InObjectID, const TCHAR* InName)
	: FGameplayTimingEventsTrack(InSharedData.GetGameplaySharedData(), InObjectID, FText::Format(LOCTEXT("TrackNameFormat", "Notifies - {0}"), FText::FromString(FString(InName))))
	, SharedData(InSharedData)
{
}

void FAnimNotifiesTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if(AnimationProvider)
	{
		bHasNotifies = false;

		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadNotifyTimeline(GetGameplayTrack().GetObjectId(), [this, &Context, &Builder](const FAnimationProvider::AnimNotifyTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(Context.GetViewport().GetStartTime(), Context.GetViewport().GetEndTime(), [this, &Builder](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNotifyMessage& InMessage)
			{
				Builder.AddEvent(InStartTime, InEndTime, 0, InMessage.Name);
				bHasNotifies = true;
				return TraceServices::EEventEnumerate::Continue;
			});
		});

		uint32 Depth = bHasNotifies ? 1 : 0;
		IdToDepthMap.Reset();

		AnimationProvider->EnumerateNotifyStateTimelines(GetGameplayTrack().GetObjectId(), [this, &Context, &Builder, &Depth](uint64 InNotifyId, const FAnimationProvider::AnimNotifyTimeline& InTimeline)
		{
			uint32 DepthToUse = 0;
			if(uint32* FoundDepthPtr = IdToDepthMap.Find(InNotifyId))
			{
				DepthToUse = *FoundDepthPtr;
			}
			else
			{
				IdToDepthMap.Add(InNotifyId, ++Depth);
				DepthToUse = Depth;
			}

			InTimeline.EnumerateEvents(Context.GetViewport().GetStartTime(), Context.GetViewport().GetEndTime(), [&Builder, &DepthToUse](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNotifyMessage& InMessage)
			{
				Builder.AddEvent(InStartTime, InEndTime, DepthToUse, InMessage.Name);
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}

void FAnimNotifiesTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	DrawEvents(Context);
	GetGameplayTrack().DrawHeaderForTimingTrack(Context, *this, false);
}

static const TCHAR* GetMessageType(EAnimNotifyMessageType InType, bool bInForEvent)
{
	switch (InType)
	{
	case EAnimNotifyMessageType::Begin:
	case EAnimNotifyMessageType::End:
	case EAnimNotifyMessageType::Tick:
	{
		static FText StateType = LOCTEXT("AnimNotifyState", "Anim Notify State");
		static FText BeginType = LOCTEXT("AnimNotifyStateBegin", "Anim Notify State (Begin)");
		static FText EndType = LOCTEXT("AnimNotifyStateEnd", "Anim Notify State (End)");
		static FText TickType = LOCTEXT("AnimNotifyStateTick", "Anim Notify State (Tick)");
		if (bInForEvent)
		{
			switch (InType)
			{
			case EAnimNotifyMessageType::Begin:
				return *BeginType.ToString();
			case EAnimNotifyMessageType::End:
				return *EndType.ToString();
			case EAnimNotifyMessageType::Tick:
				return *TickType.ToString();
			}
		}
		else
		{
			return *StateType.ToString();
		}
	}
	case EAnimNotifyMessageType::Event:
	{
		static FText Type = LOCTEXT("AnimNotify", "Anim Notify");
		return *Type.ToString();
	}
	case EAnimNotifyMessageType::SyncMarker:
	{
		static FText Type = LOCTEXT("SyncMarker", "Sync Marker");
		return *Type.ToString();
	}
	}

	static FText UnknownType = LOCTEXT("UnknownType", "Unknown");
	return *UnknownType.ToString();
}

void FAnimNotifiesTrack::InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const
{
	auto EventFilter = [&HoveredTimingEvent](double EventStartTime, double EventEndTime, uint32 EventDepth) 
	{ 
		return HoveredTimingEvent.GetDepth() == EventDepth;
	};

	FTimingEventSearchParameters SearchParameters(HoveredTimingEvent.GetStartTime(), HoveredTimingEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch, EventFilter);

	FindAnimNotifyMessage(SearchParameters, [this, &Tooltip](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FAnimNotifyMessage& InMessage)
	{
		Tooltip.ResetContent();

		Tooltip.AddTitle(GetName());

		Tooltip.AddNameValueTextLine(LOCTEXT("NotifyType", "Type").ToString(), GetMessageType(InMessage.NotifyEventType, InFoundDepth == 0));

		{
			const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
			if(AnimationProvider)
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
				const TCHAR* Name = AnimationProvider->GetName(InMessage.NameId);
				Tooltip.AddNameValueTextLine(LOCTEXT("NotifyName", "Name").ToString(), Name);
			}
		}

		Tooltip.AddNameValueTextLine(LOCTEXT("EventDuration", "Duration").ToString(), UE::Insights::FormatTimeAuto(InFoundEndTime - InFoundStartTime));
		Tooltip.AddNameValueTextLine(LOCTEXT("EventTime", "Time").ToString(), UE::Insights::FormatTimeAuto(InFoundStartTime));

		if(InMessage.NotifyEventType != EAnimNotifyMessageType::SyncMarker)
		{
			const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
			if(GameplayProvider)
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
				const FObjectInfo& AssetInfo = GameplayProvider->GetObjectInfo(InMessage.AssetId);
				Tooltip.AddNameValueTextLine(LOCTEXT("Asset", "Asset").ToString(), AssetInfo.PathName);

				const FObjectInfo& NotifyInfo = GameplayProvider->GetObjectInfo(InMessage.NotifyId);
				const FClassInfo& NotifyClassInfo = GameplayProvider->GetClassInfo(NotifyInfo.ClassId);
				bool bIsState = InMessage.NotifyEventType == EAnimNotifyMessageType::Begin || InMessage.NotifyEventType == EAnimNotifyMessageType::End || InMessage.NotifyEventType == EAnimNotifyMessageType::Tick;
				Tooltip.AddNameValueTextLine((bIsState ? LOCTEXT("Notify State Class", "Notify State Class") : LOCTEXT("NotifyClass", "Notify Class")).ToString(), NotifyClassInfo.PathName);
			}
		}

		Tooltip.AddNameValueTextLine(LOCTEXT("EventWorld", "World").ToString(), GetGameplayTrack().GetWorldName(SharedData.GetAnalysisSession()).ToString());

		Tooltip.UpdateLayout();
	});
}

const TSharedPtr<const ITimingEvent> FAnimNotifiesTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	FindAnimNotifyMessage(InSearchParameters, [this, &FoundEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FAnimNotifyMessage& InFoundMessage)
	{
		FoundEvent = MakeShared<const FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth);
	});

	return FoundEvent;
}

void FAnimNotifiesTrack::FindAnimNotifyMessage(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FAnimNotifyMessage&)> InFoundPredicate) const
{
	TTimingEventSearch<FAnimNotifyMessage>::Search(
		InParameters,

		[this](TTimingEventSearch<FAnimNotifyMessage>::FContext& InContext)
		{
			const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

			if(AnimationProvider)
			{
				if(bHasNotifies)
				{
					TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
					AnimationProvider->ReadNotifyTimeline(GetGameplayTrack().GetObjectId(), [this, &InContext](const FAnimationProvider::AnimNotifyTimeline& InTimeline)
					{
						InTimeline.EnumerateEvents(InContext.GetParameters().StartTime, InContext.GetParameters().EndTime, [this, &InContext](double InEventStartTime, double InEventEndTime, uint32 InDepth, const FAnimNotifyMessage& InMessage)
						{
							InContext.Check(InEventStartTime, InEventEndTime, 0, InMessage);
							return TraceServices::EEventEnumerate::Continue;
						});
					});
				}

				if(IdToDepthMap.Num() > 0)
				{
					TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
					AnimationProvider->EnumerateNotifyStateTimelines(GetGameplayTrack().GetObjectId(), [this, &InContext](uint32 InNotifyNameId, const FAnimationProvider::AnimNotifyTimeline& InTimeline)
					{
						InTimeline.EnumerateEvents(InContext.GetParameters().StartTime, InContext.GetParameters().EndTime, [this, &InContext, InNotifyNameId](double InEventStartTime, double InEventEndTime, uint32 InDepth, const FAnimNotifyMessage& InMessage)
						{
							if(const uint32* FoundDepthPtr = IdToDepthMap.Find(InMessage.NotifyId))
							{
								InContext.Check(InEventStartTime, InEventEndTime, *FoundDepthPtr, InMessage);
							}
							return TraceServices::EEventEnumerate::Continue;
						});
					});
				}
			}
		},

		[&InFoundPredicate](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FAnimNotifyMessage& InEvent)
		{
			InFoundPredicate(InFoundStartTime, InFoundEndTime, InFoundDepth, InEvent);
		});
}

void FAnimNotifiesTrack::GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const 
{
	TSharedRef<FVariantTreeNode> Header = OutVariants.Add_GetRef(FVariantTreeNode::MakeHeader(LOCTEXT("AnimNotifiesHeader", "Notifies and Sync Markers"), 0));

	const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(SharedData.GetAnalysisSession());
	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
	const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);

	if(AnimationProvider && GameplayProvider)
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);

		bool bForEvent = true;
		auto ProcessEvent = [this, &AnimationProvider, &GameplayProvider, &Header, &bForEvent](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNotifyMessage& InMessage)
		{
			const TCHAR* Name = AnimationProvider->GetName(InMessage.NameId);
			TSharedRef<FVariantTreeNode> NotifyHeader = Header->AddChild(FVariantTreeNode::MakeHeader(FText::FromString(Name), InMessage.NameId));

			NotifyHeader->AddChild(FVariantTreeNode::MakeString(LOCTEXT("EventType", "Type"), GetMessageType(InMessage.NotifyEventType, bForEvent)));
			NotifyHeader->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("EventDuration", "Duration"), InEndTime - InStartTime));
			NotifyHeader->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("EventTime", "Time"), InStartTime));

			if (InMessage.NotifyEventType != EAnimNotifyMessageType::SyncMarker)
			{
				NotifyHeader->AddChild(FVariantTreeNode::MakeObject(LOCTEXT("Asset", "Asset"), InMessage.AssetId));

				uint64 NotifyInfoClassId;
				{
					TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
					const FObjectInfo& NotifyInfo = GameplayProvider->GetObjectInfo(InMessage.NotifyId);
					NotifyInfoClassId = NotifyInfo.ClassId;
				}
				const bool bIsState = InMessage.NotifyEventType == EAnimNotifyMessageType::Begin || InMessage.NotifyEventType == EAnimNotifyMessageType::End || InMessage.NotifyEventType == EAnimNotifyMessageType::Tick;
				NotifyHeader->AddChild(FVariantTreeNode::MakeClass((bIsState ? LOCTEXT("Notify State Class", "Notify State Class") : LOCTEXT("NotifyClass", "Notify Class")), NotifyInfoClassId));
			}

			return TraceServices::EEventEnumerate::Continue;
		};

		AnimationProvider->ReadNotifyTimeline(GetGameplayTrack().GetObjectId(), [&InFrame, &ProcessEvent](const FAnimationProvider::AnimNotifyTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, ProcessEvent);
		});

		bForEvent = false;
		AnimationProvider->EnumerateNotifyStateTimelines(GetGameplayTrack().GetObjectId(), [&InFrame, &ProcessEvent](uint32 InNotifyNameId, const FAnimationProvider::AnimNotifyTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, ProcessEvent);
		});
	}
}

#undef LOCTEXT_NAMESPACE
