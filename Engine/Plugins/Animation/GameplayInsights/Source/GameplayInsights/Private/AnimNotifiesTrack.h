// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTrack.h"
#include "Insights/ViewModels/TimingEventsTrack.h"

class FAnimationSharedData;
class FTimingEventSearchParameters;
struct FAnimNotifyMessage;

class FAnimNotifiesTrack : public FGameplayTimingEventsTrack
{
	INSIGHTS_DECLARE_RTTI(FAnimNotifiesTrack, FGameplayTimingEventsTrack)

public:
	FAnimNotifiesTrack(const FAnimationSharedData& InSharedData, uint64 InObjectID, const TCHAR* InName);

	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void Draw(const ITimingTrackDrawContext& Context) const override;
	virtual void InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const override;
	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;
	virtual void GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const override;

private:
	// Helper function used to find an anim notify message
	void FindAnimNotifyMessage(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FAnimNotifyMessage&)> InFoundPredicate) const;

private:
	// The shared data
	const FAnimationSharedData& SharedData;

	// Map of notify ID->depth of notify states that are currently drawn
	TMap<uint64, uint32> IdToDepthMap;

	// Whether we are currently drawing any notifies
	bool bHasNotifies;
};
