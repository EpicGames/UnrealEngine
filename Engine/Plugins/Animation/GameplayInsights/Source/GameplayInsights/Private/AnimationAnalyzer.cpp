// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationAnalyzer.h"

#include "AnimationProvider.h"
#include "Containers/ArrayView.h"
#include "HAL/LowLevelMemTracker.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Utils.h"

FAnimationAnalyzer::FAnimationAnalyzer(TraceServices::IAnalysisSession& InSession, FAnimationProvider& InAnimationProvider)
	: Session(InSession)
	, AnimationProvider(InAnimationProvider)
{
}

void FAnimationAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_TickRecord, "Animation", "TickRecord");
	Builder.RouteEvent(RouteId_TickRecord2, "Animation", "TickRecord2");
	Builder.RouteEvent(RouteId_TickRecord3, "Animation", "TickRecord3");
	Builder.RouteEvent(RouteId_SkeletalMesh, "Animation", "SkeletalMesh");
	Builder.RouteEvent(RouteId_SkeletalMesh2, "Animation", "SkeletalMesh2");
	Builder.RouteEvent(RouteId_SkeletalMesh3, "Animation", "SkeletalMesh3");
	Builder.RouteEvent(RouteId_SkeletalMeshComponent, "Animation", "SkeletalMeshComponent");
	Builder.RouteEvent(RouteId_SkeletalMeshComponent2, "Animation", "SkeletalMeshComponent2");
	Builder.RouteEvent(RouteId_SkeletalMeshComponent3, "Animation", "SkeletalMeshComponent3");
	Builder.RouteEvent(RouteId_SkeletalMeshComponent4, "Animation", "SkeletalMeshComponent4");
	Builder.RouteEvent(RouteId_SkeletalMeshFrame, "Animation", "SkeletalMeshFrame");
	Builder.RouteEvent(RouteId_AnimGraph, "Animation", "AnimGraph");
	Builder.RouteEvent(RouteId_AnimNodeStart, "Animation", "AnimNodeStart");
	Builder.RouteEvent(RouteId_AnimNodeAttribute, "Animation", "AnimNodeAttribute");
	Builder.RouteEvent(RouteId_AnimNodeValueBool, "Animation", "AnimNodeValueBool");
	Builder.RouteEvent(RouteId_AnimNodeValueInt, "Animation", "AnimNodeValueInt");
	Builder.RouteEvent(RouteId_AnimNodeValueFloat, "Animation", "AnimNodeValueFloat");
	Builder.RouteEvent(RouteId_AnimNodeValueVector2D, "Animation", "AnimNodeValueVector2D");
	Builder.RouteEvent(RouteId_AnimNodeValueVector, "Animation", "AnimNodeValueVector");
	Builder.RouteEvent(RouteId_AnimNodeValueString, "Animation", "AnimNodeValueString");
	Builder.RouteEvent(RouteId_AnimNodeValueObject, "Animation", "AnimNodeValueObject");
	Builder.RouteEvent(RouteId_AnimNodeValueClass, "Animation", "AnimNodeValueClass");
	Builder.RouteEvent(RouteId_AnimNodeValueAnimNode, "Animation", "AnimNodeValueAnimNode");
	Builder.RouteEvent(RouteId_AnimSequencePlayer, "Animation", "AnimSequencePlayer");
	Builder.RouteEvent(RouteId_BlendSpacePlayer, "Animation", "BlendSpacePlayer");
	Builder.RouteEvent(RouteId_StateMachineState, "Animation", "StateMachineState");
	Builder.RouteEvent(RouteId_Name, "Animation", "Name");
	Builder.RouteEvent(RouteId_Notify, "Animation", "Notify");
	Builder.RouteEvent(RouteId_Notify2, "Animation", "Notify2");
	Builder.RouteEvent(RouteId_SyncMarker, "Animation", "SyncMarker");
	Builder.RouteEvent(RouteId_SyncMarker2, "Animation", "SyncMarker2");
	Builder.RouteEvent(RouteId_Montage, "Animation", "Montage");
	Builder.RouteEvent(RouteId_Montage2, "Animation", "Montage2");
	Builder.RouteEvent(RouteId_Sync, "Animation", "Sync");
	Builder.RouteEvent(RouteId_PoseWatch, "Animation", "PoseWatch");
	Builder.RouteEvent(RouteId_PoseWatch2, "Animation", "PoseWatch2");
	Builder.RouteEvent(RouteId_Inertialization, "Animation", "Inertialization");
}

bool FAnimationAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	using namespace TraceServices;

	LLM_SCOPE_BYNAME(TEXT("Insights/FAnimationAnalyzer"));

	FProviderEditScopeLock ProviderEditScope(AnimationProvider);

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
	case RouteId_TickRecord3:
	{
		double RecordingTime = EventData.GetValue<double>("RecordingTime");
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		uint64 AssetId = EventData.GetValue<uint64>("AssetId");
		int32 NodeId = EventData.GetValue<int32>("NodeId");
		float BlendWeight = EventData.GetValue<float>("BlendWeight");
		float PlaybackTime = EventData.GetValue<float>("PlaybackTime");
		float RootMotionWeight = EventData.GetValue<float>("RootMotionWeight");
		float PlayRate = EventData.GetValue<float>("PlayRate");
		float BlendSpacePositionX = EventData.GetValue<float>("BlendSpacePositionX");
		float BlendSpacePositionY = EventData.GetValue<float>("BlendSpacePositionY");
		float BlendSpaceFilteredPositionX = EventData.GetValue<float>("BlendSpaceFilteredPositionX");
		float BlendSpaceFilteredPositionY = EventData.GetValue<float>("BlendSpaceFilteredPositionY");
		uint16 FrameCounter = EventData.GetValue<uint16>("FrameCounter");
		bool bLooping = EventData.GetValue<bool>("Looping");
		bool bIsBlendSpace = EventData.GetValue<bool>("IsBlendSpace");
		uint32 SyncGroupNameId = EventData.GetValue<uint32>("SyncGroupNameId");
		bool IsSyncGroupLeader = EventData.GetValue<bool>("IsSyncGroupLeader");
		float SyncLeaderScore = EventData.GetValue<float>("SyncLeaderScore");
		float TimeToPrevMarker = EventData.GetValue<float>("TimeToPrevMarker");
		float TimeToNextMarker = EventData.GetValue<float>("TimeToNextMarker");
		int32 PrevMarkerIndex = EventData.GetValue<int32>("PrevMarkerIndex");
		int32 NextMarkerIndex = EventData.GetValue<int32>("NextMarkerIndex");
			
		AnimationProvider.AppendTickRecord(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, AssetId, NodeId, BlendWeight, PlaybackTime, RootMotionWeight, PlayRate, BlendSpacePositionX, BlendSpacePositionY, BlendSpaceFilteredPositionX, BlendSpaceFilteredPositionY, FrameCounter, bLooping, bIsBlendSpace, SyncGroupNameId, IsSyncGroupLeader, SyncLeaderScore, TimeToPrevMarker, TimeToNextMarker, PrevMarkerIndex, NextMarkerIndex);
		break;
	}
	case RouteId_TickRecord2:
	{
		double RecordingTime = EventData.GetValue<double>("RecordingTime");
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		uint64 AssetId = EventData.GetValue<uint64>("AssetId");
		int32 NodeId = EventData.GetValue<int32>("NodeId");
		float BlendWeight = EventData.GetValue<float>("BlendWeight");
		float PlaybackTime = EventData.GetValue<float>("PlaybackTime");
		float RootMotionWeight = EventData.GetValue<float>("RootMotionWeight");
		float PlayRate = EventData.GetValue<float>("PlayRate");
		float BlendSpacePositionX = EventData.GetValue<float>("BlendSpacePositionX");
		float BlendSpacePositionY = EventData.GetValue<float>("BlendSpacePositionY");
		float BlendSpaceFilteredPositionX = EventData.GetValue<float>("BlendSpaceFilteredPositionX");
		float BlendSpaceFilteredPositionY = EventData.GetValue<float>("BlendSpaceFilteredPositionY");
		uint16 FrameCounter = EventData.GetValue<uint16>("FrameCounter");
		bool bLooping = EventData.GetValue<bool>("Looping");
		bool bIsBlendSpace = EventData.GetValue<bool>("IsBlendSpace");
		AnimationProvider.AppendTickRecord(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, AssetId, NodeId, BlendWeight, PlaybackTime, RootMotionWeight, PlayRate, BlendSpacePositionX, BlendSpacePositionY, BlendSpaceFilteredPositionX, BlendSpaceFilteredPositionY, FrameCounter, bLooping, bIsBlendSpace);
		break;
	}
        	
	case RouteId_TickRecord:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		uint64 AssetId = EventData.GetValue<uint64>("AssetId");
		int32 NodeId = EventData.GetValue<int32>("NodeId");
		float BlendWeight = EventData.GetValue<float>("BlendWeight");
		float PlaybackTime = EventData.GetValue<float>("PlaybackTime");
		float RootMotionWeight = EventData.GetValue<float>("RootMotionWeight");
		float PlayRate = EventData.GetValue<float>("PlayRate");
		float BlendSpacePositionX = EventData.GetValue<float>("BlendSpacePositionX");
		float BlendSpacePositionY = EventData.GetValue<float>("BlendSpacePositionY");
		float BlendSpaceFilteredPositionX = EventData.GetValue<float>("BlendSpaceFilteredPositionX");
		float BlendSpaceFilteredPositionY = EventData.GetValue<float>("BlendSpaceFilteredPositionY");
		uint16 FrameCounter = EventData.GetValue<uint16>("FrameCounter");
		bool bLooping = EventData.GetValue<bool>("Looping");
		bool bIsBlendSpace = EventData.GetValue<bool>("IsBlendSpace");
		AnimationProvider.AppendTickRecord(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), 0.0, AssetId, NodeId, BlendWeight, PlaybackTime, RootMotionWeight, PlayRate, BlendSpacePositionX, BlendSpacePositionY, BlendSpaceFilteredPositionX, BlendSpaceFilteredPositionY, FrameCounter, bLooping, bIsBlendSpace);
		break;
	}
	case RouteId_SkeletalMesh:
	{
		uint64 Id = EventData.GetValue<uint64>("Id");
		uint32 BoneCount = EventData.GetValue<uint32>("BoneCount");
		TArrayView<const int32> ParentIndices(reinterpret_cast<const int32*>(EventData.GetAttachment()), BoneCount);
		uint64 SkeletonId = 0;
		AnimationProvider.AppendSkeletalMesh(Id, SkeletonId, ParentIndices);
		break;
	}
	case RouteId_SkeletalMesh2:
	{
		uint64 Id = EventData.GetValue<uint64>("Id");
		uint32 BoneCount = EventData.GetValue<uint32>("BoneCount");
		TArrayView<const int32> ParentIndices = EventData.GetArrayView<int32>("ParentIndices");
		uint64 SkeletonId = 0;
		AnimationProvider.AppendSkeletalMesh(Id, SkeletonId, ParentIndices);
		break;
	}
	case RouteId_SkeletalMesh3:
	{
		uint64 Id = EventData.GetValue<uint64>("Id");
		uint32 BoneCount = EventData.GetValue<uint32>("BoneCount");
		TArrayView<const int32> ParentIndices = EventData.GetArrayView<int32>("ParentIndices");
		uint64 SkeletonId = EventData.GetValue<uint64>("SkeletonId");
		AnimationProvider.AppendSkeletalMesh(Id, SkeletonId, ParentIndices);
		break;
	}
	case RouteId_SkeletalMeshComponent:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 ComponentId = EventData.GetValue<uint64>("ComponentId");
		uint64 MeshId = EventData.GetValue<uint64>("MeshId");
		uint32 BoneCount = EventData.GetValue<uint32>("BoneCount");
		uint32 CurveCount = EventData.GetValue<uint32>("CurveCount");
		uint16 FrameCounter = EventData.GetValue<uint16>("FrameCounter");
		uint16 LodIndex = EventData.GetValue<uint16>("LodIndex");
		TArrayView<const FTransform> Pose(reinterpret_cast<const FTransform*>(EventData.GetAttachment()), BoneCount);
		TArrayView<const FSkeletalMeshNamedCurve> Curves(reinterpret_cast<const FSkeletalMeshNamedCurve*>(EventData.GetAttachment() + (sizeof(FTransform) * BoneCount)), CurveCount);
		AnimationProvider.AppendSkeletalMeshComponent(ComponentId, MeshId, Context.EventTime.AsSeconds(Cycle), 0.0, LodIndex, FrameCounter, Pose, Curves);
		break;
	}
	case RouteId_SkeletalMeshComponent2:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 ComponentId = EventData.GetValue<uint64>("ComponentId");
		uint64 MeshId = EventData.GetValue<uint64>("MeshId");
		uint32 BoneCount = EventData.GetValue<uint32>("BoneCount");
		uint32 CurveCount = EventData.GetValue<uint32>("CurveCount");
		uint16 FrameCounter = EventData.GetValue<uint16>("FrameCounter");
		uint16 LodIndex = EventData.GetValue<uint16>("LodIndex");

		TArrayView<const float> ComponentToWorldFloatArray = EventData.GetArrayView<float>("ComponentToWorld");
		TArrayView<const float> PoseFloatArray = EventData.GetArrayView<float>("Pose");
		TArrayView<const uint32> CurveIds = EventData.GetArrayView<uint32>("CurveIds");
		TArrayView<const float> CurveValues = EventData.GetArrayView<float>("CurveValues");
		check(CurveIds.Num() == CurveValues.Num());

		AnimationProvider.AppendSkeletalMeshComponent(ComponentId, MeshId, Context.EventTime.AsSeconds(Cycle), 0.0,  LodIndex, FrameCounter, ComponentToWorldFloatArray, PoseFloatArray, CurveIds, CurveValues);
		break;
	}
	case RouteId_SkeletalMeshComponent3:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double RecordingTime = EventData.GetValue<double>("RecordingTime");
		
		uint64 ComponentId = EventData.GetValue<uint64>("ComponentId");
		uint64 MeshId = EventData.GetValue<uint64>("MeshId");
		uint32 BoneCount = EventData.GetValue<uint32>("BoneCount");
		uint32 CurveCount = EventData.GetValue<uint32>("CurveCount");
		uint16 FrameCounter = EventData.GetValue<uint16>("FrameCounter");
		uint16 LodIndex = EventData.GetValue<uint16>("LodIndex");

		TArrayView<const float> ComponentToWorldFloatArray = EventData.GetArrayView<float>("ComponentToWorld");
		TArrayView<const float> PoseFloatArray = EventData.GetArrayView<float>("Pose");
		TArrayView<const uint32> CurveIds = EventData.GetArrayView<uint32>("CurveIds");
		TArrayView<const float> CurveValues = EventData.GetArrayView<float>("CurveValues");
		check(CurveIds.Num() == CurveValues.Num());

		AnimationProvider.AppendSkeletalMeshComponent(ComponentId, MeshId, Context.EventTime.AsSeconds(Cycle), RecordingTime, LodIndex, FrameCounter, ComponentToWorldFloatArray, PoseFloatArray, CurveIds, CurveValues);
		break;
	}
	case RouteId_SkeletalMeshComponent4:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double RecordingTime = EventData.GetValue<double>("RecordingTime");
		bool bIsVisible = EventData.GetValue<bool>("IsVisible", true);	
		uint64 ComponentId = EventData.GetValue<uint64>("ComponentId");
		uint64 MeshId = EventData.GetValue<uint64>("MeshId");
		uint32 BoneCount = EventData.GetValue<uint32>("BoneCount");
		uint32 CurveCount = EventData.GetValue<uint32>("CurveCount");
		uint16 FrameCounter = EventData.GetValue<uint16>("FrameCounter");
		uint16 LodIndex = EventData.GetValue<uint16>("LodIndex");

		TArrayView<const float> ComponentToWorldFloatArray = EventData.GetArrayView<float>("ComponentToWorld");
		TArrayView<const float> PoseFloatArray = EventData.GetArrayView<float>("Pose");
		TArrayView<const uint32> CurveIds = EventData.GetArrayView<uint32>("CurveIds");
		TArrayView<const float> CurveValues = EventData.GetArrayView<float>("CurveValues");
		check(CurveIds.Num() == CurveValues.Num());

		AnimationProvider.AppendSkeletalMeshComponent(
			ComponentId, 
			MeshId,
			Context.EventTime.AsSeconds(Cycle),
			RecordingTime,
			LodIndex,
			FrameCounter,
			bIsVisible,
			ComponentToWorldFloatArray,
			PoseFloatArray,
			CurveIds,
			CurveValues,
			EventData.GetArrayView<float>("ExternalMorphSetWeights"),
			EventData.GetArrayView<int32>("ExternalMorphSetWeightCounts"));
		break;
	}
	case RouteId_Name:
	{
		uint32 Id = EventData.GetValue<uint32>("Id");
		FString Name = FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("Name", Context);
		AnimationProvider.AppendName(Id, *Name);
		break;
	}
	case RouteId_SkeletalMeshFrame:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 ComponentId = EventData.GetValue<uint64>("ComponentId");
		uint16 FrameCounter = EventData.GetValue<uint16>("FrameCounter");
		AnimationProvider.AppendSkeletalMeshFrame(ComponentId, Context.EventTime.AsSeconds(Cycle), FrameCounter);
		break;
	}
	case RouteId_AnimGraph:
	{
		uint64 StartCycle = EventData.GetValue<uint64>("StartCycle");
		uint64 EndCycle = EventData.GetValue<uint64>("EndCycle");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		int32 NodeCount = EventData.GetValue<int32>("NodeCount");
		uint16 FrameCounter = EventData.GetValue<uint16>("FrameCounter");
		uint8 Phase = EventData.GetValue<uint8>("Phase");
		AnimationProvider.AppendAnimGraph(AnimInstanceId, Context.EventTime.AsSeconds(StartCycle), Context.EventTime.AsSeconds(EndCycle), NodeCount, FrameCounter, Phase);
		break;
	}
	case RouteId_AnimNodeStart:
	{
		uint64 StartCycle = EventData.GetValue<uint64>("StartCycle");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		int32 NodeId = EventData.GetValue<int32>("NodeId");
		int32 PreviousNodeId = EventData.GetValue<int32>("PreviousNodeId");
		FString NodeName = FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("Name", Context);
		float Weight = EventData.GetValue<float>("Weight");
		float RootMotionWeight = EventData.GetValue<float>("RootMotionWeight");
		uint16 FrameCounter = EventData.GetValue<uint16>("FrameCounter");
		uint8 Phase = EventData.GetValue<uint8>("Phase");
		FString NodeDisplayName = FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("DisplayName", Context);
		AnimationProvider.AppendAnimNodeStart(AnimInstanceId, Context.EventTime.AsSeconds(StartCycle), FrameCounter, NodeId, PreviousNodeId, *NodeName, Weight, RootMotionWeight, *NodeDisplayName, Phase);
		break;
	}
	case RouteId_AnimNodeAttribute:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 SourceAnimInstanceId = EventData.GetValue<uint64>("SourceAnimInstanceId");
		uint64 TargetAnimInstanceId = EventData.GetValue<uint64>("TargetAnimInstanceId");
		int32 SourceNodeId = EventData.GetValue<int32>("SourceNodeId");
		int32 TargetNodeId = EventData.GetValue<int32>("TargetNodeId");
		uint32 NameId = EventData.GetValue<uint32>("NameId");
		AnimationProvider.AppendAnimGraphAttribute(SourceAnimInstanceId, TargetAnimInstanceId, Context.EventTime.AsSeconds(Cycle), SourceNodeId, TargetNodeId, NameId);
		break;
	}
	case RouteId_AnimNodeValueBool:
	case RouteId_AnimNodeValueInt:
	case RouteId_AnimNodeValueFloat:
	case RouteId_AnimNodeValueVector2D:
	case RouteId_AnimNodeValueVector:
	case RouteId_AnimNodeValueString:
	case RouteId_AnimNodeValueObject:
	case RouteId_AnimNodeValueClass:
	case RouteId_AnimNodeValueAnimNode:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double RecordingTime = EventData.GetValue<double>("RecordingTime");

		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		int32 NodeId = EventData.GetValue<int32>("NodeId");
		uint16 FrameCounter = EventData.GetValue<uint16>("FrameCounter");
		FString Key = FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("Key", Context);

		switch (RouteId)
		{
		case RouteId_AnimNodeValueBool:
		{
			bool Value = EventData.GetValue<bool>("Value");
			AnimationProvider.AppendAnimNodeValue(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, FrameCounter, NodeId, *Key, Value);
			break;
		}
		case RouteId_AnimNodeValueInt:
		{
			int32 Value = EventData.GetValue<int32>("Value");
			AnimationProvider.AppendAnimNodeValue(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, FrameCounter, NodeId, *Key, Value);
			break;
		}
		case RouteId_AnimNodeValueFloat:
		{
			float Value = EventData.GetValue<float>("Value");
			AnimationProvider.AppendAnimNodeValue(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, FrameCounter, NodeId, *Key, Value);
			break;
		}
		case RouteId_AnimNodeValueVector2D:
		{
			FVector2D Value(EventData.GetValue<float>("ValueX"), EventData.GetValue<float>("ValueY"));
			AnimationProvider.AppendAnimNodeValue(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, FrameCounter, NodeId, *Key, Value);
			break;
		}
		case RouteId_AnimNodeValueVector:
		{
			FVector Value(EventData.GetValue<float>("ValueX"), EventData.GetValue<float>("ValueY"), EventData.GetValue<float>("ValueZ"));
			AnimationProvider.AppendAnimNodeValue(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, FrameCounter, NodeId, *Key, Value);
			break;
		}
		case RouteId_AnimNodeValueString:
		{
			FString Value;
			if (!EventData.GetString("Value", Value))
			{
				int32 KeyLength = EventData.GetValue<uint32>("KeyLength");
				Value = reinterpret_cast<const TCHAR*>(EventData.GetAttachment()) + KeyLength;
			}
			AnimationProvider.AppendAnimNodeValue(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, FrameCounter, NodeId, *Key, *Value);
			break;
		}
		case RouteId_AnimNodeValueObject:
		{
			uint64 Value = EventData.GetValue<uint64>("Value");
			AnimationProvider.AppendAnimNodeValueObject(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, FrameCounter, NodeId, *Key, Value);
			break;
		}
		case RouteId_AnimNodeValueClass:
		{
			uint64 Value = EventData.GetValue<uint64>("Value");
			AnimationProvider.AppendAnimNodeValueClass(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, FrameCounter, NodeId, *Key, Value);
			break;
		}
		case RouteId_AnimNodeValueAnimNode:
		{
			int32 Value = EventData.GetValue<int32>("Value");
			uint64 ValueAnimInstanceId = EventData.GetValue<uint64>("ValueAnimInstanceId");
			AnimationProvider.AppendAnimNodeValueAnimNode(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, FrameCounter, NodeId, *Key, Value, ValueAnimInstanceId);
			break;
		}
		}
		break;
	}
	case RouteId_AnimSequencePlayer:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		int32 NodeId = EventData.GetValue<int32>("NodeId");
		float Position = EventData.GetValue<float>("Position");
		float Length = EventData.GetValue<float>("Length");
		uint16 FrameCounter = EventData.GetValue<uint16>("FrameCounter");
		AnimationProvider.AppendAnimSequencePlayer(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), NodeId, Position, Length, FrameCounter);
		break;
	}
	case RouteId_BlendSpacePlayer:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		uint64 BlendSpaceId = EventData.GetValue<uint64>("BlendSpaceId");
		int32 NodeId = EventData.GetValue<int32>("NodeId");
		FVector BlendPosition(EventData.GetValue<float>("PositionX"), EventData.GetValue<float>("PositionY"), EventData.GetValue<float>("PositionZ"));
		FVector FilteredBlendPosition(EventData.GetValue<float>("FilteredPositionX"), EventData.GetValue<float>("FilteredPositionY"), EventData.GetValue<float>("FilteredPositionZ"));
		AnimationProvider.AppendBlendSpacePlayer(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), NodeId, BlendSpaceId, BlendPosition, FilteredBlendPosition);
		break;
	}
	case RouteId_StateMachineState:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		int32 NodeId = EventData.GetValue<int32>("NodeId");
		int32 StateMachineIndex = EventData.GetValue<int32>("StateMachineIndex");
		int32 StateIndex = EventData.GetValue<int32>("StateIndex");
		float StateWeight = EventData.GetValue<float>("StateWeight");
		float ElapsedTime = EventData.GetValue<float>("ElapsedTime");
		AnimationProvider.AppendStateMachineState(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), NodeId, StateMachineIndex, StateIndex, StateWeight, ElapsedTime);
		break;
	}
	case RouteId_Notify:
		{
			uint64 Cycle = EventData.GetValue<uint64>("Cycle");
			uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
			uint64 AssetId = EventData.GetValue<uint64>("AssetId");
			uint64 NotifyId = EventData.GetValue<uint64>("NotifyId");
			uint32 NameId = EventData.GetValue<uint32>("NameId");
			float Time = EventData.GetValue<float>("Time");
			float Duration = EventData.GetValue<float>("Duration");
			uint8 NotifyEventType = EventData.GetValue<uint8>("NotifyEventType");
			AnimationProvider.AppendNotify(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), 0.0, AssetId, NotifyId, NameId, Time, Duration, (EAnimNotifyMessageType)NotifyEventType);
			break;
		}
	case RouteId_Notify2:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double RecordingTime = EventData.GetValue<double>("RecordingTime");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		uint64 AssetId = EventData.GetValue<uint64>("AssetId");
		uint64 NotifyId = EventData.GetValue<uint64>("NotifyId");
		uint32 NameId = EventData.GetValue<uint32>("NameId");
		float Time = EventData.GetValue<float>("Time");
		float Duration = EventData.GetValue<float>("Duration");
		uint8 NotifyEventType = EventData.GetValue<uint8>("NotifyEventType");
		AnimationProvider.AppendNotify(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, AssetId, NotifyId, NameId, Time, Duration, (EAnimNotifyMessageType)NotifyEventType);
		break;
	}
	case RouteId_SyncMarker:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		uint32 NameId = EventData.GetValue<uint32>("NameId");
		AnimationProvider.AppendNotify(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), 0.0, 0, 0, NameId, 0.0f, 0.0f, EAnimNotifyMessageType::SyncMarker);
		break;
	}
	case RouteId_SyncMarker2:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double RecordingTime = EventData.GetValue<double>("RecordingTime");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		uint32 NameId = EventData.GetValue<uint32>("NameId");
		AnimationProvider.AppendNotify(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, 0, 0, NameId, 0.0f, 0.0f, EAnimNotifyMessageType::SyncMarker);
		break;
	}
	case RouteId_Montage:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		uint64 MontageId = EventData.GetValue<uint64>("MontageId");
		uint32 CurrentSectionNameId = EventData.GetValue<uint32>("CurrentSectionNameId");
		uint32 NextSectionNameId = EventData.GetValue<uint32>("NextSectionNameId");
		float Weight = EventData.GetValue<float>("Weight");
		float DesiredWeight = EventData.GetValue<float>("DesiredWeight");
		float Position = EventData.GetValue<float>("Position");
		uint16 FrameCounter = EventData.GetValue<uint16>("FrameCounter");
		AnimationProvider.AppendMontage(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), 0.0, MontageId, CurrentSectionNameId, NextSectionNameId, Weight, DesiredWeight, Position, FrameCounter);
		break;
	}
	case RouteId_Montage2:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double RecordingTime = EventData.GetValue<double>("RecordingTime");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		uint64 MontageId = EventData.GetValue<uint64>("MontageId");
		uint32 CurrentSectionNameId = EventData.GetValue<uint32>("CurrentSectionNameId");
		uint32 NextSectionNameId = EventData.GetValue<uint32>("NextSectionNameId");
		float Weight = EventData.GetValue<float>("Weight");
		float DesiredWeight = EventData.GetValue<float>("DesiredWeight");
		float Position = EventData.GetValue<float>("Position");
		uint16 FrameCounter = EventData.GetValue<uint16>("FrameCounter");
		AnimationProvider.AppendMontage(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, MontageId, CurrentSectionNameId, NextSectionNameId, Weight, DesiredWeight, Position, FrameCounter);
		break;
	}
	case RouteId_Sync:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		int32 SourceNodeId = EventData.GetValue<int32>("SourceNodeId");
		uint32 GroupNameId = EventData.GetValue<uint32>("GroupNameId");
		AnimationProvider.AppendSync(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), SourceNodeId, GroupNameId);
		break;
	}
	case RouteId_PoseWatch:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double RecordingTime = EventData.GetValue<double>("RecordingTime");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		uint64 PoseWatchId = EventData.GetValue<uint64>("PoseWatchId");
		bool bIsEnabled = EventData.GetValue<bool>("bIsEnabled");
		TArrayView<const float> WorldTransformFloatArray = EventData.GetArrayView<float>("WorldTransform");
		TArrayView<const uint16> RequiredBonesIntArray = EventData.GetArrayView<uint16>("RequiredBones");
		TArrayView<const float> BoneTransformsFloatArray = EventData.GetArrayView<float>("BoneTransforms");
		AnimationProvider.AppendPoseWatch(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, PoseWatchId, BoneTransformsFloatArray, RequiredBonesIntArray, WorldTransformFloatArray, bIsEnabled);
		break;
	}
	case RouteId_PoseWatch2:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double RecordingTime = EventData.GetValue<double>("RecordingTime");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		uint64 ComponentId = EventData.GetValue<uint64>("ComponentId");
		uint64 PoseWatchId = EventData.GetValue<uint64>("PoseWatchId");
		uint32 NameId = EventData.GetValue<uint32>("NameId");
		uint32 Color = EventData.GetValue<uint32>("Color");
		bool bIsEnabled = EventData.GetValue<bool>("bIsEnabled");
		TArrayView<const float> WorldTransformFloatArray = EventData.GetArrayView<float>("WorldTransform");
		TArrayView<const uint16> RequiredBonesIntArray = EventData.GetArrayView<uint16>("RequiredBones");
		TArrayView<const float> BoneTransformsFloatArray = EventData.GetArrayView<float>("BoneTransforms");
		TArrayView<const uint32> CurveIds = EventData.GetArrayView<uint32>("CurveIds");
		TArrayView<const float> CurveValues = EventData.GetArrayView<float>("CurveValues");
		check(CurveIds.Num() == CurveValues.Num());

		AnimationProvider.AppendPoseWatch(ComponentId, AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, PoseWatchId, NameId, FColor(Color), BoneTransformsFloatArray, CurveIds, CurveValues, RequiredBonesIntArray, WorldTransformFloatArray, bIsEnabled);
		break;
	}
	case RouteId_Inertialization:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double RecordingTime = EventData.GetValue<double>("RecordingTime");
		uint64 AnimInstanceId = EventData.GetValue<uint64>("AnimInstanceId");
		int32 NodeId = EventData.GetValue<int32>("NodeId");
		float Weight = EventData.GetValue<float>("Weight");
		EInertializationType Type = static_cast<EInertializationType>(EventData.GetValue<uint8>("Type"));	
			
		AnimationProvider.AppendInertialization(AnimInstanceId, Context.EventTime.AsSeconds(Cycle), RecordingTime, NodeId, Weight, Type);
		break;
	}
	}

	return true;
}