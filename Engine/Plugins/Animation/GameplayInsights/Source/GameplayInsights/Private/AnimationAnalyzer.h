// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "HAL/IConsoleManager.h"

class FAnimationProvider;
namespace TraceServices { class IAnalysisSession; }

class FAnimationAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FAnimationAnalyzer(TraceServices::IAnalysisSession& InSession, FAnimationProvider& InAnimationProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override {}
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_TickRecord,
		RouteId_TickRecord2,
		RouteId_TickRecord3,
		RouteId_SkeletalMesh,
		RouteId_SkeletalMesh2,
		RouteId_SkeletalMesh3,
		RouteId_SkeletalMeshComponent,
		RouteId_SkeletalMeshComponent2,
		RouteId_SkeletalMeshComponent3,
		RouteId_SkeletalMeshComponent4,
		RouteId_SkeletalMeshFrame,
		RouteId_AnimGraph,
		RouteId_AnimNodeStart,
		RouteId_AnimNodeAttribute,
		RouteId_AnimNodeValueBool,
		RouteId_AnimNodeValueInt,
		RouteId_AnimNodeValueFloat,
		RouteId_AnimNodeValueVector2D,
		RouteId_AnimNodeValueVector,
		RouteId_AnimNodeValueString,
		RouteId_AnimNodeValueObject,
		RouteId_AnimNodeValueClass,
		RouteId_AnimNodeValueAnimNode,
		RouteId_AnimSequencePlayer,
		RouteId_BlendSpacePlayer,
		RouteId_StateMachineState,
		RouteId_Name,
		RouteId_Notify,
		RouteId_Notify2,
		RouteId_SyncMarker,
		RouteId_SyncMarker2,
		RouteId_Montage,
		RouteId_Montage2,
		RouteId_Sync,
		RouteId_PoseWatch,
		RouteId_PoseWatch2,
		RouteId_Inertialization,
	};

	TraceServices::IAnalysisSession& Session;
	FAnimationProvider& AnimationProvider;
};
