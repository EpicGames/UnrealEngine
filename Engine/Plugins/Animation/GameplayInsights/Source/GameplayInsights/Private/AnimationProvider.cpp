// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationProvider.h"
#include "GameplayProvider.h"
#include "Insights/ViewModels/TimingEventsTrack.h"
#include "AnimationAnalyzer.h"

namespace TraceServices
{
thread_local FProviderLock::FThreadLocalState GAnimationProviderLockState;
}

#define LOCTEXT_NAMESPACE "AnimationProvider"

FName FAnimationProvider::ProviderName("AnimationProvider");

FAnimationProvider::FAnimationProvider(TraceServices::IAnalysisSession& InSession, FGameplayProvider& InGameplayProvider)
	: Session(InSession)
	, GameplayProvider(InGameplayProvider)
	, SkeletalMeshPoseTransforms(InSession.GetLinearAllocator(), 256)
	, SkeletalMeshCurves(InSession.GetLinearAllocator(), 256)
	, ExternalMorphWeights(InSession.GetLinearAllocator(), 256)
	, SkeletalMeshParentIndices(InSession.GetLinearAllocator(), 256)
	, PoseWatchRequiredBones(InSession.GetLinearAllocator(), 256)
	, bHasAnyData(false)
{
	GameplayProvider.OnObjectEndPlay().AddRaw(this, &FAnimationProvider::HandleObjectEndPlay);
}

void FAnimationProvider::UpdateSessionDurationSeconds(const double InTime) const
{
	TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
	Session.UpdateDurationSeconds(InTime);
}

void FAnimationProvider::EnumerateSkeletalMeshPoseTimelines(TFunctionRef<void(uint64 ObjectId, const SkeletalMeshPoseTimeline&)> Callback) const
{
	ReadAccessCheck();
	
	for(auto& IndexMapping : ObjectIdToSkeletalMeshPoseTimelines)
	{
		const TSharedPtr<FSkeletalMeshTimelineStorage>& TimelineStorage = SkeletalMeshPoseTimelineStorage[IndexMapping.Value];
		if (TimelineStorage->Timeline.IsValid())
		{
			Callback(IndexMapping.Key, *TimelineStorage->Timeline);
		}
	}
}

bool FAnimationProvider::ReadSkeletalMeshPoseTimeline(uint64 InObjectId, TFunctionRef<void(const SkeletalMeshPoseTimeline&, bool)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToSkeletalMeshPoseTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(SkeletalMeshPoseTimelineStorage.Num()))
		{
			const TSharedRef<FSkeletalMeshTimelineStorage>& SkeletalMeshTimelineStorage = SkeletalMeshPoseTimelineStorage[*IndexPtr];
			Callback(*SkeletalMeshTimelineStorage->Timeline, SkeletalMeshTimelineStorage->AllCurveIds.Num() > 0);
			return true;
		}
	}

	return false;
}

void FAnimationProvider::GetSkeletalMeshComponentSpacePose(const FSkeletalMeshPoseMessage& InMessage, const FSkeletalMeshInfo& InMeshInfo, FTransform& OutComponentToWorld, TArray<FTransform>& OutTransforms) const
{
	ReadAccessCheck();

	if(InMeshInfo.BoneCount == InMessage.NumTransforms)
	{
		// Pre-alloc array
		OutTransforms.SetNumUninitialized(InMessage.NumTransforms);

		// First transform is always component to world
		OutComponentToWorld = InMessage.ComponentToWorld;
		uint64 StartTransformIndex = InMessage.TransformStartIndex;
		uint64 EndTransformIndex = InMessage.TransformStartIndex + InMessage.NumTransforms;
		uint64 SourceTransformIndex;
		int32 TargetTransformIndex = 0;
		for(SourceTransformIndex = StartTransformIndex; SourceTransformIndex < EndTransformIndex; ++SourceTransformIndex, ++TargetTransformIndex)
		{
			OutTransforms[TargetTransformIndex] = SkeletalMeshPoseTransforms[SourceTransformIndex];
		}
	}
}

void FAnimationProvider::GetPoseWatchData(const FPoseWatchMessage& InMessage, TArray<FTransform>& BoneTransforms, TArray<uint16>& RequiredBones) const
{
	ReadAccessCheck();

	// Pre-alloc arrays
	BoneTransforms.Empty();
	BoneTransforms.SetNumUninitialized(InMessage.NumBoneTransforms);

	RequiredBones.Empty();
	RequiredBones.SetNumUninitialized(InMessage.NumRequiredBones);

	// Write into BoneTransforms
	{
		uint64 StartTransformIndex = InMessage.BoneTransformsStartIndex;
		uint64 EndTransformIndex = InMessage.BoneTransformsStartIndex + InMessage.NumBoneTransforms;
		int32 TargetTransformIndex = 0;
		for (uint64 SourceTransformIndex = StartTransformIndex; SourceTransformIndex < EndTransformIndex; ++SourceTransformIndex, ++TargetTransformIndex)
		{
			BoneTransforms[TargetTransformIndex] = SkeletalMeshPoseTransforms[SourceTransformIndex];
		}
	}

	// Write into RequiredBones
	{
		uint64 StartIndex = InMessage.RequiredBonesStartIndex;
		uint64 EndIndex = InMessage.RequiredBonesStartIndex + InMessage.NumRequiredBones;
		int32 TargetIndex = 0;
		for (uint64 SourceIndex = StartIndex; SourceIndex < EndIndex; ++SourceIndex, ++TargetIndex)
		{
			RequiredBones[TargetIndex] = PoseWatchRequiredBones[SourceIndex];
		}
	}
}

void FAnimationProvider::EnumerateSkeletalMeshCurveIds(uint64 InObjectId, TFunctionRef<void(uint32)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToSkeletalMeshPoseTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(SkeletalMeshPoseTimelineStorage.Num()))
		{
			const TSharedRef<FSkeletalMeshTimelineStorage>& TimelineStorage = SkeletalMeshPoseTimelineStorage[*IndexPtr];
			for(uint32 Id : TimelineStorage->AllCurveIds)
			{
				Callback(Id);
			}
		}
	}
}

void FAnimationProvider::EnumerateSkeletalMeshCurves(const FSkeletalMeshPoseMessage& InMessage, TFunctionRef<void(const FSkeletalMeshNamedCurve&)> Callback) const
{
	ReadAccessCheck();

	uint64 StartCurveIndex = InMessage.CurveStartIndex;
	uint64 EndCurveIndex = InMessage.CurveStartIndex + InMessage.NumCurves;

	for(uint64 CurveIndex = StartCurveIndex; CurveIndex < EndCurveIndex; ++CurveIndex)
	{
		Callback(SkeletalMeshCurves[CurveIndex]);
	}
}

void FAnimationProvider::EnumeratePoseWatchCurves(const FPoseWatchMessage& InMessage, TFunctionRef<void(const FSkeletalMeshNamedCurve&)> Callback) const
{
	ReadAccessCheck();

	uint64 StartCurveIndex = InMessage.CurveStartIndex;
	uint64 EndCurveIndex = InMessage.CurveStartIndex + InMessage.NumCurves;

	for(uint64 CurveIndex = StartCurveIndex; CurveIndex < EndCurveIndex; ++CurveIndex)
	{
		Callback(SkeletalMeshCurves[CurveIndex]);
	}
}

void FAnimationProvider::EnumerateExternalMorphSets(const FSkeletalMeshPoseMessage& InMessage, TFunctionRef<void(const FExternalMorphWeightMessage&)> Callback) const
{
	ReadAccessCheck();

	const uint64 StartIndex = InMessage.ExternalMorphStartIndex;
	const uint64 EndIndex = StartIndex + InMessage.NumExternalMorphSets;
	for (uint64 Index = StartIndex; Index < EndIndex; ++Index)
	{
		Callback(ExternalMorphWeights[Index]);
	}
}

void FAnimationProvider::EnumerateTickRecordIds(uint64 InObjectId, TFunctionRef<void(uint64, int32)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToTickRecordTimelineStorage.Find(InObjectId);
	if (IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(TickRecordTimelineStorage.Num()))
		{
			const TSharedRef<FTickRecordTimelineStorage>& TimelineStorage = TickRecordTimelineStorage[*IndexPtr];
			for (auto AssetIdPair : TimelineStorage->AssetIdAndPlayers)
			{
				Callback(AssetIdPair.Key, AssetIdPair.Value);
			}
		}
	}
}

bool FAnimationProvider::ReadTickRecordTimeline(uint64 InObjectId, TFunctionRef<void(const TickRecordTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToTickRecordTimelineStorage.Find(InObjectId);
	if (IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(TickRecordTimelineStorage.Num()))
		{
			Callback(*TickRecordTimelineStorage[*IndexPtr]->Timeline.Get());
			return true;
		}
	}

	return false;
}

bool FAnimationProvider::ReadInertializationTimeline(uint64 InObjectId, TFunctionRef<void(const InertializationTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToInertializationTimelines.Find(InObjectId);
	if (IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(InertializationTimelineStorage.Num()))
		{
			Callback(*InertializationTimelineStorage[*IndexPtr]->Timeline.Get());
			return true;
		}
	}

	return false;
}

void FAnimationProvider::EnumerateInertializationNodes(uint64 InObjectId, TFunctionRef<void(int32 NodeId, EInertializationType Type)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToInertializationTimelines.Find(InObjectId);
	if (IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(InertializationTimelineStorage.Num()))
		{
			for(auto& Entry : InertializationTimelineStorage[*IndexPtr]->NodeTypes)
			{
				Callback(Entry.Key, Entry.Value);
			}
		}
	}
}

void FAnimationProvider::EnumerateAnimGraphTimelines(TFunctionRef<void(uint64 ObjectId, const AnimGraphTimeline&)> Callback) const
{
	ReadAccessCheck();
	
	for(auto& IndexMapping : ObjectIdToAnimGraphTimelines)
	{
		Callback(IndexMapping.Key, AnimGraphTimelines[IndexMapping.Value].Get());
	}
}

bool FAnimationProvider::ReadAnimGraphTimeline(uint64 InObjectId, TFunctionRef<void(const AnimGraphTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToAnimGraphTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(AnimGraphTimelines.Num()))
		{
			Callback(*AnimGraphTimelines[*IndexPtr]);
			return true;
		}
	}

	return false;
}

bool FAnimationProvider::ReadAnimNodesTimeline(uint64 InObjectId, TFunctionRef<void(const AnimNodesTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToAnimNodeTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(AnimNodeTimelines.Num()))
		{
			Callback(*AnimNodeTimelines[*IndexPtr]);
			return true;
		}
	}

	return false;
}

bool FAnimationProvider::ReadAnimNodeValuesTimeline(uint64 InObjectId, TFunctionRef<void(const AnimNodeValuesTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToAnimNodeValueTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(AnimNodeValueTimelines.Num()))
		{
			Callback(*AnimNodeValueTimelines[*IndexPtr]);
			return true;
		}
	}

	return false;
}

bool FAnimationProvider::ReadAnimAttributesTimeline(uint64 InObjectId, TFunctionRef<void(const AnimAttributeTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToAnimAttributeTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(AnimAttributeTimelines.Num()))
		{
			Callback(*AnimAttributeTimelines[*IndexPtr]);
			return true;
		}
	}

	return false;
}

bool FAnimationProvider::ReadAnimSequencePlayersTimeline(uint64 InObjectId, TFunctionRef<void(const AnimSequencePlayersTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToAnimSequencePlayerTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(AnimSequencePlayerTimelines.Num()))
		{
			Callback(*AnimSequencePlayerTimelines[*IndexPtr]);
			return true;
		}
	}

	return false;
}

bool FAnimationProvider::ReadAnimBlendSpacePlayersTimeline(uint64 InObjectId, TFunctionRef<void(const BlendSpacePlayersTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToBlendSpacePlayerTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(BlendSpacePlayerTimelines.Num()))
		{
			Callback(*BlendSpacePlayerTimelines[*IndexPtr]);
			return true;
		}
	}

	return false;
}

bool FAnimationProvider::ReadStateMachinesTimeline(uint64 InObjectId, TFunctionRef<void(const StateMachinesTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToStateMachineTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(StateMachineTimelines.Num()))
		{
			Callback(*StateMachineTimelines[*IndexPtr]);
			return true;
		}
	}

	return false;
}

bool FAnimationProvider::ReadNotifyTimeline(uint64 InObjectId, TFunctionRef<void(const AnimNotifyTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToAnimNotifyTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(AnimNotifyTimelines.Num()))
		{
			Callback(*AnimNotifyTimelines[*IndexPtr]);
			return true;
		}
	}

	return false;
}

void FAnimationProvider::EnumerateNotifyStateTimelines(uint64 InObjectId, TFunctionRef<void(uint64, const AnimNotifyTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToAnimNotifyStateTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(AnimNotifyStateTimelineStorage.Num()))
		{
			const TSharedRef<FAnimNotifyStateTimelineStorage>& TimelineStorage = AnimNotifyStateTimelineStorage[*IndexPtr];
			for(auto NotifyIdPair : TimelineStorage->NotifyIdToAnimNotifyStateTimeline)
			{
				Callback(NotifyIdPair.Key, TimelineStorage->Timelines[NotifyIdPair.Value].Get());
			}
		}
	}
}

bool FAnimationProvider::ReadMontageTimeline(uint64 InObjectId, TFunctionRef<void(const AnimMontageTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToAnimMontageTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(AnimMontageTimelineStorage.Num()))
		{
			Callback(*AnimMontageTimelineStorage[*IndexPtr]->Timeline.Get());
			return true;
		}
	}

	return false;
}

void FAnimationProvider::EnumerateMontageIds(uint64 InObjectId, TFunctionRef<void(uint64)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToAnimMontageTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(AnimMontageTimelineStorage.Num()))
		{
			const TSharedRef<FMontageTimelineStorage>& TimelineStorage = AnimMontageTimelineStorage[*IndexPtr];
			for(uint64 Id : TimelineStorage->AllMontageIds)
			{
				Callback(Id);
			}
		}
	}
}

bool FAnimationProvider::ReadAnimSyncTimeline(uint64 InObjectId, TFunctionRef<void(const AnimSyncTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToAnimSyncTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(AnimSyncTimelines.Num()))
		{
			Callback(*AnimSyncTimelines[*IndexPtr]);
			return true;
		}
	}

	return false;
}

bool FAnimationProvider::ReadPoseWatchTimeline(uint64 InObjectId, TFunctionRef<void(const PoseWatchTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToPoseWatchTimelines.Find(InObjectId);
	if (IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(PoseWatchTimelines.Num()))
		{
			Callback(*PoseWatchTimelines[*IndexPtr]);
			return true;
		}
	}

	return false;
}

const FSkeletalMeshInfo* FAnimationProvider::FindSkeletalMeshInfo(uint64 InObjectId) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = SkeletalMeshIdToIndexMap.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(SkeletalMeshInfos.Num()))
		{
			return &SkeletalMeshInfos[*IndexPtr];
		}
	}

	return nullptr;
}

const FAnimNodeInfo* FAnimationProvider::FindAnimNodeInfo(int32 InAnimNodeId, uint64 InAnimInstanceId) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = AnimNodeIdToIndexMap.Find({ InAnimNodeId, InAnimInstanceId });
	if (IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(AnimNodeInfos.Num()))
		{
			return &AnimNodeInfos[*IndexPtr];
		}
	}

	return nullptr;
}

const TCHAR* FAnimationProvider::GetName(uint32 InId) const
{
	ReadAccessCheck();

	const TCHAR* const* FoundName = NameMap.Find(InId);
	if(FoundName != nullptr)
	{
		return *FoundName;
	}

	static FText UnknownText(LOCTEXT("UnknownName", "Unknown"));
	return *UnknownText.ToString();
}

FText FAnimationProvider::FormatNodeKeyValue(const FAnimNodeValueMessage& InMessage) const
{
	ReadAccessCheck();

	return FText::Format(LOCTEXT("KeyValueFormat", "{0} = {1}"), FText::FromString(InMessage.Key), FormatNodeValue(InMessage));
}

FText FAnimationProvider::FormatNodeValue(const FAnimNodeValueMessage& InMessage) const
{
	ReadAccessCheck();

	FText Text;

	switch(InMessage.Value.Type)
	{
	case EAnimNodeValueType::Bool:
		Text = InMessage.Value.Bool.bValue ? LOCTEXT("True", "true") : LOCTEXT("False", "false");
		break;
	case EAnimNodeValueType::Int32:
		Text =  FText::AsNumber(InMessage.Value.Int32.Value);
		break;
	case EAnimNodeValueType::Float:
		Text = FText::AsNumber(InMessage.Value.Float.Value);
		break;
	case EAnimNodeValueType::Vector2D:
		Text = FText::Format(LOCTEXT("Vector2Format", "({0}, {1})"), FText::AsNumber(InMessage.Value.Vector.Value.X), FText::AsNumber(InMessage.Value.Vector.Value.Y));
		break;
	case EAnimNodeValueType::Vector:
		Text = FText::Format(LOCTEXT("VectorFormat", "({0}, {1}, {2})"), FText::AsNumber(InMessage.Value.Vector.Value.X), FText::AsNumber(InMessage.Value.Vector.Value.Y), FText::AsNumber(InMessage.Value.Vector.Value.Z));
		break;
	case EAnimNodeValueType::String:
		Text = FText::FromString(InMessage.Value.String.Value);
		break;
	case EAnimNodeValueType::Object:
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(GameplayProvider);
		const FObjectInfo& ObjectInfo = GameplayProvider.GetObjectInfo(InMessage.Value.Object.Value);
		Text = FText::FromString(ObjectInfo.Name);
		break;
	}
	case EAnimNodeValueType::Class:
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(GameplayProvider);
		const FClassInfo& ClassInfo = GameplayProvider.GetClassInfo(InMessage.Value.Class.Value);
		Text = FText::FromString(ClassInfo.PathName);
		break;
	}
	case EAnimNodeValueType::AnimNode:
	{
		const FAnimNodeInfo* AnimNodeInfo = FindAnimNodeInfo(InMessage.Value.AnimNode.Value, InMessage.Value.AnimNode.AnimInstanceId);
		Text = FText::FromString(AnimNodeInfo ? AnimNodeInfo->Name : TEXT(""));
		break;
	}
	}

	return Text;
}

bool FAnimationProvider::HasExternalMorphSets(uint64 InObjectId) const
{
	//ObjectIdToSkeletalMeshPoseTimelines
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToSkeletalMeshPoseTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(SkeletalMeshPoseTimelineStorage.Num()))
		{
			const TSharedRef<FSkeletalMeshTimelineStorage>& SkeletalMeshTimelineStorage = SkeletalMeshPoseTimelineStorage[*IndexPtr];
			return (SkeletalMeshTimelineStorage->NumExternalMorphSets > 0);
		}
	}

	return false;
}

bool FAnimationProvider::HasAnyData() const
{
	ReadAccessCheck();

	return bHasAnyData;
}

void FAnimationProvider::AppendTickRecord(uint64 InAnimInstanceId, double InProfileTime, double InRecordingTime, uint64 InAssetId, int32 InNodeId, float InBlendWeight, float InPlaybackTime, float InRootMotionWeight, float InPlayRate, float InBlendSpacePositionX, float InBlendSpacePositionY, float InBlendSpaceFilteredPositionX, float InBlendSpaceFilteredPositionY, uint16 InFrameCounter, bool bInLooping, bool bInIsBlendSpace)
{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<FTickRecordTimelineStorage> TimelineStorage;
	uint32* TimelineStorageIndexPtr = ObjectIdToTickRecordTimelineStorage.Find(InAnimInstanceId);
	if(TimelineStorageIndexPtr != nullptr)
	{
		TimelineStorage = TickRecordTimelineStorage[*TimelineStorageIndexPtr];
	}
	else
	{
		ObjectIdToTickRecordTimelineStorage.Add(InAnimInstanceId, TickRecordTimelineStorage.Num());
		TimelineStorage = TickRecordTimelineStorage.Add_GetRef(MakeShared<FTickRecordTimelineStorage>());
		TimelineStorage->Timeline = MakeShared<TraceServices::TPointTimeline<FTickRecordMessage>>(Session.GetLinearAllocator());
		TimelineStorage->Timeline->SetEnumerateOutsideRange(true);
	}

	check(TimelineStorage.IsValid());

	TimelineStorage->AssetIdAndPlayers.Add(TTuple<uint64, int32>(InAssetId, InNodeId));

	FTickRecordMessage Message;
	Message.AnimInstanceId = InAnimInstanceId;
	Message.AssetId = InAssetId;
	Message.RecordingTime = InRecordingTime;
	Message.NodeId = InNodeId;
	Message.BlendWeight = InBlendWeight;
	Message.PlaybackTime = InPlaybackTime;
	Message.RootMotionWeight = InRootMotionWeight;
	Message.PlayRate = InPlayRate;
	Message.BlendSpacePositionX = InBlendSpacePositionX;
	Message.BlendSpacePositionY = InBlendSpacePositionY;
	Message.BlendSpaceFilteredPositionX = InBlendSpaceFilteredPositionX;
	Message.BlendSpaceFilteredPositionY = InBlendSpaceFilteredPositionY;
	Message.FrameCounter = InFrameCounter;
	Message.bLooping = bInLooping;
	Message.bIsBlendSpace = bInIsBlendSpace;

	TimelineStorage->Timeline->AppendEvent(InProfileTime, Message);

	UpdateSessionDurationSeconds(InProfileTime);
}

void FAnimationProvider::AppendTickRecord(uint64 InAnimInstanceId, double InProfileTime, double InRecordingTime, uint64 InAssetId, int32 InNodeId, float InBlendWeight, float InPlaybackTime, float InRootMotionWeight, float InPlayRate, float InBlendSpacePositionX, float InBlendSpacePositionY, float InBlendSpaceFilteredPositionX, float InBlendSpaceFilteredPositionY, uint16 InFrameCounter, bool bInLooping, bool bInIsBlendSpace, uint32 InSyncGroupName, bool bInIsSyncGroupLeader, float InSyncLeaderScore, float TimeToPrevMarker, float TimeToNextMarker, int32 PrevMarkerIndex, int32 NextMarkerIndex)
{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<FTickRecordTimelineStorage> TimelineStorage;
	uint32* TimelineStorageIndexPtr = ObjectIdToTickRecordTimelineStorage.Find(InAnimInstanceId);
	if(TimelineStorageIndexPtr != nullptr)
	{
		TimelineStorage = TickRecordTimelineStorage[*TimelineStorageIndexPtr];
	}
	else
	{
		ObjectIdToTickRecordTimelineStorage.Add(InAnimInstanceId, TickRecordTimelineStorage.Num());
		TimelineStorage = TickRecordTimelineStorage.Add_GetRef(MakeShared<FTickRecordTimelineStorage>());
		TimelineStorage->Timeline = MakeShared<TraceServices::TPointTimeline<FTickRecordMessage>>(Session.GetLinearAllocator());
		TimelineStorage->Timeline->SetEnumerateOutsideRange(true);
	}

	check(TimelineStorage.IsValid());

	TimelineStorage->AssetIdAndPlayers.Add(TTuple<uint64, int32>(InAssetId, InNodeId));

	FTickRecordMessage Message;
	Message.AnimInstanceId = InAnimInstanceId;
	Message.AssetId = InAssetId;
	Message.RecordingTime = InRecordingTime;
	Message.NodeId = InNodeId;
	Message.BlendWeight = InBlendWeight;
	Message.PlaybackTime = InPlaybackTime;
	Message.RootMotionWeight = InRootMotionWeight;
	Message.PlayRate = InPlayRate;
	Message.BlendSpacePositionX = InBlendSpacePositionX;
	Message.BlendSpacePositionY = InBlendSpacePositionY;
	Message.BlendSpaceFilteredPositionX = InBlendSpaceFilteredPositionX;
	Message.BlendSpaceFilteredPositionY = InBlendSpaceFilteredPositionY;
	Message.FrameCounter = InFrameCounter;
	Message.bLooping = bInLooping;
	Message.bIsBlendSpace = bInIsBlendSpace;
	Message.bSyncGroupNameId = InSyncGroupName;
	Message.bIsSyncGroupLeader = bInIsSyncGroupLeader;
	Message.SyncLeaderScore = InSyncLeaderScore;
	Message.PrevMarkerIndex = PrevMarkerIndex;
	Message.NextMarkerIndex = NextMarkerIndex;
	Message.TimeToNextMarker = TimeToNextMarker;
	Message.TimeToPrevMarker = TimeToPrevMarker;
	
	TimelineStorage->Timeline->AppendEvent(InProfileTime, Message);

	UpdateSessionDurationSeconds(InProfileTime);
}

void FAnimationProvider::AppendSkeletalMesh(uint64 InObjectId, uint64 SkeletonId, const TArrayView<const int32>& InParentIndices)
{
	EditAccessCheck();

	if(SkeletalMeshIdToIndexMap.Find(InObjectId) == nullptr)
	{
		bHasAnyData = true;

		FSkeletalMeshInfo NewSkeletalMeshInfo;
		NewSkeletalMeshInfo.Id = InObjectId;
		NewSkeletalMeshInfo.BoneCount = (uint32)InParentIndices.Num();
		NewSkeletalMeshInfo.ParentIndicesStartIndex = SkeletalMeshParentIndices.Num();
		NewSkeletalMeshInfo.SkeletonId = SkeletonId;

		for(const int32& ParentIndex : InParentIndices)
		{
			SkeletalMeshParentIndices.PushBack() = ParentIndex;
		}

		uint32 NewSkeletalMeshInfoIndex = SkeletalMeshInfos.Add(NewSkeletalMeshInfo);
		SkeletalMeshIdToIndexMap.Add(InObjectId, NewSkeletalMeshInfoIndex);
	}
}

void FAnimationProvider::AppendSkeletalMeshComponent(uint64 InObjectId, uint64 InMeshId, double InProfileTime, double InRecordingTime, uint16 InLodIndex, uint16 InFrameCounter, const TArrayView<const FTransform>& InPose, const TArrayView<const FSkeletalMeshNamedCurve>& InCurves)
{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<FSkeletalMeshTimelineStorage> TimelineStorage;
	uint32* IndexPtr = ObjectIdToSkeletalMeshPoseTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		TimelineStorage = SkeletalMeshPoseTimelineStorage[*IndexPtr];
	}
	else
	{
		TimelineStorage = MakeShared<FSkeletalMeshTimelineStorage>();
		TimelineStorage->Timeline = MakeShared<TraceServices::TIntervalTimeline<FSkeletalMeshPoseMessage>>(Session.GetLinearAllocator());
		ObjectIdToSkeletalMeshPoseTimelines.Add(InObjectId, SkeletalMeshPoseTimelineStorage.Num());
		SkeletalMeshPoseTimelineStorage.Add(TimelineStorage.ToSharedRef());
	}

	// terminate existing scopes
	uint64 NumEvents = TimelineStorage->Timeline->GetEventCount();
	if(NumEvents > 0)
	{
		// Add end event at current time
		TimelineStorage->Timeline->EndEvent(NumEvents - 1, InProfileTime);
	}

	FSkeletalMeshPoseMessage Message;
	Message.RecordingTime = InRecordingTime;
	Message.ComponentToWorld = InPose[0];
	Message.TransformStartIndex = SkeletalMeshPoseTransforms.Num();
	Message.CurveStartIndex = SkeletalMeshCurves.Num();
	Message.ComponentId = InObjectId;
	Message.MeshId = InMeshId;
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(GameplayProvider);
		Message.MeshName = GameplayProvider.GetObjectInfo(Message.MeshId).Name;
	}
	Message.NumTransforms = (uint16)InPose.Num() - 1;
	Message.NumCurves = (uint16)InCurves.Num();
	Message.LodIndex = InLodIndex;
	Message.FrameCounter = InFrameCounter;

	TimelineStorage->Timeline->AppendBeginEvent(InProfileTime, Message);

	for(int32 TransformIndex = 1; TransformIndex < InPose.Num(); ++TransformIndex)
	{
		SkeletalMeshPoseTransforms.PushBack() = InPose[TransformIndex];
	}

	for(const FSkeletalMeshNamedCurve& Curve : InCurves)
	{
		SkeletalMeshCurves.PushBack() = Curve;
		TimelineStorage->AllCurveIds.Add(Curve.Id);
	}

	UpdateSessionDurationSeconds(InProfileTime);
}

// support for deserializing LWC/non LWC Transforms from builds with non-matching LWC setting
static FTransform ConvertTransform(int TransformSize, const float* TransformFloats)
{
	FQuat Rotation;
	FVector Translation;
	FVector Scale3D;

	static const int LWCSize = 4 + 8 * 2; // 4 floats + 8 doubles
	static const int NonLWCSize = 12; // 12 floats

	check (TransformSize == NonLWCSize || TransformSize == LWCSize);

	if (TransformSize == NonLWCSize)
	{
		// non LWC (rotation/translation/scale, all floats)
		Rotation.X = TransformFloats[0];
		Rotation.Y = TransformFloats[1];
		Rotation.Z = TransformFloats[2];
		Rotation.W = TransformFloats[3];
		Translation.X = TransformFloats[4];
		Translation.Y = TransformFloats[5];
		Translation.Z = TransformFloats[6];
		Scale3D.X = TransformFloats[8];
		Scale3D.Y = TransformFloats[9];
		Scale3D.Z = TransformFloats[10];
	}
	if (TransformSize == LWCSize)
	{
		// LWC (rotation in floats translation/scale in doubles)
		Rotation.X = TransformFloats[0];
		Rotation.Y = TransformFloats[1];
		Rotation.Z = TransformFloats[2];
		Rotation.W = TransformFloats[3];
		const double* TransformDoubles = reinterpret_cast<const double*>((void*)(TransformFloats + 4));
		Translation.X = TransformDoubles[0];
		Translation.Y = TransformDoubles[1];
		Translation.Z = TransformDoubles[2];
		Scale3D.X = TransformDoubles[4];
		Scale3D.Y = TransformDoubles[5];
		Scale3D.Z = TransformDoubles[6];
	}

	return FTransform(Rotation, Translation, Scale3D);
}

void FAnimationProvider::AppendSkeletalMeshComponent(
	uint64 InObjectId,
	uint64 InMeshId,
	double InProfileTime,
	double InRecordingTime,
	uint16 InLodIndex,
	uint16 InFrameCounter,
	const TArrayView<const float>& InComponentToWorldRaw,
	const TArrayView<const float>& InPoseRaw,
	const TArrayView<const uint32>& InCurveIds,
	const TArrayView<const float>& InCurveValues)

{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<FSkeletalMeshTimelineStorage> TimelineStorage;
	uint32* IndexPtr = ObjectIdToSkeletalMeshPoseTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		TimelineStorage = SkeletalMeshPoseTimelineStorage[*IndexPtr];
	}
	else
	{
		TimelineStorage = MakeShared<FSkeletalMeshTimelineStorage>();
		TimelineStorage->Timeline = MakeShared<TraceServices::TIntervalTimeline<FSkeletalMeshPoseMessage>>(Session.GetLinearAllocator());
		ObjectIdToSkeletalMeshPoseTimelines.Add(InObjectId, SkeletalMeshPoseTimelineStorage.Num());
		SkeletalMeshPoseTimelineStorage.Add(TimelineStorage.ToSharedRef());
	}

	// terminate existing scopes
	uint64 NumEvents = TimelineStorage->Timeline->GetEventCount();
	if (NumEvents > 0)
	{
		// Add end event at current time
		if (TimelineStorage->Timeline->GetEventEndTime(NumEvents - 1) > InProfileTime)
		{
			TimelineStorage->Timeline->EndEvent(NumEvents - 1, InProfileTime);
		}
	}

	const int32 NumCurves = InCurveIds.Num();

	const int CaptureTransformSize = InComponentToWorldRaw.Num();
	const int LocalTransformSize = sizeof (FTransform) / sizeof(float);

	FTransform ComponentToWorld;

	if (CaptureTransformSize == LocalTransformSize)
	{
	 	FMemory::Memcpy(&ComponentToWorld, &InComponentToWorldRaw[0], sizeof(FTransform));
	}
	else
	{
	 	ComponentToWorld = ConvertTransform(CaptureTransformSize, &InComponentToWorldRaw[0]);
	}

	const int PoseTransformCount = InPoseRaw.Num()/CaptureTransformSize;

	FSkeletalMeshPoseMessage Message;
	Message.RecordingTime = InRecordingTime;
	Message.ComponentToWorld = ComponentToWorld;
	Message.TransformStartIndex = SkeletalMeshPoseTransforms.Num();
	Message.CurveStartIndex = SkeletalMeshCurves.Num();
	Message.ComponentId = InObjectId;
	Message.MeshId = InMeshId;
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(GameplayProvider);
		Message.MeshName = GameplayProvider.GetObjectInfo(Message.MeshId).Name;
	}
	Message.NumTransforms = (uint16)PoseTransformCount;
	Message.NumCurves = (uint16)NumCurves;
	Message.LodIndex = InLodIndex;
	Message.FrameCounter = InFrameCounter;

	TimelineStorage->Timeline->AppendBeginEvent(InProfileTime, Message);

	if (CaptureTransformSize == LocalTransformSize)
	{
		for(int i=0; i<PoseTransformCount; i++)
		{
			FMemory::Memcpy(&SkeletalMeshPoseTransforms.PushBack(), &InPoseRaw[CaptureTransformSize * i], sizeof(FTransform));
		}
	}
	else
	{
		for(int i=0; i<PoseTransformCount; i++)
		{
			SkeletalMeshPoseTransforms.PushBack() = ConvertTransform(CaptureTransformSize, &InPoseRaw[CaptureTransformSize * i]);
		}
	}

	for(int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
	{
		SkeletalMeshCurves.PushBack() = { InCurveIds[CurveIndex], InCurveValues[CurveIndex] };
		TimelineStorage->AllCurveIds.Add(InCurveIds[CurveIndex]);
	}

	UpdateSessionDurationSeconds(InProfileTime);
}

void FAnimationProvider::AppendSkeletalMeshComponent(
	uint64 InObjectId,
	uint64 InMeshId,
	double InProfileTime,
	double InRecordingTime,
	uint16 InLodIndex,
	uint16 InFrameCounter,
	bool bInIsVisible,
	const TArrayView<const float>& InComponentToWorldRaw,
	const TArrayView<const float>& InPoseRaw,
	const TArrayView<const uint32>& InCurveIds,
	const TArrayView<const float>& InCurveValues,
	const TArrayView<const float>& InExternalMorphWeights,
	const TArrayView<const int32>& InExternalMorphCounts)
{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<FSkeletalMeshTimelineStorage> TimelineStorage;
	uint32* IndexPtr = ObjectIdToSkeletalMeshPoseTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		TimelineStorage = SkeletalMeshPoseTimelineStorage[*IndexPtr];
	}
	else
	{
		TimelineStorage = MakeShared<FSkeletalMeshTimelineStorage>();
		TimelineStorage->Timeline = MakeShared<TraceServices::TIntervalTimeline<FSkeletalMeshPoseMessage>>(Session.GetLinearAllocator());
		ObjectIdToSkeletalMeshPoseTimelines.Add(InObjectId, SkeletalMeshPoseTimelineStorage.Num());
		SkeletalMeshPoseTimelineStorage.Add(TimelineStorage.ToSharedRef());
	}

	// terminate existing scopes
	uint64 NumEvents = TimelineStorage->Timeline->GetEventCount();
	if (NumEvents > 0)
	{
		// Add end event at current time
		if (TimelineStorage->Timeline->GetEventEndTime(NumEvents - 1) > InProfileTime)
		{
			TimelineStorage->Timeline->EndEvent(NumEvents - 1, InProfileTime);
		}
	}

	const int32 NumCurves = InCurveIds.Num();

	const int CaptureTransformSize = InComponentToWorldRaw.Num();
	const int LocalTransformSize = sizeof (FTransform) / sizeof(float);

	FTransform ComponentToWorld;

	if (CaptureTransformSize == LocalTransformSize)
	{
	 	FMemory::Memcpy(&ComponentToWorld, &InComponentToWorldRaw[0], sizeof(FTransform));
	}
	else
	{
	 	ComponentToWorld = ConvertTransform(CaptureTransformSize, &InComponentToWorldRaw[0]);
	}

	const int PoseTransformCount = InPoseRaw.Num()/CaptureTransformSize;

	FSkeletalMeshPoseMessage Message;
	Message.RecordingTime = InRecordingTime;
	Message.bIsVisible = bInIsVisible;
	Message.ComponentToWorld = ComponentToWorld;
	Message.TransformStartIndex = SkeletalMeshPoseTransforms.Num();
	Message.CurveStartIndex = SkeletalMeshCurves.Num();
	Message.ComponentId = InObjectId;
	Message.MeshId = InMeshId;
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(GameplayProvider);
		Message.MeshName = GameplayProvider.GetObjectInfo(Message.MeshId).Name;
	}
	Message.NumTransforms = (uint16)PoseTransformCount;
	Message.NumCurves = (uint16)NumCurves;
	Message.LodIndex = InLodIndex;
	Message.FrameCounter = InFrameCounter;
	Message.ExternalMorphStartIndex = static_cast<uint64>(ExternalMorphWeights.Num());
	Message.NumExternalMorphSets = static_cast<uint16>(InExternalMorphCounts.Num());

	TimelineStorage->Timeline->AppendBeginEvent(InProfileTime, Message);

	if (CaptureTransformSize == LocalTransformSize)
	{
		for(int i=0; i<PoseTransformCount; i++)
		{
			FMemory::Memcpy(&SkeletalMeshPoseTransforms.PushBack(), &InPoseRaw[CaptureTransformSize * i], sizeof(FTransform));
		}
	}
	else
	{
		for(int i=0; i<PoseTransformCount; i++)
		{
			SkeletalMeshPoseTransforms.PushBack() = ConvertTransform(CaptureTransformSize, &InPoseRaw[CaptureTransformSize * i]);
		}
	}

	for(int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
	{
		SkeletalMeshCurves.PushBack() = { InCurveIds[CurveIndex], InCurveValues[CurveIndex] };
		TimelineStorage->AllCurveIds.Add(InCurveIds[CurveIndex]);
	}

	// Save the weights of all morph targets inside each external morph set.
	int32 WeightOffset = 0;
	for (int32 MorphSetIndex = 0; MorphSetIndex < InExternalMorphCounts.Num(); ++MorphSetIndex)
	{
		// Get the weight values for the current morph set.
		const int32 NumWeightsInMorphSet = InExternalMorphCounts[MorphSetIndex];
		const TConstArrayView<float> SetWeights(&InExternalMorphWeights[WeightOffset], NumWeightsInMorphSet);

		// Record them in our buffer by copying them.
		FExternalMorphWeightMessage& NewWeights = ExternalMorphWeights.PushBack();
		NewWeights.Weights = SetWeights;
		NewWeights.NumMorphs = NumWeightsInMorphSet;
		NewWeights.Index = MorphSetIndex;

		WeightOffset += NumWeightsInMorphSet;
	}

	TimelineStorage->NumExternalMorphSets = InExternalMorphCounts.Num();

	UpdateSessionDurationSeconds(InProfileTime);
}
	
void FAnimationProvider::AppendName(uint32 InId, const TCHAR* InName)
{
	EditAccessCheck();

	bHasAnyData = true;

	NameMap.Add(InId, Session.StoreString(InName));
}

void FAnimationProvider::HandleObjectEndPlay(uint64 InObjectId, double InTime, const FObjectInfo& InObjectInfo)
{
	// terminate all existing scopes for this object
	uint32* SkelMeshPoseIndexPtr = ObjectIdToSkeletalMeshPoseTimelines.Find(InObjectId);
	if(SkelMeshPoseIndexPtr != nullptr)
	{
		TSharedPtr<TraceServices::TIntervalTimeline<FSkeletalMeshPoseMessage>> Timeline = SkeletalMeshPoseTimelineStorage[*SkelMeshPoseIndexPtr]->Timeline;
		uint64 NumEvents = Timeline->GetEventCount();
		if(NumEvents > 0)
		{
			Timeline->EndEvent(NumEvents - 1, InTime);
		}
	}
}

void FAnimationProvider::AppendSkeletalMeshFrame(uint64 InObjectId, double InTime, uint16 InFrameCounter)
{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<TraceServices::TIntervalTimeline<FSkeletalMeshFrameMessage>> Timeline;
	uint32* IndexPtr = ObjectIdToSkeletalMeshFrameTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		check(SkeletalMeshFrameTimelines.IsValidIndex(*IndexPtr));
		Timeline = SkeletalMeshFrameTimelines[*IndexPtr];
	}
	else
	{
		Timeline = MakeShared<TraceServices::TIntervalTimeline<FSkeletalMeshFrameMessage>>(Session.GetLinearAllocator());
		ObjectIdToSkeletalMeshFrameTimelines.Add(InObjectId, SkeletalMeshFrameTimelines.Num());
		SkeletalMeshFrameTimelines.Add(Timeline.ToSharedRef());
	}

	// terminate existing scopes
	uint64 NumEvents = Timeline->GetEventCount();
	if(NumEvents > 0)
	{
		// Add end event at current time
		Timeline->EndEvent(NumEvents - 1, InTime);
	}

	FSkeletalMeshFrameMessage Message;
	Message.ComponentId = InObjectId;
	Message.FrameCounter = InFrameCounter;

	Timeline->AppendBeginEvent(InTime, Message);

	UpdateSessionDurationSeconds(InTime);
}

void FAnimationProvider::AppendAnimGraph(uint64 InAnimInstanceId, double InStartTime, double InEndTime, int32 InNodeCount, uint16 InFrameCounter, uint8 InPhase)
{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<TraceServices::TIntervalTimeline<FAnimGraphMessage>> Timeline;
	uint32* IndexPtr = ObjectIdToAnimGraphTimelines.Find(InAnimInstanceId);
	if(IndexPtr != nullptr)
	{
		check(AnimGraphTimelines.IsValidIndex(*IndexPtr));
		Timeline = AnimGraphTimelines[*IndexPtr];
	}
	else
	{
		Timeline = MakeShared<TraceServices::TIntervalTimeline<FAnimGraphMessage>>(Session.GetLinearAllocator());
		ObjectIdToAnimGraphTimelines.Add(InAnimInstanceId, AnimGraphTimelines.Num());
		AnimGraphTimelines.Add(Timeline.ToSharedRef());
	}

	FAnimGraphMessage Message;
	Message.AnimInstanceId = InAnimInstanceId;
	Message.NodeCount = InNodeCount;
	Message.FrameCounter = InFrameCounter;
	Message.Phase = (EAnimGraphPhase)InPhase;

	uint64 EventIndex = Timeline->AppendBeginEvent(InStartTime, Message);
	Timeline->EndEvent(EventIndex, InEndTime);

	UpdateSessionDurationSeconds(InStartTime);
}

void FAnimationProvider::AppendAnimNodeStart(uint64 InAnimInstanceId, double InStartTime, uint16 InFrameCounter, int32 InNodeId, int32 PreviousNodeId, const TCHAR* InTargetNodeName, float InWeight, float InRootMotionWeight, const TCHAR* InTargetNodeDisplayName, uint8 InPhase)
{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<TraceServices::TPointTimeline<FAnimNodeMessage>> Timeline;
	uint32* IndexPtr = ObjectIdToAnimNodeTimelines.Find(InAnimInstanceId);
	if(IndexPtr != nullptr)
	{
		check(AnimNodeTimelines.IsValidIndex(*IndexPtr));
		Timeline = AnimNodeTimelines[*IndexPtr];
	}
	else
	{
		Timeline = MakeShared<TraceServices::TPointTimeline<FAnimNodeMessage>>(Session.GetLinearAllocator());
		ObjectIdToAnimNodeTimelines.Add(InAnimInstanceId, AnimNodeTimelines.Num());
		AnimNodeTimelines.Add(Timeline.ToSharedRef());
	}

	FAnimNodeMessage Message;
	Message.NodeName = Session.StoreString(InTargetNodeDisplayName);
	Message.NodeTypeName = Session.StoreString(InTargetNodeName);
	Message.AnimInstanceId = InAnimInstanceId;
	Message.PreviousNodeId = PreviousNodeId;
	Message.NodeId = InNodeId;
	Message.Weight = InWeight;
	Message.RootMotionWeight = InRootMotionWeight;
	Message.FrameCounter = InFrameCounter;
	Message.Phase = (EAnimGraphPhase)InPhase;

	Timeline->AppendEvent(InStartTime, Message);

	if (AnimNodeIdToIndexMap.Find({ InNodeId, InAnimInstanceId }) == nullptr)
	{
		bHasAnyData = true;

		FAnimNodeInfo NewAnimNodeInfo;
		NewAnimNodeInfo.Id = Message.NodeId;
		NewAnimNodeInfo.AnimInstanceId = Message.AnimInstanceId;
		NewAnimNodeInfo.Name = Message.NodeName;
		NewAnimNodeInfo.TypeName = Message.NodeTypeName;

		uint32 NewAnimNodeInfoIndex = AnimNodeInfos.Add(NewAnimNodeInfo);
		AnimNodeIdToIndexMap.Add({ InNodeId, InAnimInstanceId }, NewAnimNodeInfoIndex);
	}

	UpdateSessionDurationSeconds(InStartTime);
}

void FAnimationProvider::AppendAnimSequencePlayer(uint64 InAnimInstanceId, double InTime, int32 InNodeId, float InPosition, float InLength, uint16 InFrameCounter)
{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<TraceServices::TPointTimeline<FAnimSequencePlayerMessage>> Timeline;
	uint32* IndexPtr = ObjectIdToAnimSequencePlayerTimelines.Find(InAnimInstanceId);
	if(IndexPtr != nullptr)
	{
		check(AnimSequencePlayerTimelines.IsValidIndex(*IndexPtr));
		Timeline = AnimSequencePlayerTimelines[*IndexPtr];
	}
	else
	{
		Timeline = MakeShared<TraceServices::TPointTimeline<FAnimSequencePlayerMessage>>(Session.GetLinearAllocator());
		ObjectIdToAnimSequencePlayerTimelines.Add(InAnimInstanceId, AnimSequencePlayerTimelines.Num());
		AnimSequencePlayerTimelines.Add(Timeline.ToSharedRef());
	}

	FAnimSequencePlayerMessage Message;
	Message.AnimInstanceId = InAnimInstanceId;
	Message.NodeId = InNodeId;
	Message.Position = InPosition;
	Message.Length = InLength;
	Message.FrameCounter = InFrameCounter;

	Timeline->AppendEvent(InTime, Message);

	UpdateSessionDurationSeconds(InTime);
}

void FAnimationProvider::AppendBlendSpacePlayer(uint64 InAnimInstanceId, double InTime, int32 InNodeId, uint64 InBlendSpaceId, const FVector& InBlendPosition, const FVector& InFilteredBlendPosition)
{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<TraceServices::TPointTimeline<FBlendSpacePlayerMessage>> Timeline;
	uint32* IndexPtr = ObjectIdToBlendSpacePlayerTimelines.Find(InAnimInstanceId);
	if(IndexPtr != nullptr)
	{
		check(BlendSpacePlayerTimelines.IsValidIndex(*IndexPtr));
		Timeline = BlendSpacePlayerTimelines[*IndexPtr];
	}
	else
	{
		Timeline = MakeShared<TraceServices::TPointTimeline<FBlendSpacePlayerMessage>>(Session.GetLinearAllocator());
		ObjectIdToBlendSpacePlayerTimelines.Add(InAnimInstanceId, BlendSpacePlayerTimelines.Num());
		BlendSpacePlayerTimelines.Add(Timeline.ToSharedRef());
	}

	FBlendSpacePlayerMessage Message;
	Message.AnimInstanceId = InAnimInstanceId;
	Message.BlendSpaceId = InBlendSpaceId;
	Message.NodeId = InNodeId;
	Message.PositionX = InBlendPosition.X;
	Message.PositionY = InBlendPosition.Y;
	Message.PositionZ = InBlendPosition.Z;
	Message.FilteredPositionX = InFilteredBlendPosition.X;
	Message.FilteredPositionY = InFilteredBlendPosition.Y;
	Message.FilteredPositionZ = InFilteredBlendPosition.Z;

	Timeline->AppendEvent(InTime, Message);

	UpdateSessionDurationSeconds(InTime);
}

void FAnimationProvider::AppendAnimNodeValue(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, bool bInValue)
{
	FAnimNodeValueMessage Message;
	Message.Value.Bool.bValue = bInValue;
	Message.Value.Type = EAnimNodeValueType::Bool;

	AppendAnimNodeValue(InAnimInstanceId, InTime, InRecordingTime, InFrameCounter, InNodeId, InKey, Message);
}

void FAnimationProvider::AppendAnimNodeValue(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, int32 InValue)
{
	FAnimNodeValueMessage Message;
	Message.Value.Int32.Value = InValue;
	Message.Value.Type = EAnimNodeValueType::Int32;

	AppendAnimNodeValue(InAnimInstanceId, InTime, InRecordingTime, InFrameCounter, InNodeId, InKey, Message);
}

void FAnimationProvider::AppendAnimNodeValue(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, float InValue)
{
	FAnimNodeValueMessage Message;
	Message.Value.Float.Value = InValue;
	Message.Value.Type = EAnimNodeValueType::Float;

	AppendAnimNodeValue(InAnimInstanceId, InTime, InRecordingTime, InFrameCounter, InNodeId, InKey, Message);
}

void FAnimationProvider::AppendAnimNodeValue(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, const FVector2D& InValue)
{
	FAnimNodeValueMessage Message;
	Message.Value.Vector2D.Value = InValue;
	Message.Value.Type = EAnimNodeValueType::Vector2D;

	AppendAnimNodeValue(InAnimInstanceId, InTime, InRecordingTime, InFrameCounter, InNodeId, InKey, Message);
}

void FAnimationProvider::AppendAnimNodeValue(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, const FVector& InValue)
{
	FAnimNodeValueMessage Message;
	Message.Value.Vector.Value = InValue;
	Message.Value.Type = EAnimNodeValueType::Vector;

	AppendAnimNodeValue(InAnimInstanceId, InTime, InRecordingTime, InFrameCounter, InNodeId, InKey, Message);
}

void FAnimationProvider::AppendAnimNodeValue(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, const TCHAR* InValue)
{
	FAnimNodeValueMessage Message;
	Message.Value.String.Value = Session.StoreString(InValue);
	Message.Value.Type = EAnimNodeValueType::String;

	AppendAnimNodeValue(InAnimInstanceId, InTime, InRecordingTime, InFrameCounter, InNodeId, InKey, Message);
}

void FAnimationProvider::AppendAnimNodeValueObject(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, uint64 InValue)
{
	FAnimNodeValueMessage Message;
	Message.Value.Object.Value = InValue;
	Message.Value.Type = EAnimNodeValueType::Object;

	AppendAnimNodeValue(InAnimInstanceId, InTime, InRecordingTime, InFrameCounter, InNodeId, InKey, Message);
}

void FAnimationProvider::AppendAnimNodeValueClass(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, uint64 InValue)
{
	FAnimNodeValueMessage Message;
	Message.Value.Class.Value = InValue;
	Message.Value.Type = EAnimNodeValueType::Class;

	AppendAnimNodeValue(InAnimInstanceId, InTime, InRecordingTime, InFrameCounter, InNodeId, InKey, Message);
}

void FAnimationProvider::AppendAnimNodeValueAnimNode(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, int32 InValue, uint64 InValueAnimInstanceId)
{
	FAnimNodeValueMessage Message;
	Message.Value.AnimNode.Value = InValue;
	Message.Value.AnimNode.AnimInstanceId = InValueAnimInstanceId;
	Message.Value.Type = EAnimNodeValueType::AnimNode;

	AppendAnimNodeValue(InAnimInstanceId, InTime, InRecordingTime, InFrameCounter, InNodeId, InKey, Message);
}

void FAnimationProvider::AppendAnimNodeValue(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, FAnimNodeValueMessage& InMessage)
{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<TraceServices::TPointTimeline<FAnimNodeValueMessage>> Timeline;
	uint32* IndexPtr = ObjectIdToAnimNodeValueTimelines.Find(InAnimInstanceId);
	if(IndexPtr != nullptr)
	{
		Timeline = AnimNodeValueTimelines[*IndexPtr];
	}
	else
	{
		Timeline = MakeShared<TraceServices::TPointTimeline<FAnimNodeValueMessage>>(Session.GetLinearAllocator());
		ObjectIdToAnimNodeValueTimelines.Add(InAnimInstanceId, AnimNodeValueTimelines.Num());
		AnimNodeValueTimelines.Add(Timeline.ToSharedRef());
	}

	InMessage.Key = Session.StoreString(InKey);
	InMessage.AnimInstanceId = InAnimInstanceId;
	InMessage.NodeId = InNodeId;
	InMessage.FrameCounter = InFrameCounter;
	InMessage.RecordingTime = InRecordingTime;

	Timeline->AppendEvent(InTime, InMessage);

	UpdateSessionDurationSeconds(InTime);
}

void FAnimationProvider::AppendAnimGraphAttribute(uint64 InSourceAnimInstanceId, uint64 InTargetAnimInstanceId, double InTime, int32 InSourceNodeId, int32 InTargetNodeId, uint32 InAttributeNameId)
{
	EditAccessCheck();

	bHasAnyData = true;

	if(InSourceAnimInstanceId == InTargetAnimInstanceId)
	{
		TSharedPtr<TraceServices::TPointTimeline<FAnimAttributeMessage>> Timeline;
		uint32* IndexPtr = ObjectIdToAnimAttributeTimelines.Find(InSourceAnimInstanceId);
		if(IndexPtr != nullptr)
		{
			Timeline = AnimAttributeTimelines[*IndexPtr];
		}
		else
		{
			Timeline = MakeShared<TraceServices::TPointTimeline<FAnimAttributeMessage>>(Session.GetLinearAllocator());
			ObjectIdToAnimAttributeTimelines.Add(InSourceAnimInstanceId, AnimAttributeTimelines.Num());
			AnimAttributeTimelines.Add(Timeline.ToSharedRef());
		}

		FAnimAttributeMessage Message;
		Message.SourceNodeId = InSourceNodeId;
		Message.TargetNodeId = InTargetNodeId;
		Message.AttributeNameId = InAttributeNameId;

		Timeline->AppendEvent(InTime, Message);
	}
	else
	{
		// If we are using two different anim instances, we need to append to two different timelines
		TSharedPtr<TraceServices::TPointTimeline<FAnimAttributeMessage>> SourceTimeline;
		uint32* SourceIndexPtr = ObjectIdToAnimAttributeTimelines.Find(InSourceAnimInstanceId);
		if(SourceIndexPtr != nullptr)
		{
			SourceTimeline = AnimAttributeTimelines[*SourceIndexPtr];
		}
		else
		{
			SourceTimeline = MakeShared<TraceServices::TPointTimeline<FAnimAttributeMessage>>(Session.GetLinearAllocator());
			ObjectIdToAnimAttributeTimelines.Add(InSourceAnimInstanceId, AnimAttributeTimelines.Num());
			AnimAttributeTimelines.Add(SourceTimeline.ToSharedRef());
		}

		TSharedPtr<TraceServices::TPointTimeline<FAnimAttributeMessage>> TargetTimeline;
		uint32* TargetIndexPtr = ObjectIdToAnimAttributeTimelines.Find(InTargetAnimInstanceId);
		if(TargetIndexPtr != nullptr)
		{
			TargetTimeline = AnimAttributeTimelines[*TargetIndexPtr];
		}
		else
		{
			TargetTimeline = MakeShared<TraceServices::TPointTimeline<FAnimAttributeMessage>>(Session.GetLinearAllocator());
			ObjectIdToAnimAttributeTimelines.Add(InTargetAnimInstanceId, AnimAttributeTimelines.Num());
			AnimAttributeTimelines.Add(TargetTimeline.ToSharedRef());
		}

		FAnimAttributeMessage TargetMessage;
		TargetMessage.SourceNodeId = INDEX_NONE;
		TargetMessage.TargetNodeId = InTargetNodeId;
		TargetMessage.AttributeNameId = InAttributeNameId;

		TargetTimeline->AppendEvent(InTime, TargetMessage);

		FAnimAttributeMessage SourceMessage;
		SourceMessage.SourceNodeId = InSourceNodeId;
		SourceMessage.TargetNodeId = INDEX_NONE;
		SourceMessage.AttributeNameId = InAttributeNameId;

		SourceTimeline->AppendEvent(InTime, SourceMessage);
	}

	UpdateSessionDurationSeconds(InTime);
}

void FAnimationProvider::AppendStateMachineState(uint64 InAnimInstanceId, double InTime, int32 InNodeId, int32 InStateMachineIndex, int32 InStateIndex, float InStateWeight, float InElapsedTime)
{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<TraceServices::TPointTimeline<FAnimStateMachineMessage>> Timeline;
	uint32* IndexPtr = ObjectIdToStateMachineTimelines.Find(InAnimInstanceId);
	if(IndexPtr != nullptr)
	{
		Timeline = StateMachineTimelines[*IndexPtr];
	}
	else
	{
		Timeline = MakeShared<TraceServices::TPointTimeline<FAnimStateMachineMessage>>(Session.GetLinearAllocator());
		ObjectIdToStateMachineTimelines.Add(InAnimInstanceId, StateMachineTimelines.Num());
		StateMachineTimelines.Add(Timeline.ToSharedRef());
	}

	FAnimStateMachineMessage Message;
	Message.AnimInstanceId = InAnimInstanceId;
	Message.NodeId = InNodeId;
	Message.StateMachineIndex = InStateMachineIndex;
	Message.StateIndex = InStateIndex;
	Message.StateWeight = InStateWeight;
	Message.ElapsedTime = InElapsedTime;

	Timeline->AppendEvent(InTime, Message);

	UpdateSessionDurationSeconds(InTime);
}

void FAnimationProvider::AppendNotify(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint64 InAssetId, uint64 InNotifyId, uint32 InNameId, float InNotifyTime, float InNotifyDuration, EAnimNotifyMessageType InNotifyEventType)
{
	EditAccessCheck();

	bHasAnyData = true;

	// Check if stateful or event-based
	if(InNotifyEventType == EAnimNotifyMessageType::Begin || InNotifyEventType == EAnimNotifyMessageType::End)
	{
		TSharedPtr<TraceServices::TIntervalTimeline<FAnimNotifyMessage>> Timeline;
		TSharedPtr<FAnimNotifyStateTimelineStorage> TimelineStorage;
		uint32* TimelineStorageIndexPtr = ObjectIdToAnimNotifyStateTimelines.Find(InAnimInstanceId);
		if(TimelineStorageIndexPtr != nullptr)
		{
			TimelineStorage = AnimNotifyStateTimelineStorage[*TimelineStorageIndexPtr];
		}
		else
		{
			ObjectIdToAnimNotifyStateTimelines.Add(InAnimInstanceId, AnimNotifyStateTimelineStorage.Num());
			TimelineStorage = AnimNotifyStateTimelineStorage.Add_GetRef(MakeShared<FAnimNotifyStateTimelineStorage>());
		}

		check(TimelineStorage.IsValid());

		uint32* TimelineIndexPtr = TimelineStorage->NotifyIdToAnimNotifyStateTimeline.Find(InNotifyId);
		if(TimelineIndexPtr != nullptr)
		{
			Timeline = TimelineStorage->Timelines[*TimelineIndexPtr];
		}
		else
		{
			Timeline = MakeShared<TraceServices::TIntervalTimeline<FAnimNotifyMessage>>(Session.GetLinearAllocator());
			TimelineStorage->NotifyIdToAnimNotifyStateTimeline.Add(InNotifyId, TimelineStorage->Timelines.Num());
			TimelineStorage->Timelines.Add(Timeline.ToSharedRef());
		}

		if(InNotifyEventType == EAnimNotifyMessageType::Begin)
		{
			FAnimNotifyMessage Message;
			Message.RecordingTime = InRecordingTime;
			Message.AnimInstanceId = InAnimInstanceId;
			Message.AssetId = InAssetId;
			Message.NotifyId = InNotifyId;
			Message.NameId = InNameId;
			Message.Name = GetName(InNameId);
			Message.Time = InNotifyTime; 
			Message.Duration = InNotifyDuration;
			Message.NotifyEventType = InNotifyEventType;

			Timeline->AppendBeginEvent(InTime, Message);
		}
		else if(Timeline->GetEventCount() > 0)
		{
			Timeline->EndEvent(Timeline->GetEventCount() - 1, InTime);
		}
	}
	else
	{
		TSharedPtr<TraceServices::TPointTimeline<FAnimNotifyMessage>> Timeline;
		uint32* IndexPtr = ObjectIdToAnimNotifyTimelines.Find(InAnimInstanceId);
		if(IndexPtr != nullptr)
		{
			Timeline = AnimNotifyTimelines[*IndexPtr];
		}
		else
		{
			Timeline = MakeShared<TraceServices::TPointTimeline<FAnimNotifyMessage>>(Session.GetLinearAllocator());
			ObjectIdToAnimNotifyTimelines.Add(InAnimInstanceId, AnimNotifyTimelines.Num());
			AnimNotifyTimelines.Add(Timeline.ToSharedRef());
		}

		FAnimNotifyMessage Message;
		Message.AnimInstanceId = InAnimInstanceId;
		Message.RecordingTime = InRecordingTime;
		Message.AssetId = InAssetId;
		Message.NotifyId = InNotifyId;
		Message.NameId = InNameId;
		Message.Name = GetName(InNameId);
		Message.Time = InNotifyTime; 
		Message.Duration = InNotifyDuration;
		Message.NotifyEventType = InNotifyEventType;

		Timeline->AppendEvent(InTime, Message);
	}

	UpdateSessionDurationSeconds(InTime);
}

void FAnimationProvider::AppendMontage(uint64 InAnimInstanceId, double InProfileTime, double InRecoringTime, uint64 InMontageId, uint32 InCurrentSectionNameId, uint32 InNextSectionNameId, float InWeight, float InDesiredWeight, float InPosition, uint16 InFrameCounter)
{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<FMontageTimelineStorage> TimelineStorage;
	uint32* IndexPtr = ObjectIdToAnimMontageTimelines.Find(InAnimInstanceId);
	if(IndexPtr != nullptr)
	{
		TimelineStorage = AnimMontageTimelineStorage[*IndexPtr];
	}
	else
	{
		TimelineStorage = MakeShared<FMontageTimelineStorage>();
		TimelineStorage->Timeline = MakeShared<TraceServices::TPointTimeline<FAnimMontageMessage>>(Session.GetLinearAllocator());
		TimelineStorage->Timeline->SetEnumerateOutsideRange(true);
		ObjectIdToAnimMontageTimelines.Add(InAnimInstanceId, AnimMontageTimelineStorage.Num());
		AnimMontageTimelineStorage.Add(TimelineStorage.ToSharedRef());
	}

	TimelineStorage->AllMontageIds.Add(InMontageId);

	FAnimMontageMessage Message;
	Message.RecordingTime = InRecoringTime;
	Message.AnimInstanceId = InAnimInstanceId;
	Message.MontageId = InMontageId;
	Message.CurrentSectionNameId = InCurrentSectionNameId;
	Message.NextSectionNameId = InNextSectionNameId;
	Message.Weight = InWeight;
	Message.DesiredWeight = InDesiredWeight;
	Message.Position = InPosition;
	Message.FrameCounter = InFrameCounter;

	TimelineStorage->Timeline->AppendEvent(InProfileTime, Message);

	UpdateSessionDurationSeconds(InProfileTime);
}

void FAnimationProvider::AppendSync(uint64 InAnimInstanceId, double InTime, int32 InSourceNodeId, uint32 InGroupNameId)
{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<TraceServices::TPointTimeline<FAnimSyncMessage>> Timeline;
	uint32* IndexPtr = ObjectIdToAnimSyncTimelines.Find(InAnimInstanceId);
	if(IndexPtr != nullptr)
	{
		check(AnimSyncTimelines.IsValidIndex(*IndexPtr));
		Timeline = AnimSyncTimelines[*IndexPtr];
	}
	else
	{
		Timeline = MakeShared<TraceServices::TPointTimeline<FAnimSyncMessage>>(Session.GetLinearAllocator());
		ObjectIdToAnimSyncTimelines.Add(InAnimInstanceId, AnimSyncTimelines.Num());
		AnimSyncTimelines.Add(Timeline.ToSharedRef());
	}

	FAnimSyncMessage Message;
	Message.SourceNodeId = InSourceNodeId;
	Message.GroupNameId = InGroupNameId;

	Timeline->AppendEvent(InTime, Message);

	UpdateSessionDurationSeconds(InTime);
}

void FAnimationProvider::AppendPoseWatch(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint64 PoseWatchId, const TArrayView<const float>& BoneTransformsRaw, const TArrayView<const uint16>& RequiredBones, const TArrayView<const float>& WorldTransformRaw, const bool bIsEnabled)
{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<TraceServices::TPointTimeline<FPoseWatchMessage>> Timeline;
	uint32* PoseWatchIndexPtr = ObjectIdToPoseWatchTimelines.Find(InAnimInstanceId);
	if (PoseWatchIndexPtr != nullptr)
	{
		check(PoseWatchTimelines.IsValidIndex(*PoseWatchIndexPtr));
		Timeline = PoseWatchTimelines[*PoseWatchIndexPtr];
	}
	else
	{
		Timeline = MakeShared<TraceServices::TPointTimeline<FPoseWatchMessage>>(Session.GetLinearAllocator());
		ObjectIdToPoseWatchTimelines.Add(InAnimInstanceId, PoseWatchTimelines.Num());
		PoseWatchTimelines.Add(Timeline.ToSharedRef());
	}

	FTransform WorldTransform;
	FMemory::Memcpy(&WorldTransform, &WorldTransformRaw[0], sizeof(FTransform));

	FPoseWatchMessage Message;
	Message.RecordingTime = InRecordingTime;
	Message.ComponentId = 0;
	Message.AnimInstanceId = InAnimInstanceId;
	Message.PoseWatchId = PoseWatchId;
	Message.BoneTransformsStartIndex = SkeletalMeshPoseTransforms.Num();
	Message.CurveStartIndex = SkeletalMeshCurves.Num();
	Message.NumBoneTransforms = BoneTransformsRaw.Num() / (sizeof(FTransform) / sizeof(float));
	Message.WorldTransform = WorldTransform;
	Message.RequiredBonesStartIndex = PoseWatchRequiredBones.Num();
	Message.NumRequiredBones = RequiredBones.Num();
	Message.NumCurves = 0;
	Message.bIsEnabled = bIsEnabled;

	Timeline->AppendEvent(InTime, Message);

	const int CaptureTransformSize = sizeof(FTransform) / sizeof(float); // TODO
	const int LocalTransformSize = sizeof(FTransform) / sizeof(float);
	const int PoseTransformCount = BoneTransformsRaw.Num() / CaptureTransformSize;

	// Dump PoseBoneTransformsRaw into SkeletalMeshPoseTransforms
	if (CaptureTransformSize == LocalTransformSize)
	{
		for (int i = 0; i < PoseTransformCount; i++)
		{
			FMemory::Memcpy(&SkeletalMeshPoseTransforms.PushBack(), &BoneTransformsRaw[CaptureTransformSize * i], sizeof(FTransform));
		}
	}
	else
	{
		for (int i = 0; i < PoseTransformCount; i++)
		{
			SkeletalMeshPoseTransforms.PushBack() = ConvertTransform(CaptureTransformSize, &BoneTransformsRaw[CaptureTransformSize * i]);
		}
	}

	// Dump RequiredBones into PoseWatchRequiredBones
	for (const uint16 RequiredBone : RequiredBones)
	{
		PoseWatchRequiredBones.PushBack() = RequiredBone;
	}

	UpdateSessionDurationSeconds(InTime);
}

void FAnimationProvider::AppendPoseWatch(uint64 InComponentId, uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint64 PoseWatchId, uint32 NameId, FColor Color, const TArrayView<const float>& BoneTransformsRaw, const TArrayView<const uint32>& CurveIds, const TArrayView<const float>& CurveValues, const TArrayView<const uint16>& RequiredBones, const TArrayView<const float>& WorldTransformRaw, const bool bIsEnabled)
{
	EditAccessCheck();

	bHasAnyData = true;

	TSharedPtr<TraceServices::TPointTimeline<FPoseWatchMessage>> Timeline;
	uint32* PoseWatchIndexPtr = ObjectIdToPoseWatchTimelines.Find(InAnimInstanceId);
	if (PoseWatchIndexPtr != nullptr)
	{
		check(PoseWatchTimelines.IsValidIndex(*PoseWatchIndexPtr));
		Timeline = PoseWatchTimelines[*PoseWatchIndexPtr];
	}
	else
	{
		Timeline = MakeShared<TraceServices::TPointTimeline<FPoseWatchMessage>>(Session.GetLinearAllocator());
		ObjectIdToPoseWatchTimelines.Add(InAnimInstanceId, PoseWatchTimelines.Num());
		PoseWatchTimelines.Add(Timeline.ToSharedRef());
	}

	TSharedPtr<FSkeletalMeshTimelineStorage> TimelineStorage;
	uint32* SkeletalMeshIndexPtr = ObjectIdToSkeletalMeshPoseTimelines.Find(InComponentId);
	if(SkeletalMeshIndexPtr != nullptr)
	{
		TimelineStorage = SkeletalMeshPoseTimelineStorage[*SkeletalMeshIndexPtr];
	}
	else
	{
		TimelineStorage = MakeShared<FSkeletalMeshTimelineStorage>();
		TimelineStorage->Timeline = MakeShared<TraceServices::TIntervalTimeline<FSkeletalMeshPoseMessage>>(Session.GetLinearAllocator());
		ObjectIdToSkeletalMeshPoseTimelines.Add(InComponentId, SkeletalMeshPoseTimelineStorage.Num());
		SkeletalMeshPoseTimelineStorage.Add(TimelineStorage.ToSharedRef());
	}
	
	const int32 NumCurves = CurveIds.Num();

	FTransform WorldTransform;
	FMemory::Memcpy(&WorldTransform, &WorldTransformRaw[0], sizeof(FTransform));

	FPoseWatchMessage Message;
	Message.RecordingTime = InRecordingTime;
	Message.ComponentId = InComponentId;
	Message.AnimInstanceId = InAnimInstanceId;
	Message.PoseWatchId = PoseWatchId;
	Message.NameId = NameId;
	Message.Color = Color;
	Message.BoneTransformsStartIndex = SkeletalMeshPoseTransforms.Num();
	Message.CurveStartIndex = SkeletalMeshCurves.Num();
	Message.NumBoneTransforms = BoneTransformsRaw.Num() / (sizeof(FTransform) / sizeof(float));
	Message.WorldTransform = WorldTransform;
	Message.RequiredBonesStartIndex = PoseWatchRequiredBones.Num();
	Message.NumRequiredBones = RequiredBones.Num();
	Message.NumCurves = NumCurves;
	Message.bIsEnabled = bIsEnabled;

	Timeline->AppendEvent(InTime, Message);

	const int CaptureTransformSize = sizeof(FTransform) / sizeof(float); // TODO
	const int LocalTransformSize = sizeof(FTransform) / sizeof(float);
	const int PoseTransformCount = BoneTransformsRaw.Num() / CaptureTransformSize;

	// Dump PoseBoneTransformsRaw into SkeletalMeshPoseTransforms
	if (CaptureTransformSize == LocalTransformSize)
	{
		for (int i = 0; i < PoseTransformCount; i++)
		{
			FMemory::Memcpy(&SkeletalMeshPoseTransforms.PushBack(), &BoneTransformsRaw[CaptureTransformSize * i], sizeof(FTransform));
		}
	}
	else
	{
		for (int i = 0; i < PoseTransformCount; i++)
		{
			SkeletalMeshPoseTransforms.PushBack() = ConvertTransform(CaptureTransformSize, &BoneTransformsRaw[CaptureTransformSize * i]);
		}
	}

	for(int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
	{
		SkeletalMeshCurves.PushBack() = { CurveIds[CurveIndex], CurveValues[CurveIndex] };
		TimelineStorage->AllCurveIds.Add(CurveIds[CurveIndex]);
	}
	
	// Dump RequiredBones into PoseWatchRequiredBones
	for (const uint16 RequiredBone : RequiredBones)
	{
		PoseWatchRequiredBones.PushBack() = RequiredBone;
	}

	UpdateSessionDurationSeconds(InTime);
}

void FAnimationProvider::AppendInertialization(uint64 InAnimInstanceId, double InProfileTime, double InRecordingTime, int32 InNodeId, float InWeight, EInertializationType InType)
{
	EditAccessCheck();
	
	TSharedPtr<FInertializationTimelineStorage> TimelineStorage;
	uint32* IndexPtr = ObjectIdToInertializationTimelines.Find(InAnimInstanceId);
	if(IndexPtr != nullptr)
	{
		TimelineStorage = InertializationTimelineStorage[*IndexPtr];
	}
	else
	{
		TimelineStorage = MakeShared<FInertializationTimelineStorage>();
		TimelineStorage->Timeline = MakeShared<TraceServices::TPointTimeline<FInertializationMessage>>(Session.GetLinearAllocator());
		TimelineStorage->Timeline->SetEnumerateOutsideRange(true);
		ObjectIdToInertializationTimelines.Add(InAnimInstanceId, InertializationTimelineStorage.Num());
		InertializationTimelineStorage.Add(TimelineStorage.ToSharedRef());
	}
	
	
	FInertializationMessage Message;
	Message.AnimInstanceId = InAnimInstanceId;
	Message.RecordingTime = InRecordingTime;
	Message.NodeId = InNodeId;
	Message.Weight = InWeight;
	Message.Type = InType;
	TimelineStorage->Timeline->AppendEvent(InProfileTime, Message);
	TimelineStorage->NodeTypes.Add(InNodeId, InType);
}

#undef LOCTEXT_NAMESPACE
