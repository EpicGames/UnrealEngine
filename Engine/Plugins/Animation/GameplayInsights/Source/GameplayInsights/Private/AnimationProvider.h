// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAnimationProvider.h"
#include "Common/PagedArray.h"
#include "Common/ProviderLock.h"
#include "Model/IntervalTimeline.h"
#include "Model/PointTimeline.h"
#include "Containers/ArrayView.h"
#include "Model/IntervalTimeline.h"

namespace TraceServices
{
	class IAnalysisSession;
	extern thread_local FProviderLock::FThreadLocalState GAnimationProviderLockState;
}
class FGameplayProvider;
struct FObjectInfo;
class USkeletalMesh;

class FAnimationProvider : public IAnimationProvider, public TraceServices::IEditableProvider
{
public:
	static FName ProviderName;

	FAnimationProvider(TraceServices::IAnalysisSession& InSession, FGameplayProvider& InGameplayProvider);

	//~ IProvider (read lock)
	virtual void BeginRead() const override
	{
		Lock.BeginRead(TraceServices::GAnimationProviderLockState);
	}

	virtual void EndRead() const override
	{
		Lock.EndRead(TraceServices::GAnimationProviderLockState);
	}

	virtual void ReadAccessCheck() const override
	{
		Lock.ReadAccessCheck(TraceServices::GAnimationProviderLockState);
	}

	//~ IEditableProvider (edit lock)
	virtual void BeginEdit() const override
	{
		Lock.BeginWrite(TraceServices::GAnimationProviderLockState);
	}

	virtual void EndEdit() const override
	{
		Lock.EndWrite(TraceServices::GAnimationProviderLockState);
	}

	virtual void EditAccessCheck() const override
	{
		Lock.WriteAccessCheck(TraceServices::GAnimationProviderLockState);
	}

	/** IAnimationProvider interface */
	virtual void EnumerateSkeletalMeshPoseTimelines(TFunctionRef<void(uint64 ObjectId, const SkeletalMeshPoseTimeline&)> Callback) const override;
	virtual bool ReadSkeletalMeshPoseTimeline(uint64 InObjectId, TFunctionRef<void(const SkeletalMeshPoseTimeline&, bool)> Callback) const override;
	virtual void GetSkeletalMeshComponentSpacePose(const FSkeletalMeshPoseMessage& InMessage, const FSkeletalMeshInfo& InMeshInfo, FTransform& OutComponentToWorld, TArray<FTransform>& OutTransforms) const override;
	virtual void GetPoseWatchData(const FPoseWatchMessage& InMessage, TArray<FTransform>& BoneTransforms, TArray<uint16>& RequiredBones) const;
	virtual void EnumerateSkeletalMeshCurveIds(uint64 InObjectId, TFunctionRef<void(uint32)> Callback) const override;
	virtual void EnumerateSkeletalMeshCurves(const FSkeletalMeshPoseMessage& InMessage, TFunctionRef<void(const FSkeletalMeshNamedCurve&)> Callback) const override;
	virtual void EnumeratePoseWatchCurves(const FPoseWatchMessage& InMessage, TFunctionRef<void(const FSkeletalMeshNamedCurve&)> Callback) const override;
	virtual void EnumerateExternalMorphSets(const FSkeletalMeshPoseMessage& InMessage, TFunctionRef<void(const FExternalMorphWeightMessage&)> Callback) const override;
	virtual bool ReadTickRecordTimeline(uint64 InObjectId, TFunctionRef<void(const TickRecordTimeline&)> Callback) const override;
	virtual bool ReadInertializationTimeline(uint64 InObjectId, TFunctionRef<void(const InertializationTimeline&)> Callback) const override;
	virtual void EnumerateInertializationNodes(uint64 InObjectId, TFunctionRef<void(int32 NodeId, EInertializationType Type)> Callback) const override;
	virtual void EnumerateTickRecordIds(uint64 InObjectId, TFunctionRef<void(uint64, int32)> Callback) const override;
	virtual void EnumerateAnimGraphTimelines(TFunctionRef<void(uint64 ObjectId, const AnimGraphTimeline&)> Callback) const override;
	virtual bool ReadAnimGraphTimeline(uint64 InObjectId, TFunctionRef<void(const AnimGraphTimeline&)> Callback) const override;
	virtual bool ReadAnimNodesTimeline(uint64 InObjectId, TFunctionRef<void(const AnimNodesTimeline&)> Callback) const override;
	virtual bool ReadAnimNodeValuesTimeline(uint64 InObjectId, TFunctionRef<void(const AnimNodeValuesTimeline&)> Callback) const override;
	virtual bool ReadAnimAttributesTimeline(uint64 InObjectId, TFunctionRef<void(const AnimAttributeTimeline&)> Callback) const override;
	virtual bool ReadAnimSequencePlayersTimeline(uint64 InObjectId, TFunctionRef<void(const AnimSequencePlayersTimeline&)> Callback) const override;
	virtual bool ReadAnimBlendSpacePlayersTimeline(uint64 InObjectId, TFunctionRef<void(const BlendSpacePlayersTimeline&)> Callback) const override;
	virtual bool ReadStateMachinesTimeline(uint64 InObjectId, TFunctionRef<void(const StateMachinesTimeline&)> Callback) const override;
	virtual bool ReadNotifyTimeline(uint64 InObjectId, TFunctionRef<void(const AnimNotifyTimeline&)> Callback) const override;
	virtual void EnumerateNotifyStateTimelines(uint64 InObjectId, TFunctionRef<void(uint64, const AnimNotifyTimeline&)> Callback) const override;
	virtual bool ReadMontageTimeline(uint64 InObjectId, TFunctionRef<void(const AnimMontageTimeline&)> Callback) const override;
	virtual void EnumerateMontageIds(uint64 InObjectId, TFunctionRef<void(uint64)> Callback) const override;
	virtual bool ReadAnimSyncTimeline(uint64 InObjectId, TFunctionRef<void(const AnimSyncTimeline&)> Callback) const override;
	virtual bool ReadPoseWatchTimeline(uint64 InObjectId, TFunctionRef<void(const PoseWatchTimeline&)> Callback) const override;
	virtual const FSkeletalMeshInfo* FindSkeletalMeshInfo(uint64 InObjectId) const override;
	virtual const FAnimNodeInfo* FindAnimNodeInfo(int32 InNodeId, uint64 InAnimInstanceId) const override;
	virtual const TCHAR* GetName(uint32 InId) const override;
	virtual FText FormatNodeKeyValue(const FAnimNodeValueMessage& InMessage) const override;
	virtual FText FormatNodeValue(const FAnimNodeValueMessage& InMessage) const override;

	/** Does the given object have external morph sets? */
	bool HasExternalMorphSets(uint64 InObjectId) const;

	// Check to see if any data is present for this provider. Used as an early-out when processing.
	bool HasAnyData() const;

	/** Add a tick record */
	void AppendTickRecord(uint64 InAnimInstanceId, double InProfileTime, double InRecordingTime, uint64 InAssetId, int32 InNodeId, float InBlendWeight, float InPlaybackTime, float InRootMotionWeight, float InPlayRate, float InBlendSpacePositionX, float InBlendSpacePositionY, float InBlendSpaceFilteredPositionX, float InBlendSpaceFilteredPositionY, uint16 InFrameCounter, bool bInLooping, bool bInIsBlendSpace);
	void AppendTickRecord(uint64 InAnimInstanceId, double InProfileTime, double InRecordingTime, uint64 InAssetId, int32 InNodeId, float InBlendWeight, float InPlaybackTime, float InRootMotionWeight, float InPlayRate, float InBlendSpacePositionX, float InBlendSpacePositionY, float InBlendSpaceFilteredPositionX, float InBlendSpaceFilteredPositionY, uint16 InFrameCounter, bool bInLooping, bool bInIsBlendSpace, uint32 InSyncGroupName, bool bInIsSyncGroupLeader, float InSyncLeaderScore, float TimeToPrevMarker, float TimeToNextMarker, int32 PrevMarkerIndex, int32 NextMarkerIndex);

	/** Add a skeletal mesh */
	void AppendSkeletalMesh(uint64 InObjectId, uint64 SkeletonId, const TArrayView<const int32>& ParentIndices);

	/** Add a skeletal mesh pose/curves etc. */
	void AppendSkeletalMeshComponent(uint64 InObjectId, uint64 InMeshId, double InProfileTime, double InRecordingTime, uint16 InLodIndex, uint16 InFrameCounter, const TArrayView<const FTransform>& InPose, const TArrayView<const FSkeletalMeshNamedCurve>& InCurves);
	void AppendSkeletalMeshComponent(uint64 InObjectId, uint64 InMeshId, double InProfileTime, double InRecordingTime, uint16 InLodIndex, uint16 InFrameCounter, const TArrayView<const float>& InComponentToWorldRaw, const TArrayView<const float>& InPoseRaw, const TArrayView<const uint32>& InCurveIds, const TArrayView<const float>& InCurveValues);
	void AppendSkeletalMeshComponent(uint64 InObjectId, uint64 InMeshId, double InProfileTime, double InRecordingTime, uint16 InLodIndex, uint16 InFrameCounter, bool bInIsVisible, const TArrayView<const float>& InComponentToWorldRaw, const TArrayView<const float>& InPoseRaw, const TArrayView<const uint32>& InCurveIds, const TArrayView<const float>& InCurveValues, const TArrayView<const float>& InExternalMorphWeights, const TArrayView<const int32>& InExternalMorphCounts);

	/** Get a skeletal mesh for a specified path. If the mesh no longer exists, make a fake one */
	USkeletalMesh* GetSkeletalMesh(const TCHAR* InPath);

	/** Add a name, referenced by ID (for curves etc.) */
	void AppendName(uint32 InId, const TCHAR* InName);

	/** Add a skeletal mesh frame */
	void AppendSkeletalMeshFrame(uint64 InObjectId, double InTime, uint16 InFrameCounter);
	
	/** Add an anim graph */
	void AppendAnimGraph(uint64 InAnimInstanceId, double InStartTime, double InEndTime, int32 InNodeCount, uint16 InFrameCounter, uint8 InPhase);

	/** Add an anim node */
	void AppendAnimNodeStart(uint64 InAnimInstanceId, double InStartTime, uint16 InFrameCounter, int32 InTargetLinkId, int32 InSourceLinkId, const TCHAR* InTargetNodeName, float InWeight, float InRootMotionWeight, const TCHAR* InTargetNodeDisplayName, uint8 InPhase);

	/** Add anim node values */
	void AppendAnimNodeValue(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, bool bInValue);
	void AppendAnimNodeValue(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, int32 InValue);
	void AppendAnimNodeValue(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, float InValue);
	void AppendAnimNodeValue(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, const FVector2D& InValue);
	void AppendAnimNodeValue(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, const FVector& InValue);
	void AppendAnimNodeValue(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, const TCHAR* InValue);
	void AppendAnimNodeValueObject(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, uint64 InValue);
	void AppendAnimNodeValueClass(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, uint64 InValue);
	void AppendAnimNodeValueAnimNode(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, int32 InValue, uint64 InValueAnimInstanceId);

	/** Add anim graph attribute */
	void AppendAnimGraphAttribute(uint64 InSourceAnimInstanceId, uint64 InTargetAnimInstanceId, double InTime, int32 InSourceNodeId, int32 InTargetNodeId, uint32 InAttributeNameId);

	/** Add a sequence player record */
	void AppendAnimSequencePlayer(uint64 InAnimInstanceId, double InTime, int32 InNodeId, float InPosition, float InLength, uint16 InFrameCounter);

	/** Add a blend space player record */
	void AppendBlendSpacePlayer(uint64 InAnimInstanceId, double InTime, int32 InNodeId, uint64 InBlendSpaceId, const FVector& InBlendPosition, const FVector& InFilteredBlendPosition);

	/** Add a state machine state */
	void AppendStateMachineState(uint64 InAnimInstanceId, double InTime, int32 InNodeId, int32 InStateMachineIndex, int32 InStateIndex, float InStateWeight, float InElapsedTime);

	/** Add an anim notify */
	void AppendNotify(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint64 InAssetId, uint64 InNotifyId, uint32 InNameId, float InNotifyTime, float InNotifyDuration, EAnimNotifyMessageType InNotifyEventType);

	/** Append montage data */
	void AppendMontage(uint64 InAnimInstanceId, double InProfileTime, double InRecordingTime, uint64 InMontageId, uint32 InCurrentSectionNameId, uint32 InNextSectionNameId, float InWeight, float InDesiredWeight, float InPosition, uint16 InFrameCounter);

	/** Append sync data */
	void AppendSync(uint64 InAnimInstanceId, double InTime, int32 InSourceNodeId, uint32 InGroupNameId);

	/** Append pose watch data */
	void AppendPoseWatch(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint64 PoseWatchId, const TArrayView<const float>& BoneTransformsRaw, const TArrayView<const uint16>& RequiredBones, const TArrayView<const float>& WorldTransformRaw, const bool bIsEnabled);
	void AppendPoseWatch(uint64 InComponentId, uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint64 PoseWatchId, uint32 NameId, FColor Color, const TArrayView<const float>& BoneTransformsRaw, const TArrayView<const uint32>& CurveIds, const TArrayView<const float>& CurveValues, const TArrayView<const uint16>& RequiredBones, const TArrayView<const float>& WorldTransformRaw, const bool bIsEnabled);

	/** Append Inertialization data */
	void AppendInertialization(uint64 InAnimInstanceId, double InProfileTime, double InRecordingTime, int32 NodeId, float InWeight, EInertializationType InType);

private:
	/** Add anim node values helper */
	void AppendAnimNodeValue(uint64 InAnimInstanceId, double InTime, double InRecordingTime, uint16 InFrameCounter, int32 InNodeId, const TCHAR* InKey, FAnimNodeValueMessage& InMessage);

	/** Handle an object ending play - terminate existing scopes */
	void HandleObjectEndPlay(uint64 InObjectId, double InTime, const FObjectInfo& InObjectInfo);

	void UpdateSessionDurationSeconds(double InTime) const;

private:
	TraceServices::IAnalysisSession& Session;
	mutable TraceServices::FProviderLock Lock;
	FGameplayProvider& GameplayProvider;

	/** Maps into timeline arrays per-object */
	TMap<uint64, uint32> ObjectIdToTickRecordTimelineStorage;
	TMap<uint64, uint32> ObjectIdToSkeletalMeshPoseTimelines;
	TMap<uint64, uint32> ObjectIdToSkeletalMeshFrameTimelines;
	TMap<uint64, uint32> ObjectIdToAnimGraphTimelines;
	TMap<uint64, uint32> ObjectIdToAnimNodeTimelines;
	TMap<uint64, uint32> ObjectIdToStateMachineTimelines;
	TMap<uint64, uint32> ObjectIdToAnimNodeValueTimelines;
	TMap<uint64, uint32> ObjectIdToAnimSequencePlayerTimelines;
	TMap<uint64, uint32> ObjectIdToBlendSpacePlayerTimelines;
	TMap<uint64, uint32> ObjectIdToAnimNotifyStateTimelines;
	TMap<uint64, uint32> ObjectIdToAnimNotifyTimelines;
	TMap<uint64, uint32> ObjectIdToAnimMontageTimelines;
	TMap<uint64, uint32> ObjectIdToInertializationTimelines;
	TMap<uint64, uint32> ObjectIdToAnimAttributeTimelines;
	TMap<uint64, uint32> ObjectIdToAnimSyncTimelines;
	TMap<uint64, uint32> ObjectIdToPoseWatchTimelines;

	/** All the skeletal mesh info we have seen, grow only for stable indices */
	TArray<FSkeletalMeshInfo> SkeletalMeshInfos;

	/** Map of skeletal mesh->index into SkeletalMeshInfos index */
	TMap<uint64, uint32> SkeletalMeshIdToIndexMap;

	/** All the anim node info we have seen, grow only for stable indices */
	TArray<FAnimNodeInfo> AnimNodeInfos;

	/** Map of (node, anim instance)->index into AnimNodeInfos index */
	TMap<TTuple<int32, uint64>, uint32> AnimNodeIdToIndexMap;

	/** Map of name IDs to name string */
	TMap<uint32, const TCHAR*> NameMap;
	
	struct FTickRecordTimelineStorage
	{
		TSet<TTuple<uint64, int32>> AssetIdAndPlayers;
		TSharedPtr<TraceServices::TPointTimeline<FTickRecordMessage>> Timeline;
	};

	struct FSkeletalMeshTimelineStorage
	{
		TSharedPtr<TraceServices::TIntervalTimeline<FSkeletalMeshPoseMessage>> Timeline;
		TSet<uint32> AllCurveIds;
		int32 NumExternalMorphSets = 0;
	};

	struct FAnimNotifyStateTimelineStorage
	{
		TMap<uint64, uint32> NotifyIdToAnimNotifyStateTimeline;
		TArray<TSharedRef<TraceServices::TIntervalTimeline<FAnimNotifyMessage>>> Timelines;
	};

	struct FMontageTimelineStorage
	{
		TSharedPtr<TraceServices::TPointTimeline<FAnimMontageMessage>> Timeline;
		TSet<uint64> AllMontageIds;
	};
	
	struct FInertializationTimelineStorage
	{
		TSharedPtr<TraceServices::TPointTimeline<FInertializationMessage>> Timeline;
		TMap<int32,EInertializationType> NodeTypes;
	};

	/** Message storage */
	TArray<TSharedRef<FTickRecordTimelineStorage>> TickRecordTimelineStorage;
	TArray<TSharedRef<FSkeletalMeshTimelineStorage>> SkeletalMeshPoseTimelineStorage;
	TArray<TSharedRef<TraceServices::TIntervalTimeline<FSkeletalMeshFrameMessage>>> SkeletalMeshFrameTimelines;
	TArray<TSharedRef<TraceServices::TIntervalTimeline<FAnimGraphMessage>>> AnimGraphTimelines;
	TArray<TSharedRef<TraceServices::TPointTimeline<FAnimNodeMessage>>> AnimNodeTimelines;
	TArray<TSharedRef<TraceServices::TPointTimeline<FAnimNodeValueMessage>>> AnimNodeValueTimelines;
	TArray<TSharedRef<TraceServices::TPointTimeline<FAnimSequencePlayerMessage>>> AnimSequencePlayerTimelines;
	TArray<TSharedRef<TraceServices::TPointTimeline<FBlendSpacePlayerMessage>>> BlendSpacePlayerTimelines;
	TArray<TSharedRef<TraceServices::TPointTimeline<FAnimStateMachineMessage>>> StateMachineTimelines;
	TArray<TSharedRef<FAnimNotifyStateTimelineStorage>> AnimNotifyStateTimelineStorage;
	TArray<TSharedRef<TraceServices::TPointTimeline<FAnimNotifyMessage>>> AnimNotifyTimelines;
	TArray<TSharedRef<FMontageTimelineStorage>> AnimMontageTimelineStorage;
	TArray<TSharedRef<FInertializationTimelineStorage>> InertializationTimelineStorage;
	TraceServices::TPagedArray<FTransform> SkeletalMeshPoseTransforms;
	TraceServices::TPagedArray<FSkeletalMeshNamedCurve> SkeletalMeshCurves;
	TraceServices::TPagedArray<FExternalMorphWeightMessage> ExternalMorphWeights;
	TraceServices::TPagedArray<int32> SkeletalMeshParentIndices;
	TArray<TSharedRef<TraceServices::TPointTimeline<FAnimAttributeMessage>>> AnimAttributeTimelines;
	TArray<TSharedRef<TraceServices::TPointTimeline<FAnimSyncMessage>>> AnimSyncTimelines;
	TArray<TSharedRef<TraceServices::TPointTimeline<FPoseWatchMessage>>> PoseWatchTimelines;
	TraceServices::TPagedArray<uint16> PoseWatchRequiredBones;

	// Flag to indicate if any data is present
	bool bHasAnyData;
};
