// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationSharedData.h"

#include "AnimationProvider.h"
#include "AnimationTickRecordsTrack.h"
#include "AnimNodesTrack.h"
#include "AnimNotifiesTrack.h"
#include "Common/ProviderLock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameplayInsightsModule.h"
#include "GameplayProvider.h"
#include "GameplaySharedData.h"
#include "GameplayTimingViewExtender.h"
#include "Modules/ModuleManager.h"
#include "MontageTrack.h"
#include "ObjectEventsTrack.h"
#include "SAnimGraphSchematicView.h"
#include "SkeletalMeshCurvesTrack.h"
#include "SkeletalMeshPoseTrack.h"
#include "STrackVariantValueView.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Widgets/Docking/SDockTab.h"

// TraceInsights
#include "Insights/ITimingViewSession.h"

#if WITH_EDITOR
#include "EditorViewportClient.h"
#include "Editor/EditorEngine.h"
#endif

#define LOCTEXT_NAMESPACE "AnimationSharedData"

FAnimationSharedData::FAnimationSharedData(FGameplaySharedData& InGameplaySharedData)
	: GameplaySharedData(InGameplaySharedData)
	, AnalysisSession(nullptr)
	, MarkerTime(0.0)
	, bTimeMarkerValid(false)
	, bMarkerFrameValid(false)
	, bSkeletalMeshPoseTracksEnabled(true)
	, bSkeletalMeshCurveTracksEnabled(true)
	, bTickRecordTracksEnabled(true)
	, bAnimNodeTracksEnabled(true)
	, bAnimNotifyTracksEnabled(true)
	, bMontageTracksEnabled(true)
	, bShowTransportControls(false)
{
}

void FAnimationSharedData::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession)
{
	TimingViewSession = &InTimingViewSession;

	SkeletalMeshPoseTracks.Reset();
	AnimationTickRecordsTracks.Reset();

	TimeMarkerChangedHandle = InTimingViewSession.OnTimeMarkerChanged().AddRaw(this, &FAnimationSharedData::OnTimeMarkerChanged);
}

void FAnimationSharedData::OnEndSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession)
{
	SkeletalMeshPoseTracks.Reset();
	AnimationTickRecordsTracks.Reset();

	InTimingViewSession.OnTimeMarkerChanged().Remove(TimeMarkerChangedHandle);

	TimingViewSession = nullptr;
}

void FAnimationSharedData::Tick(UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	AnalysisSession = &InAnalysisSession;

	const FAnimationProvider* AnimationProvider = InAnalysisSession.ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
	const FGameplayProvider* GameplayProvider = InAnalysisSession.ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);

	if(AnimationProvider && GameplayProvider)
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);

		if(AnimationProvider->HasAnyData() && AreAnyAnimationTracksEnabled())
		{
			// Add tracks for each tracked object's animation data
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			GameplayProvider->EnumerateObjects([this, &InTimingViewSession, &InAnalysisSession, &AnimationProvider](const FObjectInfo& InObjectInfo)
			{
				TSharedPtr<FObjectEventsTrack> ObjectEventsTrack;
				const uint64 ObjectId = InObjectInfo.GetUObjectId();
				AnimationProvider->ReadSkeletalMeshPoseTimeline(ObjectId, [this, ObjectId, &InObjectInfo, &ObjectEventsTrack, &InTimingViewSession, &InAnalysisSession](const IAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bInHasCurves)
				{
					if(!ObjectEventsTrack.IsValid())
					{
						ObjectEventsTrack = GameplaySharedData.GetObjectEventsTrackForId(InTimingViewSession, InAnalysisSession, InObjectInfo);
					}

					auto FindSkeletalMeshPoseTrack = [](const FBaseTimingTrack& InTrack)
					{
						return InTrack.Is<FSkeletalMeshPoseTrack>();
					};

					TSharedPtr<FSkeletalMeshPoseTrack> ExistingSkeletalMeshPoseTrack = StaticCastSharedPtr<FSkeletalMeshPoseTrack>(ObjectEventsTrack->GetGameplayTrack().FindChildTrack(ObjectId, FindSkeletalMeshPoseTrack));
					if(!ExistingSkeletalMeshPoseTrack.IsValid())
					{
						TSharedPtr<FSkeletalMeshPoseTrack> SkeletalMeshPoseTrack = MakeShared<FSkeletalMeshPoseTrack>(*this, ObjectId, InObjectInfo.Name);
						SkeletalMeshPoseTrack->SetVisibilityFlag(bSkeletalMeshPoseTracksEnabled);
						SkeletalMeshPoseTracks.Add(SkeletalMeshPoseTrack.ToSharedRef());

						InTimingViewSession.AddScrollableTrack(SkeletalMeshPoseTrack);
						GameplaySharedData.InvalidateObjectTracksOrder();

						ObjectEventsTrack->GetGameplayTrack().AddChildTrack(SkeletalMeshPoseTrack->GetGameplayTrack());

						GameplaySharedData.MakeTrackAndAncestorsVisible(ObjectEventsTrack.ToSharedRef(), true);
					}

					if(bInHasCurves)
					{
						auto FindSkeletalMeshCurvesTrack = [](const FBaseTimingTrack& InTrack)
						{
							return InTrack.Is<FSkeletalMeshCurvesTrack>();
						};

						TSharedPtr<FSkeletalMeshCurvesTrack> ExistingSkeletalMeshCurvesTrack = StaticCastSharedPtr<FSkeletalMeshCurvesTrack>(ObjectEventsTrack->GetGameplayTrack().FindChildTrack(ObjectId, FindSkeletalMeshCurvesTrack));
						if(!ExistingSkeletalMeshCurvesTrack.IsValid())
						{
							TSharedPtr<FSkeletalMeshCurvesTrack> SkeletalMeshCurvesTrack = MakeShared<FSkeletalMeshCurvesTrack>(*this, ObjectId, InObjectInfo.Name);
							SkeletalMeshCurvesTrack->SetVisibilityFlag(bSkeletalMeshCurveTracksEnabled);
							SkeletalMeshCurvesTracks.Add(SkeletalMeshCurvesTrack.ToSharedRef());

							InTimingViewSession.AddScrollableTrack(SkeletalMeshCurvesTrack);
							GameplaySharedData.InvalidateObjectTracksOrder();

							ObjectEventsTrack->GetGameplayTrack().AddChildTrack(SkeletalMeshCurvesTrack->GetGameplayTrack());

							GameplaySharedData.MakeTrackAndAncestorsVisible(ObjectEventsTrack.ToSharedRef(), true);
						}
					}
				});

				AnimationProvider->ReadTickRecordTimeline(ObjectId, [this, ObjectId, &InObjectInfo, &ObjectEventsTrack, &InTimingViewSession, &InAnalysisSession](const IAnimationProvider::TickRecordTimeline& InTimeline)
				{
					if(!ObjectEventsTrack.IsValid())
					{
						ObjectEventsTrack = GameplaySharedData.GetObjectEventsTrackForId(InTimingViewSession, InAnalysisSession, InObjectInfo);
					}

					auto FindTickRecordTrackWithAssetId = [](const FBaseTimingTrack& InTrack)
					{
						return InTrack.Is<FAnimationTickRecordsTrack>();
					};

					TSharedPtr<FAnimationTickRecordsTrack> ExistingAnimationTickRecordsTrack = StaticCastSharedPtr<FAnimationTickRecordsTrack>(ObjectEventsTrack->GetGameplayTrack().FindChildTrack(ObjectId, FindTickRecordTrackWithAssetId));
					if(!ExistingAnimationTickRecordsTrack.IsValid())
					{
						TSharedPtr<FAnimationTickRecordsTrack> AnimationTickRecordsTrack = MakeShared<FAnimationTickRecordsTrack>(*this, ObjectId, InObjectInfo.Name);
						AnimationTickRecordsTrack->SetVisibilityFlag(bTickRecordTracksEnabled);
						AnimationTickRecordsTracks.Add(AnimationTickRecordsTrack.ToSharedRef());

						InTimingViewSession.AddScrollableTrack(AnimationTickRecordsTrack);
						GameplaySharedData.InvalidateObjectTracksOrder();

						ObjectEventsTrack->GetGameplayTrack().AddChildTrack(AnimationTickRecordsTrack->GetGameplayTrack());

						GameplaySharedData.MakeTrackAndAncestorsVisible(ObjectEventsTrack.ToSharedRef(), true);
					}
				});

				AnimationProvider->ReadAnimGraphTimeline(ObjectId, [this, ObjectId, &InObjectInfo, &ObjectEventsTrack, &InTimingViewSession, &InAnalysisSession](const IAnimationProvider::AnimGraphTimeline& InTimeline)
				{
					if(!ObjectEventsTrack.IsValid())
					{
						ObjectEventsTrack = GameplaySharedData.GetObjectEventsTrackForId(InTimingViewSession, InAnalysisSession, InObjectInfo);
					}

					auto FindAnimNodesTrack = [](const FBaseTimingTrack& InTrack)
					{
						return InTrack.Is<FAnimNodesTrack>();
					};

					TSharedPtr<FAnimNodesTrack> ExistingAnimNodesTrack = StaticCastSharedPtr<FAnimNodesTrack>(ObjectEventsTrack->GetGameplayTrack().FindChildTrack(ObjectId, FindAnimNodesTrack));
					if(!ExistingAnimNodesTrack.IsValid())
					{
						TSharedPtr<FAnimNodesTrack> AnimNodesTrack = MakeShared<FAnimNodesTrack>(*this, ObjectId, InObjectInfo.Name);
						AnimNodesTrack->SetVisibilityFlag(bAnimNodeTracksEnabled);
						AnimNodesTracks.Add(AnimNodesTrack.ToSharedRef());

						InTimingViewSession.AddScrollableTrack(AnimNodesTrack);
						GameplaySharedData.InvalidateObjectTracksOrder();

						ObjectEventsTrack->GetGameplayTrack().AddChildTrack(AnimNodesTrack->GetGameplayTrack());

						GameplaySharedData.MakeTrackAndAncestorsVisible(ObjectEventsTrack.ToSharedRef(), true);
					}
				});

				auto ProcessNotifyTimeline = [this, ObjectId, &InObjectInfo, &ObjectEventsTrack, &InTimingViewSession, &InAnalysisSession]()
				{
					if(!ObjectEventsTrack.IsValid())
					{
						ObjectEventsTrack = GameplaySharedData.GetObjectEventsTrackForId(InTimingViewSession, InAnalysisSession, InObjectInfo);
					}

					auto FindAnimNotifyTrack = [](const FBaseTimingTrack& InTrack)
					{
						return InTrack.Is<FAnimNotifiesTrack>();
					};

					TSharedPtr<FAnimNotifiesTrack> ExistingAnimNotifyTrack = StaticCastSharedPtr<FAnimNotifiesTrack>(ObjectEventsTrack->GetGameplayTrack().FindChildTrack(ObjectId, FindAnimNotifyTrack));
					if(!ExistingAnimNotifyTrack.IsValid())
					{
						TSharedPtr<FAnimNotifiesTrack> AnimNotifyTrack = MakeShared<FAnimNotifiesTrack>(*this, ObjectId, InObjectInfo.Name);
						AnimNotifyTrack->SetVisibilityFlag(bAnimNotifyTracksEnabled);
						AnimNotifyTracks.Add(AnimNotifyTrack.ToSharedRef());

						InTimingViewSession.AddScrollableTrack(AnimNotifyTrack);
						GameplaySharedData.InvalidateObjectTracksOrder();

						ObjectEventsTrack->GetGameplayTrack().AddChildTrack(AnimNotifyTrack->GetGameplayTrack());

						GameplaySharedData.MakeTrackAndAncestorsVisible(ObjectEventsTrack.ToSharedRef(), true);
					}
				};

				AnimationProvider->ReadNotifyTimeline(ObjectId, [&ProcessNotifyTimeline](const IAnimationProvider::AnimNotifyTimeline& InTimeline)
				{
					ProcessNotifyTimeline();
				});

				AnimationProvider->EnumerateNotifyStateTimelines(ObjectId, [&ProcessNotifyTimeline](uint32 InNotifyNameId, const IAnimationProvider::AnimNotifyTimeline& InTimeline)
				{
					ProcessNotifyTimeline();
				});

				AnimationProvider->ReadMontageTimeline(ObjectId, [this, ObjectId, &InObjectInfo, &ObjectEventsTrack, &InTimingViewSession, &InAnalysisSession](const IAnimationProvider::AnimMontageTimeline& InTimeline)
				{
					if(!ObjectEventsTrack.IsValid())
					{
						ObjectEventsTrack = GameplaySharedData.GetObjectEventsTrackForId(InTimingViewSession, InAnalysisSession, InObjectInfo);
					}

					auto FindMontageTrack = [](const FBaseTimingTrack& InTrack)
					{
						return InTrack.Is<FMontageTrack>();
					};

					TSharedPtr<FMontageTrack> ExistingMontageTrack = StaticCastSharedPtr<FMontageTrack>(ObjectEventsTrack->GetGameplayTrack().FindChildTrack(ObjectId, FindMontageTrack));
					if(!ExistingMontageTrack.IsValid())
					{
						TSharedPtr<FMontageTrack> MontageTrack = MakeShared<FMontageTrack>(*this, ObjectId, InObjectInfo.Name);
						MontageTrack->SetVisibilityFlag(bMontageTracksEnabled);
						MontageTracks.Add(MontageTrack.ToSharedRef());

						InTimingViewSession.AddScrollableTrack(MontageTrack);
						GameplaySharedData.InvalidateObjectTracksOrder();

						ObjectEventsTrack->GetGameplayTrack().AddChildTrack(MontageTrack->GetGameplayTrack());

						GameplaySharedData.MakeTrackAndAncestorsVisible(ObjectEventsTrack.ToSharedRef(), true);
					}
				});
			});
		}
	}

	// Prevent mouse movement throttling if we are drawing stuff that can change when the mouse is dragged
	for(TSharedRef<FSkeletalMeshPoseTrack> PoseTrack : SkeletalMeshPoseTracks)
	{
		if(PoseTrack->IsVisible())
		{
			if(PoseTrack->ShouldDrawPose())
			{
				InTimingViewSession.PreventThrottling();
				break;
			}
		}
	}
}

void FAnimationSharedData::ExtendFilterMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("AnimationTracks", LOCTEXT("AnimationHeader", "Animation"));
	{
#if WITH_EDITOR
		InMenuBuilder.AddMenuEntry(
			LOCTEXT("ShowAllMeshes", "Show All Mesh Poses"),
			LOCTEXT("ShowAllMeshes_Tooltip", "Show all skeletal mesh poses"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FAnimationSharedData::ShowAllMeshes)),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("HideAllMeshes", "Hide All Mesh Poses"),
			LOCTEXT("HideAllMeshes_Tooltip", "Hide all skeletal mesh poses"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FAnimationSharedData::HideAllMeshes)),
			NAME_None,
			EUserInterfaceActionType::Button
		);
#endif

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleAnimationTracks", "Animation Tracks"),
			LOCTEXT("ToggleAnimationTracks_Tooltip", "Show/hide all animation tracks"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FAnimationSharedData::ToggleAnimationTracks),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FAnimationSharedData::AreAllAnimationTracksEnabled)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleSkelMeshPoseTracks", "Pose Tracks"),
			LOCTEXT("ToggleSkelMeshPoseTracks_Tooltip", "Show/hide the skeletal mesh pose tracks"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FAnimationSharedData::ToggleSkeletalMeshPoseTracks),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this](){ return bSkeletalMeshPoseTracksEnabled; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleSkelMeshCurveTracks", "Curve Tracks"),
			LOCTEXT("ToggleSkelMeshCurveTracks_Tooltip", "Show/hide the skeletal mesh curve tracks"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FAnimationSharedData::ToggleSkeletalMeshCurveTracks),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this](){ return bSkeletalMeshCurveTracksEnabled; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleAnimTickRecordTracks", "Blend Weights Tracks"),
			LOCTEXT("ToggleAnimTickRecordTracks_Tooltip", "Show/hide the blend weights (tick records) tracks"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FAnimationSharedData::ToggleTickRecordTracks),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this](){ return bTickRecordTracksEnabled; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleAnimNodeTracks", "Graph Tracks"),
			LOCTEXT("ToggleAnimNodeTracks_Tooltip", "Show/hide the animation graph tracks"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FAnimationSharedData::ToggleAnimNodeTracks),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this](){ return bAnimNodeTracksEnabled; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleAnimNotifyTracks", "Notify Tracks"),
			LOCTEXT("ToggleAnimNotifyTracks_Tooltip", "Show/hide the animation notify/sync marker tracks"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FAnimationSharedData::ToggleAnimNotifyTracks),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this](){ return bAnimNotifyTracksEnabled; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleMontageTracks", "Montage Tracks"),
			LOCTEXT("ToggleMontageTracks_Tooltip", "Show/hide the montage tracks"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FAnimationSharedData::ToggleMontageTracks),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this](){ return bMontageTracksEnabled; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("ShowTransportControls", "Show Transport Controls"),
			LOCTEXT("ShowTransportControls_Tooltip", "Show the transport controls"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FAnimationSharedData::ToggleTransportControls),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FAnimationSharedData::ShouldShowTransportControls)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	InMenuBuilder.EndSection();
}

void FAnimationSharedData::ToggleAnimationTracks()
{
	bool bAnimationTracksEnabled = !AreAllAnimationTracksEnabled();

	bSkeletalMeshPoseTracksEnabled = bAnimationTracksEnabled;
	bSkeletalMeshCurveTracksEnabled = bAnimationTracksEnabled;
	bTickRecordTracksEnabled = bAnimationTracksEnabled;
	bAnimNodeTracksEnabled = bAnimationTracksEnabled;
	bAnimNotifyTracksEnabled = bAnimationTracksEnabled;
	bMontageTracksEnabled = bAnimNotifyTracksEnabled;

	for(TSharedRef<FSkeletalMeshPoseTrack> PoseTrack : SkeletalMeshPoseTracks)
	{
		PoseTrack->SetVisibilityFlag(bAnimationTracksEnabled);
	}

	for(TSharedRef<FSkeletalMeshCurvesTrack> CurvesTrack : SkeletalMeshCurvesTracks)
	{
		CurvesTrack->SetVisibilityFlag(bAnimationTracksEnabled);
	}

	for(TSharedRef<FAnimationTickRecordsTrack> TickRecordTrack : AnimationTickRecordsTracks)
	{
		TickRecordTrack->SetVisibilityFlag(bAnimationTracksEnabled);
	}

	for(TSharedRef<FAnimNodesTrack> AnimNodesTrack : AnimNodesTracks)
	{
		AnimNodesTrack->SetVisibilityFlag(bAnimationTracksEnabled);
	}

	for(TSharedRef<FAnimNotifiesTrack> AnimNotifyTrack : AnimNotifyTracks)
	{
		AnimNotifyTrack->SetVisibilityFlag(bAnimationTracksEnabled);
	}

	for(TSharedRef<FMontageTrack> MontageTrack : MontageTracks)
	{
		MontageTrack->SetVisibilityFlag(bAnimationTracksEnabled);
	}
}

bool FAnimationSharedData::AreAllAnimationTracksEnabled() const
{
	return bSkeletalMeshPoseTracksEnabled && bSkeletalMeshCurveTracksEnabled && bTickRecordTracksEnabled && bAnimNodeTracksEnabled && bAnimNotifyTracksEnabled && bMontageTracksEnabled;
}

bool FAnimationSharedData::AreAnyAnimationTracksEnabled() const
{
	return bSkeletalMeshPoseTracksEnabled || bSkeletalMeshCurveTracksEnabled || bTickRecordTracksEnabled || bAnimNodeTracksEnabled || bAnimNotifyTracksEnabled || bMontageTracksEnabled;
}

void FAnimationSharedData::ToggleSkeletalMeshPoseTracks()
{
	bSkeletalMeshPoseTracksEnabled = !bSkeletalMeshPoseTracksEnabled;

	for(TSharedRef<FSkeletalMeshPoseTrack> PoseTrack : SkeletalMeshPoseTracks)
	{
		PoseTrack->SetVisibilityFlag(bSkeletalMeshPoseTracksEnabled);
	}
}

void FAnimationSharedData::ToggleSkeletalMeshCurveTracks()
{
	bSkeletalMeshCurveTracksEnabled = !bSkeletalMeshCurveTracksEnabled;

	for(TSharedRef<FSkeletalMeshCurvesTrack> CurvesTrack : SkeletalMeshCurvesTracks)
	{
		CurvesTrack->SetVisibilityFlag(bSkeletalMeshCurveTracksEnabled);
	}
}

void FAnimationSharedData::ToggleTickRecordTracks()
{
	bTickRecordTracksEnabled = !bTickRecordTracksEnabled;

	for(TSharedRef<FAnimationTickRecordsTrack> TickRecordsTrack : AnimationTickRecordsTracks)
	{
		TickRecordsTrack->SetVisibilityFlag(bTickRecordTracksEnabled);
	}
}

void FAnimationSharedData::ToggleAnimNodeTracks()
{
	bAnimNodeTracksEnabled = !bAnimNodeTracksEnabled;

	for(TSharedRef<FAnimNodesTrack> AnimNodesTrack : AnimNodesTracks)
	{
		AnimNodesTrack->SetVisibilityFlag(bAnimNodeTracksEnabled);
	}
}

void FAnimationSharedData::ToggleAnimNotifyTracks()
{
	bAnimNotifyTracksEnabled = !bAnimNotifyTracksEnabled;

	for(TSharedRef<FAnimNotifiesTrack> AnimNotifyTrack : AnimNotifyTracks)
	{
		AnimNotifyTrack->SetVisibilityFlag(bAnimNotifyTracksEnabled);
	}
}

void FAnimationSharedData::ToggleMontageTracks()
{
	bMontageTracksEnabled = !bMontageTracksEnabled;
	
	for(TSharedRef<FMontageTrack> MontageTrack : MontageTracks)
	{
		MontageTrack->SetVisibilityFlag(bMontageTracksEnabled);
	}
}

void FAnimationSharedData::ToggleTransportControls()
{
	bShowTransportControls = !bShowTransportControls;
}

void FAnimationSharedData::OnTimeMarkerChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InTimeMarker)
{
	bTimeMarkerValid = InTimeMarker != std::numeric_limits<double>::infinity();
	MarkerTime = InTimeMarker;

	if(bTimeMarkerValid && AnalysisSession != nullptr)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

		const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
		bMarkerFrameValid = FramesProvider.GetFrameFromTime(ETraceFrameType::TraceFrameType_Game, MarkerTime, MarkerFrame);
	}
	else
	{
		MarkerFrame = TraceServices::FFrame();
		bMarkerFrameValid = false;
	}

#if WITH_EDITOR
	for(const TSharedRef<FAnimNodesTrack>& AnimNodesTrack : AnimNodesTracks)
	{
		AnimNodesTrack->UpdateDebugData(MarkerFrame);
	}

	InvalidateViewports();

	// Update pose tracks even if they are disabled, as they may be being debugged
	UWorld* WorldToUse = FGameplayTimingViewExtender::GetWorldToVisualize();
	for(TSharedRef<FSkeletalMeshPoseTrack> PoseTrack : SkeletalMeshPoseTracks)
	{
		if(bTimeMarkerValid && PoseTrack->IsPotentiallyDebugged())
		{
			PoseTrack->DrawPoses(WorldToUse, MarkerTime, MarkerFrame.StartTime, MarkerFrame.EndTime);
		}
	}
#endif
}

#if WITH_EDITOR
void FAnimationSharedData::InvalidateViewports() const
{
	UEditorEngine* Engine = Cast<UEditorEngine>(GEngine);
	if (GIsEditor && Engine != nullptr)
	{
		for (FEditorViewportClient* ViewportClient : Engine->GetAllViewportClients())
		{
			if (ViewportClient)
			{
				ViewportClient->Invalidate();
			}
		}
	}
}
#endif

void FAnimationSharedData::EnumerateSkeletalMeshPoseTracks(TFunctionRef<void(const TSharedRef<FSkeletalMeshPoseTrack>&)> InCallback) const
{
	for(const TSharedRef<FSkeletalMeshPoseTrack>& Track : SkeletalMeshPoseTracks)
	{
		InCallback(Track);
	}
}

TSharedPtr<FSkeletalMeshPoseTrack> FAnimationSharedData::FindSkeletalMeshPoseTrack(uint64 InComponentId) const
{
	TSharedPtr<FSkeletalMeshPoseTrack> FoundTrack;

	EnumerateSkeletalMeshPoseTracks([&FoundTrack, InComponentId](const TSharedRef<FSkeletalMeshPoseTrack>& InTrack)
	{
		if(InTrack->GetGameplayTrack().GetObjectId() == InComponentId)
		{
			FoundTrack = InTrack;
		}
	});

	return FoundTrack;
}

void FAnimationSharedData::EnumerateAnimNodesTracks(TFunctionRef<void(const TSharedRef<FAnimNodesTrack>&)> InCallback) const
{
	for(const TSharedRef<FAnimNodesTrack>& Track : AnimNodesTracks)
	{
		InCallback(Track);
	}
}

TSharedPtr<FAnimNodesTrack> FAnimationSharedData::FindAnimNodesTrack(uint64 InAnimInstanceId) const
{
	TSharedPtr<FAnimNodesTrack> FoundTrack;

	EnumerateAnimNodesTracks([&FoundTrack, InAnimInstanceId](const TSharedRef<FAnimNodesTrack>& InTrack)
	{
		if(InTrack->GetGameplayTrack().GetObjectId() == InAnimInstanceId)
		{
			FoundTrack = InTrack;
		}
	});

	return FoundTrack;
}

#if WITH_ENGINE

void FAnimationSharedData::DrawPoses(UWorld* InWorld)
{
	for(TSharedRef<FSkeletalMeshPoseTrack> PoseTrack : SkeletalMeshPoseTracks)
	{
		if(PoseTrack->IsVisible())
		{
			if(bTimeMarkerValid && bMarkerFrameValid && PoseTrack->ShouldDrawPose())
			{
				PoseTrack->DrawPoses(InWorld, MarkerTime, MarkerFrame.StartTime, MarkerFrame.EndTime);
			}
		}
	}
}

#endif

#if WITH_EDITOR

void FAnimationSharedData::GetCustomDebugObjects(const IAnimationBlueprintEditor& InAnimationBlueprintEditor, TArray<FCustomDebugObject>& OutDebugList)
{
	for(const TSharedRef<FAnimNodesTrack>& AnimNodesTrack : AnimNodesTracks)
	{
		if(AnimNodesTrack->IsVisible())
		{
			AnimNodesTrack->GetCustomDebugObjects(InAnimationBlueprintEditor, OutDebugList);
		}
	}
}

void FAnimationSharedData::ShowAllMeshes()
{
	for(TSharedRef<FSkeletalMeshPoseTrack> PoseTrack : SkeletalMeshPoseTracks)
	{
		PoseTrack->SetDrawPose(true);
	}

	InvalidateViewports();
}

void FAnimationSharedData::HideAllMeshes()
{
	for(TSharedRef<FSkeletalMeshPoseTrack> PoseTrack : SkeletalMeshPoseTracks)
	{
		PoseTrack->SetDrawPose(false);
	}

	InvalidateViewports();
}

#endif

void FAnimationSharedData::OpenAnimGraphTab(uint64 InAnimInstanceId) const
{
	if(TimingViewSession && AnalysisSession)
	{
		FGameplayInsightsModule& GameplayInsightsModule = FModuleManager::GetModuleChecked<FGameplayInsightsModule>("GameplayInsights");
		TSharedRef<SDockTab> Tab = GameplayInsightsModule.SpawnTimingProfilerDocumentTab(
			FGameplaySharedData::FSearchForTab([this, InAnimInstanceId]()
			{
				return FGameplaySharedData::FindDocumentTab(WeakAnimGraphDocumentTabs, [InAnimInstanceId](const TSharedRef<SDockTab>& InDockTab)
				{
					return StaticCastSharedRef<SAnimGraphSchematicView>(InDockTab->GetContent())->GetAnimInstanceId() == InAnimInstanceId;
				});
			})
		);
		
		TSharedPtr<SAnimGraphSchematicView> AnimGraphView = SNew(SAnimGraphSchematicView, InAnimInstanceId, TimingViewSession->GetTimeMarker(), *AnalysisSession);
		TimingViewSession->OnTimeMarkerChanged().AddLambda(
			[AnimGraphViewWeakPtr = TWeakPtr<SAnimGraphSchematicView>(AnimGraphView)](UE::Insights::Timing::ETimeChangedFlags InFlags, double TimeMarker)
			{
				if (AnimGraphViewWeakPtr.IsValid())
				{
					AnimGraphViewWeakPtr.Pin()->SetTimeMarker(TimeMarker);
				}
			});

		Tab->SetContent(AnimGraphView.ToSharedRef());
		WeakAnimGraphDocumentTabs.Add(Tab);

		const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
		if(GameplayProvider)
		{
			FString ObjectName;
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
				ObjectName = GameplayProvider->GetObjectInfo(InAnimInstanceId).Name;
			}
			Tab->SetLabel(FText::FromString(ObjectName));
		}
	}
}

#undef LOCTEXT_NAMESPACE
