// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ViewModels/TimingEventsTrack.h"
#include "TraceServices/Model/Frames.h"

class FGameplaySharedData;
namespace TraceServices { class IAnalysisSession; }
namespace UE::Insights::Timing { class ITimingViewSession; }
namespace UE::Insights::Timing { enum class ETimeChangedFlags : int32; }
class FSkeletalMeshPoseTrack;
class FSkeletalMeshCurvesTrack;
class FAnimationTickRecordsTrack;
class FAnimNodesTrack;
class FAnimNotifiesTrack;
class FMontageTrack;
class FMenuBuilder;
class UWorld;
class IAnimationBlueprintEditor;
struct FCustomDebugObject;
class SDockTab;
class FGameplayTrack;

class FAnimationSharedData
{
public:
	FAnimationSharedData(FGameplaySharedData& InGameplaySharedData);

	void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession);
	void OnEndSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession);
	void Tick(UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession);
	void ExtendFilterMenu(FMenuBuilder& InMenuBuilder);

#if WITH_ENGINE
	void DrawPoses(UWorld* InWorld);
#endif

#if WITH_EDITOR
	// Get the debug objects to plug into the anim BP debugger
	void GetCustomDebugObjects(const IAnimationBlueprintEditor& InAnimationBlueprintEditor, TArray<FCustomDebugObject>& OutDebugList);

	// Helper function to invalidate all viewports so non-realtime viewports update correctly.
	void InvalidateViewports() const;

	// Show all the meshes
	void ShowAllMeshes();

	// Hide all the meshes
	void HideAllMeshes();
#endif

	// Check whether all animation tracks are enabled
	bool AreAllAnimationTracksEnabled() const;

	// Check whether any animation tracks are enabled
	bool AreAnyAnimationTracksEnabled() const;

	// Get the last cached analysis session
	const TraceServices::IAnalysisSession& GetAnalysisSession() const { return *AnalysisSession; }

	// Check whether the analysis session is valid
	bool IsAnalysisSessionValid() const { return AnalysisSession != nullptr; }

	// Get the gameplay shared data we are linked to
	const FGameplaySharedData& GetGameplaySharedData() const { return GameplaySharedData; }

	// Enumerate skeletal mesh pose tracks
	void EnumerateSkeletalMeshPoseTracks(TFunctionRef<void(const TSharedRef<FSkeletalMeshPoseTrack>&)> InCallback) const;

	// Find a skeletal mesh track with the specified component ID
	TSharedPtr<FSkeletalMeshPoseTrack> FindSkeletalMeshPoseTrack(uint64 InComponentId) const;

	// Enumerate anim nodes tracks
	void EnumerateAnimNodesTracks(TFunctionRef<void(const TSharedRef<FAnimNodesTrack>&)> InCallback) const;

	// Find an anim nodes track with the specified anim instance ID
	TSharedPtr<FAnimNodesTrack> FindAnimNodesTrack(uint64 InAnimInstanceId) const;

	/** Open an anim graph tab to see its schematic view */
	void OpenAnimGraphTab(uint64 InAnimInstanceId) const;

	/** Get the current marker frame, if any. @return true if the frame is valid */
	bool GetCurrentMarkerFrame(TraceServices::FFrame& OutFrame) const { OutFrame = MarkerFrame; return bMarkerFrameValid; }

	/* Returns true if the transport controls toolbar should be displayed. */
	bool ShouldShowTransportControls() { return bShowTransportControls; }

private:
	// UI handlers
	void ToggleAnimationTracks();
	void OnTimeMarkerChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InTimeMarker);
	void ToggleSkeletalMeshPoseTracks();
	void ToggleSkeletalMeshCurveTracks();
	void ToggleTickRecordTracks();
	void ToggleAnimNodeTracks();
	void ToggleAnimNotifyTracks();
	void ToggleMontageTracks();
	void ToggleTransportControls();

private:
	// The gameplay shared data we are linked to
	FGameplaySharedData& GameplaySharedData;

	// Cached analysis session, set in Tick()
	const TraceServices::IAnalysisSession* AnalysisSession;

	// Cached timing view session, set in OnBeginSession/OnEndSession
	UE::Insights::Timing::ITimingViewSession* TimingViewSession;

	// All the tracks we manage
	TArray<TSharedRef<FSkeletalMeshPoseTrack>> SkeletalMeshPoseTracks;
	TArray<TSharedRef<FSkeletalMeshCurvesTrack>> SkeletalMeshCurvesTracks;
	TArray<TSharedRef<FAnimationTickRecordsTrack>> AnimationTickRecordsTracks;
	TArray<TSharedRef<FAnimNodesTrack>> AnimNodesTracks;
	TArray<TSharedRef<FAnimNotifiesTrack>> AnimNotifyTracks;
	TArray<TSharedRef<FMontageTrack>> MontageTracks;

	// All the documents we have spawned
	mutable TArray<TWeakPtr<SDockTab>> WeakAnimGraphDocumentTabs;

	// Delegate handles for hooks into the timing view
	FDelegateHandle TimeMarkerChangedHandle;

	/** Various times and ranges */
	double MarkerTime;
	TraceServices::FFrame MarkerFrame;

	/** Validity flags for pose times/ranges */
	bool bTimeMarkerValid;
	bool bMarkerFrameValid;

	// Flags controlling whether check type of our animation tracks are enabled
	bool bSkeletalMeshPoseTracksEnabled;
	bool bSkeletalMeshCurveTracksEnabled;
	bool bTickRecordTracksEnabled;
	bool bAnimNodeTracksEnabled;
	bool bAnimNotifyTracksEnabled;
	bool bMontageTracksEnabled;
	bool bShowTransportControls;
};
