// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationTickRecordsTrack.h"

#include "AnimationProvider.h"
#include "AnimationSharedData.h"
#include "Common/ProviderLock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameplayProvider.h"
#include "GameplaySharedData.h"
#include "Templates/Invoke.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/ViewModels/GraphTrackBuilder.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"

#if WITH_EDITOR
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "IAnimationBlueprintEditor.h"
#include "Animation/AnimBlueprint.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "EdGraph/EdGraphNode.h"
#endif
#include "VariantTreeNode.h"
#include "TraceServices/Model/Frames.h"

#define LOCTEXT_NAMESPACE "AnimationTickRecordsTrack"

INSIGHTS_IMPLEMENT_RTTI(FAnimationTickRecordsTrack)

FString FTickRecordSeries::FormatValue(double Value) const
{
	switch (Type)
	{
	case ESeriesType::PlaybackTime:
		return UE::Insights::FormatTimeAuto(Value);
	case ESeriesType::BlendWeight:
	case ESeriesType::RootMotionWeight:
	case ESeriesType::PlayRate:
	case ESeriesType::BlendSpacePositionX:
	case ESeriesType::BlendSpacePositionY:
	case ESeriesType::BlendSpaceFilteredPositionX:
	case ESeriesType::BlendSpaceFilteredPositionY:
		return FText::AsNumber(Value).ToString();
	}

	return FGraphSeries::FormatValue(Value);
}

static FLinearColor MakeSeriesColor(uint32 InSeed, bool bInLine = false)
{
	FRandomStream Stream(InSeed);
	const uint8 Hue = (uint8)(Stream.FRand() * 255.0f);
	const uint8 SatVal = bInLine ? 196 : 128;
	return FLinearColor::MakeFromHSV8(Hue, SatVal, SatVal);
}

static FLinearColor MakeSeriesColor(FTickRecordSeries::ESeriesType InSeed, bool bInLine = false)
{
	return MakeSeriesColor((uint32)InSeed, bInLine);
}

FAnimationTickRecordsTrack::FAnimationTickRecordsTrack(const FAnimationSharedData& InSharedData, uint64 InObjectID, const TCHAR* InName)
	: FGameplayGraphTrack(InSharedData.GetGameplaySharedData(), InObjectID, FText::Format(LOCTEXT("AnimationTickRecordsTrackName", "Blend Weights - {0}"), FText::FromString(InName)))
	, SharedData(InSharedData)
{
	EnableOptions(ShowLabelsOption);
	Layout = EGameplayGraphLayout::Stack;

#if WITH_EDITOR
	const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);

	if(GameplayProvider)
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
		const FObjectInfo* AnimInstanceInfo = GameplayProvider->FindObjectInfo(GetGameplayTrack().GetObjectId());
		if(AnimInstanceInfo)
		{
			const FClassInfo* AnimInstanceClassInfo = GameplayProvider->FindClassInfo(AnimInstanceInfo->ClassId);
			if(AnimInstanceClassInfo)
			{
				InstanceClass = FSoftObjectPath(AnimInstanceClassInfo->PathName);
			}
		}
	}
#endif
}

void FAnimationTickRecordsTrack::AddAllSeries()
{
	struct FSeriesDescription
	{
		FText Name;
		FText Description;
		FTickRecordSeries::ESeriesType Type;
		bool bEnabled;
	};

	const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
	if(GameplayProvider && AnimationProvider)
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);

		AnimationProvider->EnumerateTickRecordIds(GetGameplayTrack().GetObjectId(), [this, &GameplayProvider](uint64 InAssetId, int32 InNodeId)
		{
			auto CheckExistingSeries = [InAssetId, InNodeId](const TSharedPtr<FGraphSeries>& InSeries)
			{
				TSharedPtr<FTickRecordSeries> TickRecordSeries = StaticCastSharedPtr<FTickRecordSeries>(InSeries);
				return TickRecordSeries->AssetId == InAssetId && TickRecordSeries->NodeId == InNodeId;
			};

			if (!AllSeries.ContainsByPredicate(CheckExistingSeries))
			{
				static const FSeriesDescription SeriesDescriptions[] =
				{
					{
						LOCTEXT("SeriesNameBlendWeight", "Blend Weight"),
						LOCTEXT("SeriesDescBlendWeight", "The final effective weight that this animation sequence was played at"),
						FTickRecordSeries::ESeriesType::BlendWeight,
						true
					},
					{
						LOCTEXT("SeriesNamePlaybackTime", "Playback Time"),
						LOCTEXT("SeriesDescPlaybackTime", "The playback time of this animation sequence"),
						FTickRecordSeries::ESeriesType::PlaybackTime,
						false
					},
					{
						LOCTEXT("SeriesNameRootMotionWeight", "Root Motion Weight"),
						LOCTEXT("SeriesDescRootMotionWeight", "The final effective root motion weight that this animation sequence was played at"),
						FTickRecordSeries::ESeriesType::RootMotionWeight,
						false
					},
					{
						LOCTEXT("SeriesNamePlayRate", "Play Rate"),
						LOCTEXT("SeriesDescPlayRate", "The play rate/speed of this animation sequence"),
						FTickRecordSeries::ESeriesType::PlayRate,
						false
					},
				};

				static const FSeriesDescription BlendSpaceSeriesDescriptions[] =
				{
					{
						LOCTEXT("SeriesNameBlendSpacePositionX", "BlendSpace Position X"),
						LOCTEXT("SeriesDescBlendSpacePositionX", "The X value used to sample this blend space"),
						FTickRecordSeries::ESeriesType::BlendSpacePositionX,
						false
					},
					{
						LOCTEXT("SeriesNameBlendSpacePositionY", "BlendSpace Position Y"),
						LOCTEXT("SeriesDescBlendSpacePositionY", "The Y value used to sample this blend space"),
						FTickRecordSeries::ESeriesType::BlendSpacePositionY,
						false
					},
					{
						LOCTEXT("SeriesNameBlendSpaceFilteredPositionX", "BlendSpace Filtered Position X"),
						LOCTEXT("SeriesDescBlendSpaceFilteredPositionX", "The X value after filtering used to sample this blend space"),
						FTickRecordSeries::ESeriesType::BlendSpaceFilteredPositionX,
						false
					},
					{
						LOCTEXT("SeriesNameBlendSpaceFilteredPositionY", "BlendSpace Filtered Position Y"),
						LOCTEXT("SeriesDescBlendSpaceFilteredPositionY", "The Y value after filtering used to sample this blend space"),
						FTickRecordSeries::ESeriesType::BlendSpaceFilteredPositionY,
						false
					},
				};

				const FClassInfo& ClassInfo = GameplayProvider->GetClassInfoFromObject(InAssetId);
				const FObjectInfo& ObjectInfo = GameplayProvider->GetObjectInfo(InAssetId);

				auto AddSeries = [this, GameplayProvider](const FSeriesDescription& InSeriesDescription, const TCHAR* InAssetName, uint64 InAssetId, int32 InNodeId)
				{
					FString Name = FText::Format(LOCTEXT("SeriesNameFormat", "{0} - {1}"), FText::FromString(InAssetName), InSeriesDescription.Name).ToString();
					uint32 Hash = GetTypeHash(Name);
					FLinearColor LineColor = MakeSeriesColor(Hash, true);
					FLinearColor FillColor = MakeSeriesColor(Hash);

					TSharedRef<FTickRecordSeries> Series = MakeShared<FTickRecordSeries>();
					Series->SetName(Name);
					Series->SetDescription(InSeriesDescription.Description.ToString());
					Series->SetColor(LineColor, LineColor, FillColor);
					Series->Type = InSeriesDescription.Type;
					Series->SetVisibility(InSeriesDescription.bEnabled);
					Series->SetBaselineY(25.0f);
					Series->SetScaleY(20.0f);
					Series->EnableAutoZoom();
					Series->AssetId = InAssetId;
					Series->NodeId = InNodeId;
					AllSeries.Add(Series);
				};

				for (const FSeriesDescription& SeriesDescription : SeriesDescriptions)
				{
					AddSeries(SeriesDescription, ObjectInfo.Name, InAssetId, InNodeId);
				}

				if (FCString::Stristr(ClassInfo.Name, TEXT("BlendSpace")) != nullptr)
				{
					for (const FSeriesDescription& BlendSpaceSeriesDescription : BlendSpaceSeriesDescriptions)
					{
						AddSeries(BlendSpaceSeriesDescription, ObjectInfo.Name, InAssetId, InNodeId);
					}
				}
			}
		});
	}
}

template<typename ProjectionType>
bool FAnimationTickRecordsTrack::UpdateSeriesBoundsHelper(FTickRecordSeries& InSeries, const FTimingTrackViewport& InViewport, ProjectionType Projection)
{
	bool bFoundEvents = false;

	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if (AnimationProvider)
	{
		InSeries.CurrentMin = 0.0;
		InSeries.CurrentMax = 0.0;

		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadTickRecordTimeline(GetGameplayTrack().GetObjectId(), [&bFoundEvents, &InViewport, &InSeries, &Projection](const FAnimationProvider::TickRecordTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(InViewport.GetStartTime(), InViewport.GetEndTime(), [&bFoundEvents, &InSeries, &Projection](double InStartTime, double InEndTime, uint32 InDepth, const FTickRecordMessage& InMessage)
			{
				if (InMessage.AssetId == InSeries.AssetId && InMessage.NodeId == InSeries.NodeId)
				{
					const float Value = Invoke(Projection, InMessage);
					InSeries.CurrentMin = FMath::Min(InSeries.CurrentMin, Value);
					InSeries.CurrentMax = FMath::Max(InSeries.CurrentMax, Value);
					bFoundEvents = true;
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}

	return bFoundEvents;
}

template<typename ProjectionType>
void FAnimationTickRecordsTrack::UpdateSeriesHelper(FTickRecordSeries& InSeries, const FTimingTrackViewport& InViewport, ProjectionType Projection)
{		
	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if(AnimationProvider)
	{
		FGraphTrackBuilder Builder(*this, InSeries, InViewport);

		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadTickRecordTimeline(GetGameplayTrack().GetObjectId(), [this, &AnimationProvider, &Builder, &InViewport, &InSeries, &Projection](const FAnimationProvider::TickRecordTimeline& InTimeline)
		{
			uint16 FrameCounter = 0;
			uint16 LastFrameWithTickRecord = 0;

			InTimeline.EnumerateEvents(InViewport.GetStartTime(), InViewport.GetEndTime(), [this, &FrameCounter, &LastFrameWithTickRecord, &Builder, &InSeries, &Projection](double InStartTime, double InEndTime, uint32 InDepth, const FTickRecordMessage& InMessage)
			{
				FrameCounter = InMessage.FrameCounter;

				if (InMessage.AssetId == InSeries.AssetId && InMessage.NodeId == InSeries.NodeId)
				{
					Builder.AddEvent(InStartTime, InEndTime - InStartTime, Invoke(Projection, InMessage), LastFrameWithTickRecord == FrameCounter - 1);

					LastFrameWithTickRecord = InMessage.FrameCounter;
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}

bool FAnimationTickRecordsTrack::UpdateSeriesBounds(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport)
{
	FTickRecordSeries& TickRecordSeries = *static_cast<FTickRecordSeries*>(&InSeries);
	switch (TickRecordSeries.Type)
	{
	case FTickRecordSeries::ESeriesType::BlendWeight:
		return UpdateSeriesBoundsHelper(TickRecordSeries, InViewport, &FTickRecordMessage::BlendWeight);
	case FTickRecordSeries::ESeriesType::PlaybackTime:
		return UpdateSeriesBoundsHelper(TickRecordSeries, InViewport, &FTickRecordMessage::PlaybackTime);
	case FTickRecordSeries::ESeriesType::RootMotionWeight:
		return UpdateSeriesBoundsHelper(TickRecordSeries, InViewport, &FTickRecordMessage::RootMotionWeight);
	case FTickRecordSeries::ESeriesType::PlayRate:
		return UpdateSeriesBoundsHelper(TickRecordSeries, InViewport, &FTickRecordMessage::PlayRate);
	case FTickRecordSeries::ESeriesType::BlendSpacePositionX:
		return UpdateSeriesBoundsHelper(TickRecordSeries, InViewport, &FTickRecordMessage::BlendSpacePositionX);
	case FTickRecordSeries::ESeriesType::BlendSpacePositionY:
		return UpdateSeriesBoundsHelper(TickRecordSeries, InViewport, &FTickRecordMessage::BlendSpacePositionY);
	case FTickRecordSeries::ESeriesType::BlendSpaceFilteredPositionX:
		return UpdateSeriesBoundsHelper(TickRecordSeries, InViewport, &FTickRecordMessage::BlendSpaceFilteredPositionX);
	case FTickRecordSeries::ESeriesType::BlendSpaceFilteredPositionY:
		return UpdateSeriesBoundsHelper(TickRecordSeries, InViewport, &FTickRecordMessage::BlendSpaceFilteredPositionY);
	}

	return false;
}

void FAnimationTickRecordsTrack::UpdateSeries(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport)
{
	FTickRecordSeries& TickRecordSeries = *static_cast<FTickRecordSeries*>(&InSeries);
	switch (TickRecordSeries.Type)
	{
	case FTickRecordSeries::ESeriesType::BlendWeight:
		UpdateSeriesHelper(TickRecordSeries, InViewport, &FTickRecordMessage::BlendWeight);
		break;
	case FTickRecordSeries::ESeriesType::PlaybackTime:
		UpdateSeriesHelper(TickRecordSeries, InViewport, &FTickRecordMessage::PlaybackTime);
		break;
	case FTickRecordSeries::ESeriesType::RootMotionWeight:
		UpdateSeriesHelper(TickRecordSeries, InViewport, &FTickRecordMessage::RootMotionWeight);
		break;
	case FTickRecordSeries::ESeriesType::PlayRate:
		UpdateSeriesHelper(TickRecordSeries, InViewport, &FTickRecordMessage::PlayRate);
		break;
	case FTickRecordSeries::ESeriesType::BlendSpacePositionX:
		UpdateSeriesHelper(TickRecordSeries, InViewport, &FTickRecordMessage::BlendSpacePositionX);
		break;
	case FTickRecordSeries::ESeriesType::BlendSpacePositionY:
		UpdateSeriesHelper(TickRecordSeries, InViewport, &FTickRecordMessage::BlendSpacePositionY);
		break;
	case FTickRecordSeries::ESeriesType::BlendSpaceFilteredPositionX:
		UpdateSeriesHelper(TickRecordSeries, InViewport, &FTickRecordMessage::BlendSpaceFilteredPositionX);
		break;
	case FTickRecordSeries::ESeriesType::BlendSpaceFilteredPositionY:
		UpdateSeriesHelper(TickRecordSeries, InViewport, &FTickRecordMessage::BlendSpaceFilteredPositionY);
		break;
	}
}

void FAnimationTickRecordsTrack::PostUpdate(const ITimingTrackUpdateContext& Context)
{
	FGameplayGraphTrack::PostUpdate(Context);

	if(Context.GetHoveredEvent().IsValid() && Context.GetHoveredEvent()->Is<FGraphTrackEvent>())
	{
		TSharedPtr<const FGraphTrackEvent> HoveredEvent = StaticCastSharedPtr<const FGraphTrackEvent>(Context.GetHoveredEvent());
		if(HoveredEvent->GetTrack() == SharedThis(this))
		{
			CachedHoveredEvent = HoveredEvent;
		}
	}
}

void FAnimationTickRecordsTrack::InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const
{
	const FGraphTrackEvent& GraphTrackEvent = *static_cast<const FGraphTrackEvent*>(&HoveredTimingEvent);

	FTimingEventSearchParameters SearchParameters(HoveredTimingEvent.GetStartTime(), HoveredTimingEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch);

	FindTickRecordMessage(SearchParameters, [this, &GraphTrackEvent, &Tooltip](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FTickRecordMessage& InMessage)
	{
		Tooltip.ResetContent();

		Tooltip.AddTitle(GetName());

		Tooltip.AddNameValueTextLine(LOCTEXT("EventTime", "Time").ToString(), FText::AsNumber(InFoundStartTime).ToString());
		Tooltip.AddNameValueTextLine(GraphTrackEvent.GetSeries()->GetName().ToString(), FText::AsNumber(GraphTrackEvent.GetValue()).ToString());
		Tooltip.AddNameValueTextLine(LOCTEXT("EventWorld", "World").ToString(), GetGameplayTrack().GetWorldName(SharedData.GetAnalysisSession()).ToString());

		Tooltip.UpdateLayout();
	});
}

const TSharedPtr<const ITimingEvent> FAnimationTickRecordsTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	FindTickRecordMessage(InSearchParameters, [this, &FoundEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FTickRecordMessage& InFoundMessage)
	{
		FoundEvent = MakeShared<const FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth);
	});

	return FoundEvent;
}

void FAnimationTickRecordsTrack::FindTickRecordMessage(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FTickRecordMessage&)> InFoundPredicate) const
{
	TTimingEventSearch<FTickRecordMessage>::Search(
		InParameters,

		[this](TTimingEventSearch<FTickRecordMessage>::FContext& InContext)
		{
			const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

			if(AnimationProvider)
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);

				AnimationProvider->ReadTickRecordTimeline(GetGameplayTrack().GetObjectId(), [this, &InContext](const FAnimationProvider::TickRecordTimeline& InTimeline)
				{
					InTimeline.EnumerateEvents(InContext.GetParameters().StartTime, InContext.GetParameters().EndTime, [&InContext](double InEventStartTime, double InEventEndTime, uint32 InDepth, const FTickRecordMessage& InMessage)
					{
						InContext.Check(InEventStartTime, InEventEndTime, 0, InMessage);
						return TraceServices::EEventEnumerate::Continue;
					});
				});
			}
		},

		[&InParameters](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FTickRecordMessage& InEvent)
		{
			// Match the start time exactly here
			return InFoundStartTime == InParameters.StartTime;
		},

		[&InFoundPredicate](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FTickRecordMessage& InEvent)
		{
			InFoundPredicate(InFoundStartTime, InFoundEndTime, InFoundDepth, InEvent);
		},
			
		TTimingEventSearch<FTickRecordMessage>::NoMatch);
}

void FAnimationTickRecordsTrack::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
#if WITH_EDITOR
	if(CachedHoveredEvent.IsValid())
	{
		int32 NodeId = StaticCastSharedRef<const FTickRecordSeries>(CachedHoveredEvent.Pin()->GetSeries())->NodeId;

		MenuBuilder.BeginSection("TrackActions", LOCTEXT("TrackActionsMenuHeader", "Track Actions"));
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("FindAssetPlayerNode", "Find Asset Player Node"),
				LOCTEXT("FindAssetPlayerNode_Tooltip", "Open the animation blueprint that this animation was played from."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, NodeId]()
					{
						if(InstanceClass.LoadSynchronous())
						{
							if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(InstanceClass.Get()->ClassGeneratedBy))
							{
								GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AnimBlueprint);

								if(IAnimationBlueprintEditor* AnimBlueprintEditor = static_cast<IAnimationBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AnimBlueprint, true)))
								{
									int32 AnimNodeIndex = InstanceClass.Get()->GetAnimNodeProperties().Num() - NodeId - 1;
									TWeakObjectPtr<const UEdGraphNode>* GraphNode = InstanceClass.Get()->AnimBlueprintDebugData.NodePropertyIndexToNodeMap.Find(AnimNodeIndex);
									if(GraphNode != nullptr && GraphNode->Get())
									{
										AnimBlueprintEditor->JumpToHyperlink(GraphNode->Get());
									}
								}
							}
						}
					})
				),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		MenuBuilder.EndSection();
	}
#endif

	FGameplayGraphTrack::BuildContextMenu(MenuBuilder);
}

void FAnimationTickRecordsTrack::GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const
{
	const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
	if(GameplayProvider && AnimationProvider)
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);

		AnimationProvider->ReadTickRecordTimeline(GetGameplayTrack().GetObjectId(), [&GameplayProvider, &OutVariants, &InFrame](const FAnimationProvider::TickRecordTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, [&GameplayProvider, &OutVariants, &InFrame](double InStartTime, double InEndTime, uint32 InDepth, const FTickRecordMessage& InMessage)
			{
				if(InStartTime >= InFrame.StartTime && InEndTime <= InFrame.EndTime)
				{
					const FClassInfo& ClassInfo = GameplayProvider->GetClassInfoFromObject(InMessage.AssetId);
					TSharedRef<FVariantTreeNode> Header = OutVariants.Add_GetRef(FVariantTreeNode::MakeObject(FText::FromString(ClassInfo.Name), InMessage.AssetId, InMessage.AssetId));
					
					Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("BlendWeight", "Blend Weight"), InMessage.BlendWeight));
					Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("PlaybackTime", "Playback Time"), InMessage.PlaybackTime));
					Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("RootMotionWeight", "Root Motion Weight"), InMessage.RootMotionWeight));
					Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("PlayRate", "Play Rate"), InMessage.PlayRate));
					if(InMessage.bIsBlendSpace)
					{
						Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("BlendSpacePositionX", "Blend Space Position X"), InMessage.BlendSpacePositionX));
						Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("BlendSpacePositionY", "Blend Space Position Y"), InMessage.BlendSpacePositionY));
						Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("BlendSpaceFilteredPositionX", "Blend Space Filtered Position X"), InMessage.BlendSpaceFilteredPositionX));
						Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("BlendSpaceFilteredPositionY", "Blend Space Filtered Position Y"), InMessage.BlendSpaceFilteredPositionY));
					}
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}

#undef LOCTEXT_NAMESPACE
