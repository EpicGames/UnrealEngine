// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayGraphTrack.h"

class FAnimationSharedData;
struct FTickRecordMessage;
class FTimingEventSearchParameters;
class FGameplaySharedData;
class UAnimBlueprintGeneratedClass;
class FAnimationTickRecordsTrack;
class FTimingTrackViewport;
class FGraphTrackEvent;

class FTickRecordSeries : public FGameplayGraphSeries
{
public:
	enum class ESeriesType : uint32
	{
		BlendWeight,
		PlaybackTime,
		RootMotionWeight,
		PlayRate,
		BlendSpacePositionX,
		BlendSpacePositionY,
		BlendSpaceFilteredPositionX,
		BlendSpaceFilteredPositionY,

		Count,
	};

	virtual FString FormatValue(double Value) const override;

public:
	/** The asset ID that this series uses */
	uint64 AssetId;

	/** The node ID that this series' data comes from */
	int32 NodeId;

	ESeriesType Type;
};

class FAnimationTickRecordsTrack : public FGameplayGraphTrack
{
	INSIGHTS_DECLARE_RTTI(FAnimationTickRecordsTrack, FGameplayGraphTrack)

public:
	FAnimationTickRecordsTrack(const FAnimationSharedData& InSharedData, uint64 InObjectId, const TCHAR* InName);

	virtual void PostUpdate(const ITimingTrackUpdateContext& Context) override;
	virtual void InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const override;
	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	virtual bool UpdateSeriesBounds(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport) override;
	virtual void UpdateSeries(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport) override;
	virtual void AddAllSeries() override;
	virtual void GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const override;

private:
	// Helper function used to find a tick record
	void FindTickRecordMessage(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FTickRecordMessage&)> InFoundPredicate) const;

	// Helper function for series bounds update
	template<typename ProjectionType>
	bool UpdateSeriesBoundsHelper(FTickRecordSeries& InSeries, const FTimingTrackViewport& InViewport, ProjectionType Projection);

	// Helper function for series update
	template<typename ProjectionType>
	void UpdateSeriesHelper(FTickRecordSeries& InSeries, const FTimingTrackViewport& InViewport, ProjectionType Projection);

private:
	/** The shared data */
	const FAnimationSharedData& SharedData;

#if WITH_EDITOR
	/** The class that output this tick record */
	TSoftObjectPtr<UAnimBlueprintGeneratedClass> InstanceClass;
#endif

	/** Cached hovered event */
	TWeakPtr<const FGraphTrackEvent> CachedHoveredEvent;
};
