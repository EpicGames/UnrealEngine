// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendWeightsTrack.h"
#include "Common/ProviderLock.h"
#include "SCurveTimelineView.h"
#include "IRewindDebugger.h"
#include "GameplayProvider.h"
#include "AnimationProvider.h"
#include "Styling/SlateIconFinder.h"
#include "SBlendWeightsView.h"

#if WITH_EDITOR
#include "Animation/AnimInstance.h"
#include "AnimPreviewInstance.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Editor.h"
#include "IAnimationEditor.h"
#include "IPersonaToolkit.h"
#include "Subsystems/AssetEditorSubsystem.h"
#endif

#define LOCTEXT_NAMESPACE "BlendWeightsTrack"

namespace RewindDebugger
{

FBlendWeightsTrack::FBlendWeightsTrack(uint64 InObjectId) : ObjectId(InObjectId)
{
#if WITH_EDITOR
	Icon = FSlateIconFinder::FindIconForClass(UAnimSequence::StaticClass());
#endif
}

TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> FBlendWeightsTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(Children.GetData()), Children.Num());
}

bool FBlendWeightsTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlendWeights::UpdateInternal);
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();

	struct TrackId
	{
		TrackId() : AssetId(0), NodeId(0) {}
		TrackId(uint64 InAssetId, uint32 InNodeId) : AssetId(InAssetId), NodeId(InNodeId) { }
		bool operator ==(const TrackId& Other) const { return AssetId == Other.AssetId && NodeId == Other.NodeId; }
		
		uint64 AssetId;
		uint32 NodeId;
	};	

	TArray<TrackId> UniqueTrackIds;
	
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();

	bool bChanged = false;
	
	// convert time range to from rewind debugger times to profiler times
	TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	double StartTime = TraceTimeRange.GetLowerBoundValue();
	double EndTime = TraceTimeRange.GetUpperBoundValue();
	
	// count number of unique animations in the current time range
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	int AnimationCount = 0;

	if (AnimationProvider)
	{
		UniqueTrackIds.SetNum(0, EAllowShrinking::No);

		{
			TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
			AnimationProvider->ReadTickRecordTimeline(ObjectId, [&UniqueTrackIds, StartTime, EndTime](const FAnimationProvider::TickRecordTimeline& InTimeline)
			{
				// this isn't very efficient, and it gets called every frame.  will need optimizing
				InTimeline.EnumerateEvents(StartTime, EndTime, [&UniqueTrackIds, StartTime, EndTime](double InStartTime, double InEndTime, uint32 InDepth, const FTickRecordMessage& InMessage)
				{
					if (InEndTime > StartTime && InStartTime < EndTime)
					{
						UniqueTrackIds.AddUnique(TrackId(InMessage.AssetId, InMessage.NodeId));
					}
					return TraceServices::EEventEnumerate::Continue;
				});
			});
		}

		UniqueTrackIds.StableSort([](const TrackId &A, const TrackId &B)
			{
				if (A.NodeId == B.NodeId)
				{
					return A.AssetId > B.AssetId;
				}
				return A.NodeId > B.NodeId;
			});
		
		AnimationCount = UniqueTrackIds.Num();

		if (Children.Num()!=AnimationCount)
			bChanged = true;
		
		Children.SetNum(AnimationCount);
		for(int i = 0; i < AnimationCount; i++)
		{
			if (!Children[i].IsValid() || Children[i].Get()->GetAssetId() != UniqueTrackIds[i].AssetId)
			{
				Children[i] = MakeShared<FBlendWeightTrack>(ObjectId, UniqueTrackIds[i].AssetId, UniqueTrackIds[i].NodeId);
				bChanged = true;
			}

			if (Children[i]->Update())
			{
				bChanged = true;
			}
		}
	}

	return bChanged;
}

TSharedPtr<SWidget> FBlendWeightsTrack::GetDetailsViewInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	return SNew(SBlendWeightsView, ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession())
				.CurrentTime_Lambda([RewindDebugger]{ return RewindDebugger->CurrentTraceTime(); });
}


FBlendWeightTrack::FBlendWeightTrack(uint64 InObjectId, uint64 InAssetId, uint32 InNodeId, FBlendWeightTrack::ECurveType InCurveType) :
	 AssetId(InAssetId)
	, NodeId(InNodeId)
	, CurveType(InCurveType)
	, ObjectId(InObjectId)
{
	SetIsExpanded(false);
}

TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> FBlendWeightTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(Children.GetData()), Children.Num());
}

TSharedPtr<SCurveTimelineView::FTimelineCurveData> FBlendWeightTrack::GetCurveData() const
{
	if (!CurveData.IsValid())
	{
		CurveData = MakeShared<SCurveTimelineView::FTimelineCurveData>();
	}
	
	CurvesUpdateRequested++;
	
	return CurveData;
}

bool FBlendWeightTrack::UpdateInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	// compute curve points
	//
	TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	double StartTime = TraceTimeRange.GetLowerBoundValue();
	double EndTime = TraceTimeRange.GetUpperBoundValue();

	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	int AnimationCount = 0;

	if (CurvesUpdateRequested > 10 && GameplayProvider && AnimationProvider)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FBlendWeightTrack::UpdateCurvePointsInternal);
		auto& CurvePoints = CurveData->Points;
		CurvePoints.SetNum(0,EAllowShrinking::No);

		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadTickRecordTimeline(ObjectId, [AnalysisSession, StartTime, EndTime, &CurvePoints, this](const FAnimationProvider::TickRecordTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(StartTime, EndTime, [&CurvePoints, AnalysisSession, this](double InStartTime, double InEndTime, uint32 InDepth, const FTickRecordMessage& InMessage)
			{
				if (InMessage.AssetId == AssetId && InMessage.NodeId == NodeId)
				{
					float Weight = 0;
					switch (CurveType)
					{
						case ECurveType::BlendWeight:					Weight = InMessage.BlendWeight; break; 
						case ECurveType::PlaybackTime:					Weight = InMessage.PlaybackTime; break; 
						case ECurveType::RootMotionWeight:				Weight = InMessage.RootMotionWeight; break; 
						case ECurveType::PlayRate:						Weight = InMessage.PlayRate; break; 
						case ECurveType::BlendSpacePositionX:			Weight = InMessage.BlendSpacePositionX; break; 
						case ECurveType::BlendSpacePositionY:			Weight = InMessage.BlendSpacePositionY; break; 
						case ECurveType::BlendSpaceFilteredPositionX:	Weight = InMessage.BlendSpaceFilteredPositionX; break; 
						case ECurveType::BlendSpaceFilteredPositionY:	Weight = InMessage.BlendSpaceFilteredPositionY; break; 
					}
					
					CurvePoints.Add({ InMessage.RecordingTime,	Weight });
				}
								
				return TraceServices::EEventEnumerate::Continue;
			});
		});

		CurvesUpdateRequested = 0;
	}

	// update Icon:
	
	bool bChanged = false;

	if (CurveType == ECurveType::BlendWeight)
	{
		// Blend Weight track gets name/icon of animation
		if (CurveName.IsEmpty() && GameplayProvider)
		{
			TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
			if (const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(AssetId))
			{
				CurveName = FText::FromString(ObjectInfo->Name);
				bChanged = true;
				bool bIsBlendSpace = false;

				if (const UStruct* FoundType = GameplayProvider->FindType(ObjectInfo->ClassId))
				{
					Icon = FSlateIconFinder::FindIconForClass(FoundType);
#if WITH_EDITOR
					if (FoundType->IsChildOf(UBlendSpace::StaticClass()))
					{
						bIsBlendSpace = true;	
					}
#endif
				}

				// Blend Weight track gets child tracks for each of the other bits of extra data
				if (Children.Num() == 0)
				{
					int MaxCurve = static_cast<int>(bIsBlendSpace ? ECurveType::BlendSpaceFilteredPositionY : ECurveType::PlayRate);
					for (int i = static_cast<int>(ECurveType::BlendWeight) + 1; i <= MaxCurve; i++)
					{
						Children.Add(MakeShared<FBlendWeightTrack>(ObjectId, AssetId, NodeId, static_cast<ECurveType>(i)));
					}
				}
			}
		}

	}
	else
	{
		// other tracks get the curve name

		if (CurveName.IsEmpty())
		{
			switch (CurveType)
			{
				case ECurveType::BlendWeight:					break;
				case ECurveType::PlaybackTime:					CurveName = LOCTEXT("PlaybackTime", "Playback Time"); break;
				case ECurveType::RootMotionWeight:				CurveName = LOCTEXT("RootMotionWeight", "Root Motion Weight"); break;
				case ECurveType::PlayRate:						CurveName = LOCTEXT("PlaybackRate", "Playback Rate"); break;
				case ECurveType::BlendSpacePositionX:			CurveName = LOCTEXT("BlendSpaceX", "X"); break;
				case ECurveType::BlendSpacePositionY:			CurveName = LOCTEXT("BlendSpaceY", "Y"); break;
				case ECurveType::BlendSpaceFilteredPositionX:	CurveName = LOCTEXT("BlendSpaceXF", "X (Filtered)"); break;
				case ECurveType::BlendSpaceFilteredPositionY:	CurveName = LOCTEXT("BlendSpaceYF", "Y (Filtered)"); break;
			}

			Icon = FSlateIcon("EditorStyle", "AnimGraph.Attribute.Curves.Icon", "AnimGraph.Attribute.Curves.Icon");

			bChanged = true;
		}
	}

	for (auto& Child : Children)
	{
		bChanged |= Child->Update();
	}

	return bChanged;
}

static FLinearColor MakeBlendWeightCurveColor(uint32 InSeed, bool bInLine = false)
{
	FRandomStream Stream(InSeed);
	const uint8 Hue = (uint8)(Stream.FRand() * 255.0f);
	const uint8 SatVal = bInLine ? 196 : 128;
	return FLinearColor::MakeFromHSV8(Hue, SatVal, SatVal);
}

TSharedPtr<SWidget> FBlendWeightTrack::GetTimelineViewInternal()
{
	FLinearColor FillColor;
	FLinearColor CurveColor;
	FLinearColor SelectedColor;
	
	switch(CurveType)
	{
	case ECurveType::BlendWeight:
		FillColor = MakeBlendWeightCurveColor(CityHash32(reinterpret_cast<char*>(&AssetId), 8));
		FillColor.A = 0.5f;
		CurveColor = FillColor;
		CurveColor.R *= 0.5;
		CurveColor.G *= 0.5;
		CurveColor.B *= 0.5;
		SelectedColor = CurveColor;
		break;
	default:
		CurveColor = FLinearColor::MakeFromHSV8(100, 50, 65);
		SelectedColor = FLinearColor::MakeFromHSV8(100, 50, 130);
		break;
	}

	TSharedPtr<SCurveTimelineView> CurveTimelineView = SNew(SCurveTimelineView)
		.TrackName(GetDisplayNameInternal())
		.FillColor(FillColor)
		.CurveColor_Lambda([CurveColor, SelectedColor,  this]() { return GetIsSelected() ? SelectedColor : CurveColor; })
		.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
		.RenderFill(CurveType == ECurveType::BlendWeight)
		.CurveData_Raw(this, &FBlendWeightTrack::GetCurveData);

	if (CurveType == ECurveType::BlendWeight)
	{
		CurveTimelineView->SetFixedRange(0, 1);
	}

	return CurveTimelineView;
}

TSharedPtr<SWidget> FBlendWeightTrack::GetDetailsViewInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	TSharedPtr<SBlendWeightsView> BlendWeightsView = SNew(SBlendWeightsView, ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession())
				.CurrentTime_Lambda([RewindDebugger]{ return RewindDebugger->CurrentTraceTime(); });
	BlendWeightsView->SetAssetFilter(AssetId,NodeId);
	return BlendWeightsView;
}

bool FBlendWeightTrack::HandleDoubleClickInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlendWeightsTrack::UpdateInternal);
#if WITH_EDITOR
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
	{
		const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
		const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

		if (!GameplayProvider || !AnimationProvider)
		{
			return false;
		}

		bool bMessageFound = false;
		float PlaybackTime = 0;
		float BlendSpaceX = 0;
		float BlendSpaceY = 0;

		FString AssetPathName;
		{
			TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
			const FObjectInfo& AssetInfo = GameplayProvider->GetObjectInfo(GetAssetId());
			AssetPathName = AssetInfo.PathName;
		}

		const float CurrentTraceTime = IRewindDebugger::Instance()->CurrentTraceTime();

		bool bFrameFound;
		TraceServices::FFrame Frame;
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
			const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
			bFrameFound = FrameProvider.GetFrameFromTime(TraceFrameType_Game, CurrentTraceTime, Frame);
		}

		if (bFrameFound)
		{
			TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
			AnimationProvider->ReadTickRecordTimeline(ObjectId, [this, &bMessageFound, &PlaybackTime, &BlendSpaceX, &BlendSpaceY, &Frame](const FAnimationProvider::TickRecordTimeline& InTimeline)
			{
				InTimeline.EnumerateEvents(Frame.StartTime, Frame.EndTime, [this, &bMessageFound, &PlaybackTime, &BlendSpaceX, &BlendSpaceY, &Frame](double InStartTime, double InEndTime, uint32 InDepth, const FTickRecordMessage& InMessage)
				{
					if(InStartTime >= Frame.StartTime && InEndTime <= Frame.EndTime)
					{
						if (InMessage.NodeId == NodeId && InMessage.AssetId == AssetId)
						{
							bMessageFound = true;
							PlaybackTime = InMessage.PlaybackTime;
							BlendSpaceX = InMessage.BlendSpacePositionX;
							BlendSpaceY = InMessage.BlendSpacePositionY;
							return TraceServices::EEventEnumerate::Stop;
						}
					}
					return TraceServices::EEventEnumerate::Continue;
				});
			});
		}

		UObject* Asset = nullptr;
		FString PackagePathString = FPackageName::ObjectPathToPackageName(AssetPathName);

		UPackage* Package = LoadPackage(NULL, ToCStr(PackagePathString), LOAD_NoRedirects);
		if (Package)
		{
			Package->FullyLoad();
                
			FString AssetName = FPaths::GetBaseFilename(AssetPathName);
			Asset = FindObject<UObject>(Package, *AssetName);
		}
		else
		{
			// fallback for unsaved assets
			Asset = FindObject<UObject>(nullptr, *AssetPathName);
		}
                    	
		if (Asset != nullptr)
		{
			if (UAssetEditorSubsystem* AssetEditorSS = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
			{
				AssetEditorSS->OpenEditorForAsset(Asset);

				if (bMessageFound)
				{
					// if the asset is playing on the current frame, scrub to the appropriate time
					if (IAssetEditorInstance* Editor = AssetEditorSS->FindEditorForAsset(Asset, true))
					{
						if (Editor->GetEditorName()=="AnimationEditor")
						{
							IAnimationEditor* AnimationEditor = static_cast<IAnimationEditor*>(Editor);
							UDebugSkelMeshComponent* PreviewComponent = AnimationEditor->GetPersonaToolkit()->GetPreviewMeshComponent();
							PreviewComponent->PreviewInstance->SetPosition(PlaybackTime);
							PreviewComponent->PreviewInstance->SetPlaying(false);
							PreviewComponent->PreviewInstance->SetBlendSpacePosition(FVector(BlendSpaceX, BlendSpaceY, 0.0f));
						}
					}
				}
			}
		}

		return true;
	}
#endif
	return false;
}

#if WITH_ENGINE
FName FBlendWeightsTrackCreator::GetTargetTypeNameInternal() const
{
	return UAnimInstance::StaticClass()->GetFName();
}
	
static const FName BlendWeightsName("BlendWeights");

FName FBlendWeightsTrackCreator::GetNameInternal() const
{
	return BlendWeightsName;
}
	
void FBlendWeightsTrackCreator::GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const
{
	Types.Add({BlendWeightsName, LOCTEXT("Blend Weights", "Blend Weights")});
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FBlendWeightsTrackCreator::CreateTrackInternal(const FObjectId& InObjectId) const
{
 	return MakeShared<RewindDebugger::FBlendWeightsTrack>(InObjectId.GetMainId());
}

bool FBlendWeightsTrackCreator::HasDebugInfoInternal(const FObjectId& InObjectId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlendWeightsTrack::HasDebugInfoInternal);
	const TraceServices::IAnalysisSession* AnalysisSession = IRewindDebugger::Instance()->GetAnalysisSession();
	bool bHasData = false;
	if (const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName))
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadTickRecordTimeline(InObjectId.GetMainId(), [&bHasData](const FAnimationProvider::TickRecordTimeline& InGraphTimeline)
		{
			bHasData = true;
		});
	}
	
	return bHasData;
}
#endif // #if WITH_ENGINE

}
#undef LOCTEXT_NAMESPACE
