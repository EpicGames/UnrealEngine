// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAnimationProvider.h"
#include "IRewindDebugger.h"
#include "RewindDebuggerTrack.h"
#include "SCurveTimelineView.h"
#include "Textures/SlateIcon.h"
#include "IRewindDebuggerTrackCreator.h"

namespace RewindDebugger
{

class FBlendWeightTrack : public FRewindDebuggerTrack
{
public:
	enum class ECurveType
	{
		BlendWeight,
		PlaybackTime,
		RootMotionWeight,
		PlayRate,
		BlendSpacePositionX,
		BlendSpacePositionY,
		BlendSpaceFilteredPositionX,
		BlendSpaceFilteredPositionY,
	};
	
	FBlendWeightTrack(uint64 InObjectId, uint64 AssetId, uint32 NodeId, ECurveType CurveType = ECurveType::BlendWeight);

	uint32 GetNodeId() const { return NodeId; };
	uint64 GetAssetId() const { return AssetId; }

private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;

	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FName GetNameInternal() const override { return "BlendWeight"; }
	virtual FText GetDisplayNameInternal() const override { return CurveName; }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }
	virtual bool HandleDoubleClickInternal() override;

	TSharedPtr<SCurveTimelineView::FTimelineCurveData> GetCurveData() const;
	
	uint64 AssetId;
	uint32 NodeId;

	ECurveType CurveType;
	
	FSlateIcon Icon;
	FText CurveName;
	uint64 ObjectId;

	mutable TSharedPtr<SCurveTimelineView::FTimelineCurveData> CurveData;
	mutable int CurvesUpdateRequested = 0;
	
	TArray<TSharedPtr<FBlendWeightTrack>> Children;
};

class FBlendWeightsTrack : public FRewindDebuggerTrack
{
public:
	FBlendWeightsTrack(uint64 InObjectId);

private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;

	virtual FName GetNameInternal() const override { return "BlendWeights"; }
	virtual FText GetDisplayNameInternal() const override { return NSLOCTEXT("RewindDebugger", "Blend Weights Track Name", "Blend Weights"); }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }

	virtual FSlateIcon GetIconInternal() override { return Icon; }

	FSlateIcon Icon;
	uint64 ObjectId;
	TArray<TSharedPtr<FBlendWeightTrack>> Children;
};

#if WITH_ENGINE
class FBlendWeightsTrackCreator : public IRewindDebuggerTrackCreator
{
public:
	virtual FName GetTargetTypeNameInternal() const;
	virtual FName GetNameInternal() const override;
	virtual void GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<FRewindDebuggerTrack> CreateTrackInternal(const FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const FObjectId& InObjectId) const override;
};
#endif

}