// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoolPropertyTrack.h"

#include "Common/ProviderLock.h"
#include "GameplayProvider.h"
#include "PropertyHelpers.h"

namespace RewindDebugger
{
	///////////////////////////////////////////////////////////////////////////////
	// Bool Property Track
	///////////////////////////////////////////////////////////////////////////////

	FBoolPropertyTrack::FBoolPropertyTrack(uint64 InObjectId, const TSharedPtr<FObjectPropertyInfo> & InObjectProperty, const TSharedPtr<FPropertyTrack> & InParentTrack) : FPropertyTrack(InObjectId, InObjectProperty, InParentTrack)
	{
		EnabledSegments = MakeShared<SSegmentedTimelineView::FSegmentData>();
	}

	bool FBoolPropertyTrack::CanBeCreated(const FObjectPropertyInfo& InObjectProperty)
	{
		const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
		check(RewindDebugger)

		if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
		{
			if (const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName))
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
				return FObjectPropertyHelpers::IsBoolProperty(InObjectProperty.Property.Value, GameplayProvider->GetPropertyName(InObjectProperty.Property.TypeStringId));
			}
		}
		
		return false;
	}
	
	bool FBoolPropertyTrack::UpdateInternal()
	{
		const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
		check(RewindDebugger)
		
		// Convert time range to from rewind debugger times to profiler times
		const TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
		const double StartTime = TraceTimeRange.GetLowerBoundValue();
		const double EndTime = TraceTimeRange.GetUpperBoundValue();

		// Clear previous data
		TArray<TRange<double>> & Segments = EnabledSegments->Segments;
		Segments.SetNum(0, EAllowShrinking::No);

		bool bPrevValue = false;
		double SegmentStartTime = 0.0;
		double PrevTime = 0.0f;
		
		ReadObjectPropertyValueOverTime(StartTime, EndTime, [&Segments, &SegmentStartTime, &bPrevValue, &PrevTime](const FObjectPropertyValue& InValue, uint32 InValueIndex, const FObjectPropertiesMessage& InMessage, const IGameplayProvider::ObjectPropertiesTimeline &  InTimeline , double InStart, double InEndTime)
		{
			PrevTime = InMessage.ElapsedTime;

			const bool CurrentValue = FString(InValue.Value).ToBool();
			
			// From false to true (Start)
			if (!bPrevValue && CurrentValue)
			{
				SegmentStartTime = InMessage.ElapsedTime;
				bPrevValue = true;
			}

			// From true to false (End)
			else if (bPrevValue && !CurrentValue)
			{
				bPrevValue = false;

				Segments.Add(TRange<double>(SegmentStartTime, InMessage.ElapsedTime));
			}
		});

		// Handle open segment.
		if (bPrevValue)
		{
			Segments.Add(TRange<double>(SegmentStartTime, PrevTime));
		}
		
		// Update child tracks.
		return FPropertyTrack::UpdateInternal();
	}

	TSharedPtr<SWidget> FBoolPropertyTrack::GetTimelineViewInternal()
	{
		const auto TimelineView = SNew(SSegmentedTimelineView)
			.FillColor(FObjectPropertyHelpers::GetPropertyColor(ObjectProperty->Property))
			.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
			.SegmentData_Raw(this, &FBoolPropertyTrack::GetSegmentData);

		return TimelineView;
	}

	TSharedPtr<SSegmentedTimelineView::FSegmentData> FBoolPropertyTrack::GetSegmentData() const
	{
		return EnabledSegments;
	}
}