// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyTrack.h"
#include "SSegmentedTimelineView.h"

namespace RewindDebugger
{
	/**
	 * Track that displays a object's boolean property value overtime
	 */
	class FBoolPropertyTrack : public FPropertyTrack
	{
	public:

		BASE_PROPERTY_TRACK()
		
		/** Constructor */
		FBoolPropertyTrack(uint64 InObjectId, const TSharedPtr<FObjectPropertyInfo> & InObjectProperty, const TSharedPtr<FPropertyTrack> & InParentTrack);

	private:
		
		/** Begin IRewindDebuggerTrack interface */
		virtual bool UpdateInternal() override;
		virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
		/** End IRewindDebuggerTrack interface */

		/** @return Segmented boolean data for watched property */
		TSharedPtr<SSegmentedTimelineView::FSegmentData> GetSegmentData() const;

		/** Keep track of segments when the traced value was true */
		TSharedPtr<SSegmentedTimelineView::FSegmentData> EnabledSegments;
	};
}
