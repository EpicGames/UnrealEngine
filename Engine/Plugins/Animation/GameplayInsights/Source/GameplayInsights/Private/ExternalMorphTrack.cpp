// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExternalMorphTrack.h"
#include "Common/ProviderLock.h"
#include "SCurveTimelineView.h"
#include "IRewindDebugger.h"
#include "GameplayProvider.h"
#include "AnimationProvider.h"
#include "SAnimationCurvesView.h"
#if WITH_ENGINE
#include "Components/SkeletalMeshComponent.h"
#endif //WITH_ENGINE 

#define LOCTEXT_NAMESPACE "ExternalMorphTrack"

namespace RewindDebugger
{
	FExternalMorphSetGroupTrack::FExternalMorphSetGroupTrack(uint64 InObjectId)
		: ObjectId(InObjectId)
	{
		SetIsExpanded(false);
		Icon = FSlateIcon("EditorStyle", "AnimGraph.Attribute.Curves.Icon", "AnimGraph.Attribute.Curves.Icon");
	}

	TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> FExternalMorphSetGroupTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
	{
		return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(Children.GetData()), Children.Num());
	}

	bool FExternalMorphSetGroupTrack::UpdateInternal()
	{
		IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
		const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
		const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
		if (AnimationProvider == nullptr)
		{
			return false;
		}

		// Convert time range to from rewind debugger times to profiler times.
		TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
		double StartTime = TraceTimeRange.GetLowerBoundValue();
		double EndTime = TraceTimeRange.GetUpperBoundValue();

		TArray<int32> MorphSetIndexValues;
		bool bChanged = false;
		{
			TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
			AnimationProvider->ReadSkeletalMeshPoseTimeline(ObjectId, [this, AnimationProvider, StartTime, EndTime, &MorphSetIndexValues](const FAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bHasCurves)
			{
				InTimeline.EnumerateEvents(StartTime, EndTime, [this, StartTime, EndTime, AnimationProvider, &MorphSetIndexValues](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InMessage)
				{
					if (InEndTime > StartTime && InStartTime < EndTime)
					{
						AnimationProvider->EnumerateExternalMorphSets(InMessage, [this, &MorphSetIndexValues](const FExternalMorphWeightMessage& InSet)
						{
							MorphSetIndexValues.AddUnique(InSet.Index);
						});
					}
					return TraceServices::EEventEnumerate::Continue;
				});
			});
		}

		if (MorphSetIndexValues.Num() != Children.Num() || PrevMorphSetIndexValues != MorphSetIndexValues)
		{
			bChanged = true;
			Children.SetNum(MorphSetIndexValues.Num());
			for (int32 Index = 0; Index < MorphSetIndexValues.Num(); ++Index)
			{
				TSharedPtr<FExternalMorphSetTrack> NewTrack = MakeShared<FExternalMorphSetTrack>(ObjectId, MorphSetIndexValues[Index]);
				Children[Index] = NewTrack;
			}
		}

		for (int32 Index = 0; Index < Children.Num(); ++Index)
		{
			bChanged |= Children[Index]->Update();
		}

		PrevMorphSetIndexValues = MorphSetIndexValues;

		return bChanged;
	}

	//----------------------------------------------------------------------------------------------

	FExternalMorphSetTrack::FExternalMorphSetTrack(uint64 InObjectId, int32 InMorphSetIndex)
		: ObjectId(InObjectId)
		, MorphSetIndex(InMorphSetIndex)
	{
		SetIsExpanded(false);
		Icon = FSlateIcon("EditorStyle", "AnimGraph.Attribute.Curves.Icon", "AnimGraph.Attribute.Curves.Icon");
	}

	TSharedPtr<SWidget> FExternalMorphSetTrack::GetDetailsViewInternal()
	{
		IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
		return SNew(SAnimationCurvesView, ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession())
					.CurrentTime_Lambda([RewindDebugger]{ return RewindDebugger->CurrentTraceTime(); });
	
	}

	TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> FExternalMorphSetTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
	{
		return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(Children.GetData()), Children.Num());
	}

	bool FExternalMorphSetTrack::UpdateInternal()
	{
		IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
		const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
		const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
		if (AnimationProvider == nullptr)
		{
			return false;
		}

		// Convert time range to from rewind debugger times to profiler times.
		TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
		double StartTime = TraceTimeRange.GetLowerBoundValue();
		double EndTime = TraceTimeRange.GetUpperBoundValue();
	
		int32 NumMorphTracks = 0;
		bool bChanged = false;
		{
			TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
			AnimationProvider->ReadSkeletalMeshPoseTimeline(ObjectId, [this, AnimationProvider, StartTime, EndTime, &NumMorphTracks](const FAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bHasCurves)
			{
				InTimeline.EnumerateEvents(StartTime, EndTime, [this, StartTime, EndTime, AnimationProvider, &NumMorphTracks](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InMessage)
				{
					if (InEndTime > StartTime && InStartTime < EndTime)
					{
						AnimationProvider->EnumerateExternalMorphSets(InMessage, [this, &NumMorphTracks](const FExternalMorphWeightMessage& InSet)
						{
							if (InSet.Index == MorphSetIndex)
							{
								NumMorphTracks = InSet.Weights.Num();
							}
						});
					}
					return TraceServices::EEventEnumerate::Continue;
				});
			});
		}

		if (NumMorphTracks != Children.Num())
		{
			bChanged = true;
			Children.SetNum(NumMorphTracks);
			for (int32 Index = 0; Index < NumMorphTracks; ++Index)
			{
				TSharedPtr<FExternalMorphTrack> NewTrack = MakeShared<FExternalMorphTrack>(ObjectId, Index, MorphSetIndex);
				Children[Index] = NewTrack;
			}
		}

		check(Children.Num() == NumMorphTracks);
		for (int32 Index = 0; Index < NumMorphTracks; ++Index)
		{
			bChanged |= Children[Index]->Update();
		}

		return bChanged;
	}

	//----------------------------------------------------------------------------------------------

	FExternalMorphTrack::FExternalMorphTrack(uint64 InObjectId, int32 InMorphIndex, int32 InMorphSetIndex) :
		MorphIndex(InMorphIndex),
		MorphSetIndex(InMorphSetIndex),
		ObjectId(InObjectId)
	{
		Icon = FSlateIcon("EditorStyle", "AnimGraph.Attribute.Curves.Icon", "AnimGraph.Attribute.Curves.Icon");
	}

	TSharedPtr<SCurveTimelineView::FTimelineCurveData> FExternalMorphTrack::GetCurveData() const
	{
		if (!CurveData.IsValid())
		{
			CurveData = MakeShared<SCurveTimelineView::FTimelineCurveData>();
		}
	
		CurvesUpdateRequested++;
		return CurveData;
	}

	bool FExternalMorphTrack::UpdateInternal()
	{
		IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
		const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
		const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

		if (CurvesUpdateRequested > 25 && AnimationProvider)
		{
			CurvesUpdateRequested = 0;
			UpdateCurvePointsInternal();
		}

		bool bChanged = false;
		if (Name.IsEmpty())
		{
			Name = FText::Format(LOCTEXT("MorphNameFormat", "Morph Target {0}"), MorphIndex);
			bChanged = true;
		}

		return bChanged;
	}

	void FExternalMorphTrack::UpdateCurvePointsInternal()
	{
		IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
		const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
		const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

		// Convert time range to from rewind debugger times to profiler times.
		TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
		double StartTime = TraceTimeRange.GetLowerBoundValue();
		double EndTime = TraceTimeRange.GetUpperBoundValue();
	
		TArray<SCurveTimelineView::FTimelineCurveData::CurvePoint>& CurvePoints = CurveData->Points;
		CurvePoints.Reset();
		CurvePoints.Reserve(256);

		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadSkeletalMeshPoseTimeline(ObjectId, [this, AnimationProvider, StartTime, EndTime, AnalysisSession, &CurvePoints](const FAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bHasCurves)
		{
			InTimeline.EnumerateEvents(StartTime, EndTime, [this, StartTime, EndTime, AnimationProvider, AnalysisSession, &CurvePoints](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InMessage)
			{
				if (InEndTime > StartTime && InStartTime < EndTime)
				{
					const double Time = InMessage.RecordingTime;		
					AnimationProvider->EnumerateExternalMorphSets(InMessage, [this, Time, &CurvePoints](const FExternalMorphWeightMessage& InWeight)
					{
						if (InWeight.Index == MorphSetIndex)
						{
							CurvePoints.Add({Time, InWeight.Weights[MorphIndex]});
						}
					});
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}

	TSharedPtr<SWidget> FExternalMorphTrack::GetDetailsViewInternal() 
	{
		IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
		TSharedPtr<SAnimationCurvesView> AnimationCurvesView = SNew(SAnimationCurvesView, ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession())
			.CurrentTime_Lambda([RewindDebugger]{ return RewindDebugger->CurrentTraceTime(); });
		return AnimationCurvesView;
	}

	TSharedPtr<SWidget> FExternalMorphTrack::GetTimelineViewInternal()
	{
		FLinearColor CurveColor(0.5,0.5,0.5);
	
		return SNew(SCurveTimelineView)
			.CurveColor(CurveColor)
			.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
			.RenderFill(false)
			.CurveData_Raw(this, &FExternalMorphTrack::GetCurveData);
	}

	//----------------------------------------------------------------------------------------------
#if WITH_ENGINE
	FName FExternalMorphSetGroupTrackCreator::GetTargetTypeNameInternal() const
	{
		return USkeletalMeshComponent::StaticClass()->GetFName();
	}
	
	static const FName ExternalMorphSetName("ExternalMorphSetGroup");
	
	FName FExternalMorphSetGroupTrackCreator::GetNameInternal() const
	{
		return ExternalMorphSetName;
	}
	
	void FExternalMorphSetGroupTrackCreator::GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const 
    {
    	Types.Add({ExternalMorphSetName, LOCTEXT("External Morph Sets", "External Morph Sets")});
    }

	TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FExternalMorphSetGroupTrackCreator::CreateTrackInternal(const FObjectId& InObjectId) const
	{
		return MakeShared<RewindDebugger::FExternalMorphSetGroupTrack>(InObjectId.GetMainId());
	}

	bool FExternalMorphSetGroupTrackCreator::HasDebugInfoInternal(const FObjectId& InObjectId) const
	{
		const TraceServices::IAnalysisSession* AnalysisSession = IRewindDebugger::Instance()->GetAnalysisSession();
		if (const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
			return AnimationProvider->HasExternalMorphSets(InObjectId.GetMainId());
		}
		return false;
	}
#endif // WITH_ENGINE
}

#undef LOCTEXT_NAMESPACE
