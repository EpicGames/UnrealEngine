// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAnimationProvider.h"
#include "IRewindDebuggerTrackCreator.h"
#include "RewindDebuggerTrack.h"
#include "SCurveTimelineView.h"
#include "Textures/SlateIcon.h"

namespace RewindDebugger
{
	// Sub Track for a external morph target.
	class FExternalMorphTrack
		: public FRewindDebuggerTrack
	{
	public:
		FExternalMorphTrack(uint64 InObjectId, int32 InMorphIndex, int32 InMorphSetIndex);
	
		int32 GetMorphIndex() const { return MorphIndex; }

	private:
		// FRewindDebuggerTrack overrides.
		virtual bool UpdateInternal() override;
		TSharedPtr<SWidget> GetDetailsViewInternal() override;
		virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
		virtual FSlateIcon GetIconInternal() override						{ return Icon; }
		virtual FName GetNameInternal() const override						{ return "ExternalMorphTarget"; }
		virtual FText GetDisplayNameInternal() const override				{ return Name; }
		virtual uint64 GetObjectIdInternal() const override					{ return ObjectId; }
		// ~END FRewindDebuggerTrack overrides.
	
		TSharedPtr<SCurveTimelineView::FTimelineCurveData> GetCurveData() const;

	protected:
		void UpdateCurvePointsInternal();

	protected:

		int32 MorphIndex = INDEX_NONE;
		int32 MorphSetIndex = INDEX_NONE;
		uint64 ObjectId = 0;
		FText Name;
		FSlateIcon Icon;
		mutable TSharedPtr<SCurveTimelineView::FTimelineCurveData> CurveData;
		mutable int CurvesUpdateRequested = 0;
	};


	// The external morph target set, which is the parent of a collection of FExternalMorphTrack items.
	class FExternalMorphSetTrack
		: public FRewindDebuggerTrack
	{
	public:
		FExternalMorphSetTrack(uint64 InObjectId, int32 InMorphSetIndex);

	private:
		// FRewindDebuggerTrack overrides.
		virtual bool UpdateInternal() override;
		virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
		virtual FSlateIcon GetIconInternal() override						{ return Icon; }
		virtual FName GetNameInternal() const override						{ return "ExternalMorphSet"; }
		virtual FText GetDisplayNameInternal() const override				{ return NSLOCTEXT("RewindDebugger", "ExternalMorphSetTrackName", "External Morph Set"); }
		virtual uint64 GetObjectIdInternal() const override					{ return ObjectId; }
		virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;
		// ~END FRewindDebuggerTrack overrides.
	
	protected:
		FSlateIcon Icon;
		uint64 ObjectId;
		int32 MorphSetIndex;
		TArray<TSharedPtr<FExternalMorphTrack>> Children;
	};


	// The external morph set group track, which is the parent of all morph sets.
	class FExternalMorphSetGroupTrack
		: public FRewindDebuggerTrack
	{
	public:
		FExternalMorphSetGroupTrack(uint64 InObjectId);

	private:
		// FRewindDebuggerTrack overrides.
		virtual bool UpdateInternal() override;
		virtual FSlateIcon GetIconInternal() override						{ return Icon; }
		virtual FName GetNameInternal() const override						{ return "ExternalMorphSetGroup"; }
		virtual FText GetDisplayNameInternal() const override				{ return NSLOCTEXT("RewindDebugger", "ExternalMorphSetGroupTrackName", "External Morph Sets"); }
		virtual uint64 GetObjectIdInternal() const override					{ return ObjectId; }
		virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;
		// ~END FRewindDebuggerTrack overrides.
	
	protected:
		FSlateIcon Icon;
		uint64 ObjectId;
		TArray<TSharedPtr<FExternalMorphSetTrack>> Children;
		TArray<int32> PrevMorphSetIndexValues;
	};


#if WITH_ENGINE
	// The creator for the external morph set track.
	class FExternalMorphSetGroupTrackCreator
		: public IRewindDebuggerTrackCreator
	{
	private:
		// IRewindDebuggerTrackCreator overrides.
		virtual FName GetTargetTypeNameInternal() const override;
		virtual FName GetNameInternal() const override;
		virtual void GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const override;
		virtual TSharedPtr<FRewindDebuggerTrack> CreateTrackInternal(const FObjectId& InObjectId) const override;
		virtual bool HasDebugInfoInternal(const FObjectId& InObjectId) const override;
		// ~END IRewindDebuggerTrackCreator overrides.
	};
#endif // WITH_ENGINE

}	// namespace RewindDebugger