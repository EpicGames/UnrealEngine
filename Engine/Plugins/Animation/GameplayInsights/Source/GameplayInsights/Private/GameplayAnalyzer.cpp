// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayAnalyzer.h"

#include "GameplayProvider.h"
#include "HAL/LowLevelMemTracker.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Utils.h"

FGameplayAnalyzer::FGameplayAnalyzer(TraceServices::IAnalysisSession& InSession, FGameplayProvider& InGameplayProvider)
	: Session(InSession)
	, GameplayProvider(InGameplayProvider)
{
}

void FGameplayAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_RecordingInfo, "Object", "RecordingInfo");
	Builder.RouteEvent(RouteId_Type, "Object", "Class");
	Builder.RouteEvent(RouteId_Type, "Object", "Type");
	Builder.RouteEvent(RouteId_Object, "Object", "Object");
	Builder.RouteEvent(RouteId_ObjectTransform, "Object", "ObjectTransform");
	Builder.RouteEvent(RouteId_ObjectEvent, "Object", "ObjectEvent");
	Builder.RouteEvent(RouteId_ObjectLifetimeBegin, "Object", "ObjectLifetimeBegin");
	Builder.RouteEvent(RouteId_ObjectLifetimeBegin2, "Object", "ObjectLifetimeBegin2");
	Builder.RouteEvent(RouteId_ObjectLifetimeEnd, "Object", "ObjectLifetimeEnd");
	Builder.RouteEvent(RouteId_ObjectLifetimeEnd2, "Object", "ObjectLifetimeEnd2");
	Builder.RouteEvent(RouteId_PawnPossess, "Object", "PawnPossess");
	Builder.RouteEvent(RouteId_World, "Object", "World");
	Builder.RouteEvent(RouteId_View, "Object", "View");
	Builder.RouteEvent(RouteId_ClassPropertyStringId, "Object", "ClassPropertyStringId");
	Builder.RouteEvent(RouteId_PropertiesStart, "Object", "PropertiesStart");
	Builder.RouteEvent(RouteId_PropertiesStart2, "Object", "PropertiesStart2");
	Builder.RouteEvent(RouteId_PropertiesEnd, "Object", "PropertiesEnd");
	Builder.RouteEvent(RouteId_PropertyValue2, "Object", "PropertyValue2");
	Builder.RouteEvent(RouteId_PropertyValue3, "Object", "PropertyValue3");
}

bool FGameplayAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FGameplayAnalyzer"));

	TraceServices::FProviderEditScopeLock ProviderEditScope(GameplayProvider);

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
	case RouteId_World:
	{
		uint64 Id = EventData.GetValue<uint64>("Id");
		int32 PIEInstanceId = EventData.GetValue<int32>("PIEInstanceId");
		uint8 Type = EventData.GetValue<uint8>("Type");
		uint8 NetMode = EventData.GetValue<uint8>("NetMode");
		bool bIsSimulating = EventData.GetValue<bool>("IsSimulating");
		GameplayProvider.AppendWorld(Id, PIEInstanceId, Type, NetMode, bIsSimulating);
		break;
	}
	case RouteId_RecordingInfo:
	{
		uint64 WorldId = EventData.GetValue<uint64>("WorldId");
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint32 RecordingIndex = EventData.GetValue<uint32>("RecordingIndex");
		uint32 FrameIndex = EventData.GetValue<uint32>("FrameIndex");
		double ElapsedTime = EventData.GetValue<double>("ElapsedTime");
		GameplayProvider.AppendRecordingInfo(WorldId, Context.EventTime.AsSeconds(Cycle), RecordingIndex, FrameIndex, ElapsedTime);
		break;
	}
	case RouteId_Type:
	{
		FString TypeName, TypePathName;
		if (EventData.GetString("Name", TypeName))
		{
			EventData.GetString("Path", TypePathName);
		}
		else
		{
			const TCHAR* TypeNameAndPathName = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
			TypeName = TypeNameAndPathName;
			int32 TypeNameStringLength = EventData.GetValue<int32>("TypeNameStringLength");
			TypePathName = TypeNameAndPathName + TypeNameStringLength;
		}

		uint64 Id = EventData.GetValue<uint64>("Id");
		uint64 SuperId = EventData.GetValue<uint64>("SuperId");
		GameplayProvider.AppendClass(Id, SuperId, *TypeName, *TypePathName);
		break;
	}
	case RouteId_Object:
	{
		FString ObjectName, ObjectPathName;
		if (EventData.GetString("Name", ObjectName))
		{
			EventData.GetString("Path", ObjectPathName);
		}
		else
		{
			const TCHAR* ObjectNameAndPathName = reinterpret_cast<const TCHAR*>(EventData.GetAttachment());
			ObjectName = ObjectNameAndPathName;
			int32 NameStringLength = EventData.GetValue<int32>("ObjectNameStringLength");
			ObjectPathName = ObjectNameAndPathName + NameStringLength;
		}
		uint64 Id = EventData.GetValue<uint64>("Id");
		uint64 OuterId = EventData.GetValue<uint64>("OuterId");
		uint64 ClassId = EventData.GetValue<uint64>("ClassId");
		EObjectInfoFlags Flags = static_cast<EObjectInfoFlags>(EventData.GetValue<uint8>("Flags", 0));
		GameplayProvider.AppendObject(RewindDebugger::FObjectId{Id}, RewindDebugger::FObjectId{OuterId}, ClassId, *ObjectName, *ObjectPathName, Flags);
		break;
	}
	case RouteId_ObjectTransform:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double RecordingTime = EventData.GetValue<double>("RecordingTime");
		uint64 Id = EventData.GetValue<uint64>("Id");
		TArrayView<const double> TransformArray = EventData.GetArrayView<double>("Transform");
		FTransform Transform;
	 	FMemory::Memcpy(&Transform, &TransformArray[0], sizeof(FTransform));
		GameplayProvider.AppendObjectTransform(Id, Context.EventTime.AsSeconds(Cycle), RecordingTime, Transform);
		break;
	}
	case RouteId_ObjectEvent:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 Id = EventData.GetValue<uint64>("Id");
		FString Event = TraceServices::FTraceAnalyzerUtils::LegacyAttachmentString<TCHAR>("Event", Context);
		GameplayProvider.AppendObjectEvent(RewindDebugger::FObjectId{Id}, Context.EventTime.AsSeconds(Cycle), *Event);
		break;
	}
	case RouteId_ObjectLifetimeBegin:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 Id = EventData.GetValue<uint64>("Id");
		GameplayProvider.AppendObjectLifetimeBegin(RewindDebugger::FObjectId{Id}, Context.EventTime.AsSeconds(Cycle), 0.0);
		break;
	}
	case RouteId_ObjectLifetimeBegin2:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double RecordingTime = EventData.GetValue<double>("RecordingTime");
		uint64 Id = EventData.GetValue<uint64>("Id");
		GameplayProvider.AppendObjectLifetimeBegin(RewindDebugger::FObjectId{Id}, Context.EventTime.AsSeconds(Cycle), RecordingTime);
		break;
	}
	case RouteId_ObjectLifetimeEnd:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 Id = EventData.GetValue<uint64>("Id");
		GameplayProvider.AppendObjectLifetimeEnd(RewindDebugger::FObjectId{Id}, Context.EventTime.AsSeconds(Cycle), 0.0);
		break;
	}
	case RouteId_ObjectLifetimeEnd2:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		double RecordingTime = EventData.GetValue<double>("RecordingTime");
		uint64 Id = EventData.GetValue<uint64>("Id");
		GameplayProvider.AppendObjectLifetimeEnd(RewindDebugger::FObjectId{Id}, Context.EventTime.AsSeconds(Cycle), RecordingTime);
		break;
	}
	case RouteId_PawnPossess:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 ControllerId = EventData.GetValue<uint64>("ControllerId");
		uint64 PawnId = EventData.GetValue<uint64>("PawnId");
		GameplayProvider.AppendPawnPossess(ControllerId, PawnId, Context.EventTime.AsSeconds(Cycle));
		break;
	}
	case RouteId_View:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 PlayerId = EventData.GetValue<uint64>("PlayerId");
		FVector Position( EventData.GetValue<double>("PosX"), EventData.GetValue<double>("PosY"), EventData.GetValue<double>("PosZ"));
		FRotator Rotation( EventData.GetValue<float>("Pitch"), EventData.GetValue<float>("Yaw"), EventData.GetValue<float>("Roll"));
		float Fov = EventData.GetValue<float>("Fov");
		float AspectRatio  = EventData.GetValue<float>("AspectRatio");
		GameplayProvider.AppendView(PlayerId, Context.EventTime.AsSeconds(Cycle), Position, Rotation, Fov, AspectRatio);
		break;
	}
	case RouteId_ClassPropertyStringId:
	{
		uint32 Id = EventData.GetValue<uint32>("Id");
		FStringView Value; EventData.GetString("Value", Value);
		GameplayProvider.AppendClassPropertyStringId(Id, Value);
		break;
	}
	case RouteId_PropertiesStart:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 ObjectId = EventData.GetValue<uint64>("ObjectId");
		GameplayProvider.AppendPropertiesStart(ObjectId, Context.EventTime.AsSeconds(Cycle), Cycle, 0);
		break;
	}
	case RouteId_PropertiesStart2:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 ObjectId = EventData.GetValue<uint64>("ObjectId");
		double RecordingTime = EventData.GetValue<double>("RecordingTime");
		GameplayProvider.AppendPropertiesStart(ObjectId, Context.EventTime.AsSeconds(Cycle), Cycle, RecordingTime);
		break;
	}
	case RouteId_PropertiesEnd:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 ObjectId = EventData.GetValue<uint64>("ObjectId");
		GameplayProvider.AppendPropertiesEnd(ObjectId, Context.EventTime.AsSeconds(Cycle));
		break;
	}
	case RouteId_PropertyValue2:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 ObjectId = EventData.GetValue<uint64>("ObjectId");
		int32 ParentId = EventData.GetValue<int32>("ParentId");
		uint32 TypeId = EventData.GetValue<uint32>("TypeId");
		uint32 KeyId = EventData.GetValue<uint32>("KeyId");

		FStringView Value; EventData.GetString("Value", Value);
		GameplayProvider.AppendPropertyValue(ObjectId, Context.EventTime.AsSeconds(Cycle), Cycle, ParentId, TypeId, KeyId, 0, Value);
		break;
	}
	case RouteId_PropertyValue3:
	{
		uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		uint64 ObjectId = EventData.GetValue<uint64>("ObjectId");
		int32 ParentId = EventData.GetValue<int32>("ParentId");
		uint32 TypeId = EventData.GetValue<uint32>("TypeId");
		uint32 NameId = EventData.GetValue<uint32>("NameId");
		uint32 ParentNameId = EventData.GetValue<uint32>("ParentNameId");

		FStringView Value; EventData.GetString("Value", Value);
		GameplayProvider.AppendPropertyValue(ObjectId, Context.EventTime.AsSeconds(Cycle), Cycle, ParentId, TypeId, NameId, ParentNameId, Value);
		break;
	}
	}

	return true;
}
