// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

class FGameplayProvider;
namespace TraceServices { class IAnalysisSession; }

class FGameplayAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FGameplayAnalyzer(TraceServices::IAnalysisSession& InSession, FGameplayProvider& InGameplayProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override {}
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_World,
		RouteId_Type,
		RouteId_Object,
		RouteId_ObjectTransform,
		RouteId_ObjectEvent,
		RouteId_ObjectLifetimeBegin,
		RouteId_ObjectLifetimeBegin2,
		RouteId_ObjectLifetimeEnd,
		RouteId_ObjectLifetimeEnd2,
		RouteId_PawnPossess,
		RouteId_View,
		RouteId_ClassPropertyStringId,
		RouteId_ClassProperty,
		RouteId_PropertiesStart,
		RouteId_PropertiesStart2,
		RouteId_PropertiesEnd,
		RouteId_PropertyValue2,
		RouteId_PropertyValue3,
		RouteId_RecordingInfo,
	};

	TraceServices::IAnalysisSession& Session;
	FGameplayProvider& GameplayProvider;
};
