// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayGraphTrack.h"

#include "AnimationProvider.h"
#include "Fonts/FontMeasure.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameplayProvider.h"
#include "GameplaySharedData.h"
#include "Modules/ModuleManager.h"

// TraceServices
#include "TraceServices/Model/Frames.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"

// TraceInsights
#include "Insights/ViewModels/GraphTrackBuilder.h"
#include "Insights/ViewModels/ITimingViewDrawHelper.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"

#define LOCTEXT_NAMESPACE "GameplayGraphTrack"

INSIGHTS_IMPLEMENT_RTTI(FGameplayGraphTrack)

static float GetSeriesHeight(const FTimingViewLayout& InLayout)
{
	// 1.0f is for the top horizontal line of each track
	return (InLayout.EventDY + InLayout.EventH);
}

void FGameplayGraphSeries::ComputePosition(const FTimingTrackViewport& InViewport, const FGameplayGraphTrack& InTrack, int32 InActiveSeriesIndex, float& OutTopY, float& OutBottomY) const
{
	const float ActiveSeriesIndex = InTrack.GetLayout() == EGameplayGraphLayout::Overlay ? 0.0f : (float)InActiveSeriesIndex;
	const float TimelineDY = FMath::Max(1.0f, InViewport.GetLayout().TimelineDY);
	const float TopY = FMath::Max(1.0f, TimelineDY);
	const float SeriesHeight = FMath::Max(0.0f, GetSeriesHeight(InViewport.GetLayout())) * InTrack.GetRequestedTrackSizeScale();
	const float BottomY = TopY + SeriesHeight;
	const float OffsetY = SeriesHeight * ActiveSeriesIndex;

	OutTopY = OffsetY + TopY;
	OutBottomY = OffsetY + BottomY;
}

void FGameplayGraphSeries::UpdateAutoZoom(const FTimingTrackViewport& InViewport, const FGameplayGraphTrack& InTrack, int32 InActiveSeriesIndex)
{
	float TopY, BottomY;
	ComputePosition(InViewport, InTrack, InActiveSeriesIndex, TopY, BottomY);

	FGraphSeries::UpdateAutoZoom(TopY, BottomY, CurrentMin, CurrentMax, false);
}

FGameplayGraphTrack::FGameplayGraphTrack(const FGameplaySharedData& InGameplaySharedData, uint64 InObjectID, const FText& InName)
	: TGameplayTrackMixin<FGraphTrack>(InObjectID, InName)
	, GameplaySharedData(InGameplaySharedData)
	, RequestedTrackSizeScale(1.0f)
	, BorderY(0.0f)
	, NumActiveSeries(0)
	, Layout(EGameplayGraphLayout::Stack)
{
	EnabledOptions = //EGraphOptions::ShowDebugInfo |
					 //EGraphOptions::ShowPoints |
					 EGraphOptions::ShowPointsWithBorder |
					 EGraphOptions::ShowLines |
					 EGraphOptions::ShowPolygon |
					 //EGraphOptions::UseEventDuration |
					 //EGraphOptions::ShowBars |
					 //EGraphOptions::ShowBaseline |
					 //EGraphOptions::ShowVerticalAxisGrid |
					 //EGraphOptions::ShowHeader |
					 //ShowLabelsOption |
					 EGraphOptions::None;

	VisibleOptions &= ~(EGraphOptions::ShowBars | EGraphOptions::UseEventDuration);
}

void FGameplayGraphTrack::UpdateTrackHeight(const ITimingTrackUpdateContext& Context)
{
	const FTimingTrackViewport& Viewport = Context.GetViewport();

	int32 NumLanes = 0;
	if(Layout == EGameplayGraphLayout::Overlay) 
	{
		NumLanes = (NumActiveSeries > 0 ? 1 : 0);
	}
	else
	{
		NumLanes = NumActiveSeries;
	}
	
	const float CurrentTrackHeight = GetHeight();
	const float TimelineDY2 = 2.0f * Viewport.GetLayout().TimelineDY;
	const float DesiredTrackHeight = FMath::Max(0.0f, ((Viewport.GetLayout().ComputeTrackHeight(NumLanes) - TimelineDY2) * RequestedTrackSizeScale) + TimelineDY2);

	// Dont interpolate here, it is too expensive to continually invalidate the track right now
	SetHeight(DesiredTrackHeight);
}

void FGameplayGraphTrack::UpdateSeriesInternal(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport, int32 InActiveSeriesIndex)
{
	if(UpdateSeriesBounds(InSeries, InViewport))
	{
		InSeries.UpdateAutoZoom(InViewport, *this, InActiveSeriesIndex);
	}

	UpdateSeries(InSeries, InViewport);
}

void FGameplayGraphTrack::PreUpdate(const ITimingTrackUpdateContext& Context)
{
	AddAllSeries();

	FGraphTrack::PreUpdate(Context);

	// update border size
	BorderY = Context.GetViewport().GetLayout().TimelineDY;

	const bool bIsEntireGraphTrackDirty = IsDirty() || Context.GetViewport().IsHorizontalViewportDirty() || Context.GetViewport().IsDirty(ETimingTrackViewportDirtyFlags::VLayoutChanged);
	bool bNeedsUpdate = bIsEntireGraphTrackDirty;

	if (!bNeedsUpdate)
	{
		for (TSharedPtr<FGraphSeries>& Series : AllSeries)
		{
			if (Series->IsVisible() && Series->IsDirty())
			{
				// At least one series is dirty.
				bNeedsUpdate = true;
				break;
			}
		}
	}

	if (bNeedsUpdate)
	{
		ClearDirtyFlag();

		NumActiveSeries = 0;

		const FTimingTrackViewport& Viewport = Context.GetViewport();

		for (TSharedPtr<FGraphSeries>& Series : AllSeries)
		{
			FGameplayGraphSeries& GameplayGraphSeries = *StaticCastSharedPtr<FGameplayGraphSeries>(Series);

			if (Series->IsVisible() && (bIsEntireGraphTrackDirty || Series->IsDirty()))
			{
				// Clear the flag before updating, because the update itself may further need to set the series as dirty.
				Series->ClearDirtyFlag();

				UpdateSeriesInternal(GameplayGraphSeries, Viewport, NumActiveSeries);
			}

			if(GameplayGraphSeries.IsDrawn())
			{
				NumActiveSeries++;
			}
		}

		UpdateStats();
	}

	UpdateTrackHeight(Context);
}

void FGameplayGraphTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	FGraphTrack::Draw(Context);
	GetGameplayTrack().DrawHeaderForTimingTrack(Context, *this, false);

	const FTimingTrackViewport& Viewport = Context.GetViewport();
	if(IsAnyOptionEnabled(ShowLabelsOption) && !Viewport.GetLayout().bIsCompactMode)
	{
		FDrawContext& DrawContext = Context.GetDrawContext();
		const ITimingViewDrawHelper& DrawHelper = Context.GetHelper();

		int32 ActiveSeriesIndex = 0;
		for (const TSharedPtr<FGraphSeries>& Series : AllSeries)
		{
			FGameplayGraphSeries& GameplayGraphSeries = *StaticCastSharedPtr<FGameplayGraphSeries>(Series);

			if (GameplayGraphSeries.IsDrawn())
			{
				const FString NameString = Series->GetName().ToString();
				const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
				const float NameHeight = FontMeasureService->Measure(NameString, DrawHelper.GetEventFont()).Y;

				const float LocalPosY = FMath::RoundToFloat(GetPosY());
				float TopY, BottomY;
				GameplayGraphSeries.ComputePosition(Viewport, *this, ActiveSeriesIndex, TopY, BottomY);
				const float BaselineY = ((TopY + BottomY) * 0.5f) - (NameHeight * 0.5f);
				const float X = ((float)GetGameplayTrack().GetIndent() * GameplayTrackConstants::IndentSize) + 2.0f;
				DrawContext.DrawText(DrawHelper.GetHeaderBackgroundLayerId(), X + 1.0f, BaselineY + LocalPosY + 1.0f, NameString, DrawHelper.GetEventFont(), FLinearColor::Black);
				DrawContext.DrawText(DrawHelper.GetHeaderBackgroundLayerId() + 1, X, BaselineY + LocalPosY, NameString, DrawHelper.GetEventFont(), Series->GetColor());

				ActiveSeriesIndex++;
			}
		}
	}
}

void FGameplayGraphTrack::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("View", LOCTEXT("ViewHeader", "View"));
	{
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ViewProperties", "View Properties"),
			LOCTEXT("ViewProperties_Tooltip", "Open a window to view the properties of this track. You can scrub the timeline to see properties change in real-time."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ GameplaySharedData.OpenTrackVariantsTab(GetGameplayTrack()); })
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	FGraphTrack::BuildContextMenu(MenuBuilder);

	MenuBuilder.BeginSection("Layout", LOCTEXT("TrackLayoutMenuHeader", "Track Layout"));
	{
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("OverlayLayout", "Overlay"),
			LOCTEXT("OverlayLayout_Tooltip", "Draw series overlaid one on top of the other."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ Layout = EGameplayGraphLayout::Overlay; DisableOptions(ShowLabelsOption); SetDirtyFlag(); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this](){ return Layout == EGameplayGraphLayout::Overlay; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("StackLayout", "Stack"),
			LOCTEXT("StackLayout_Tooltip", "Draw series in a vertical stack."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ Layout = EGameplayGraphLayout::Stack; SetDirtyFlag(); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this](){ return Layout == EGameplayGraphLayout::Stack; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("DrawLabels", "Labels"),
			LOCTEXT("DrawLabels_Tooltip", "Draw series labels (stack view only)."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ ToggleOptions(ShowLabelsOption); }),
				FCanExecuteAction::CreateLambda([this](){ return Layout == EGameplayGraphLayout::Stack; }),
				FIsActionChecked::CreateLambda([this](){ return IsAnyOptionEnabled(ShowLabelsOption); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("TrackSize", LOCTEXT("TrackSizeMenuHeader", "Track Size"));
	{
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("NormalTrack", "Normal"),
			LOCTEXT("NormalTrack_Tooltip", "Draw this track at the standard size."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ RequestedTrackSizeScale = 1.0f; SetDirtyFlag(); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this](){ return RequestedTrackSizeScale == 1.0f; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("LargeTrack", "Large"),
			LOCTEXT("LargeTrack_Tooltip", "Make this track larger than normal."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ RequestedTrackSizeScale = 2.0f; SetDirtyFlag(); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this](){ return RequestedTrackSizeScale == 2.0f; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ExtraLargeTrack", "Extra Large"),
			LOCTEXT("ExtraLargeTrack_Tooltip", "Make this track much larger than normal."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ RequestedTrackSizeScale = 4.0f; SetDirtyFlag(); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this](){ return RequestedTrackSizeScale == 4.0f; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();
}

void FGameplayGraphTrack::GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const
{ 
	GetVariantsAtTime(InFrame.StartTime, OutVariants); 
}

#undef LOCTEXT_NAMESPACE
