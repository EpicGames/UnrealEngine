// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTrack.h"
#include "Insights/ViewModels/GraphTrack.h"
#include "Insights/ViewModels/GraphSeries.h"

class FTimingEventSearchParameters;
class FGameplayGraphTrack;
struct FVariantTreeNode;
namespace TraceServices { struct FFrame; }

/** The various layouts that we display series with */
enum class EGameplayGraphLayout : int32
{
	/** Draw series overlaid one on top of the other */
	Overlay,

	/** Draw series in a vertical stack */
	Stack
};

// Common functionality for gameplay-related graph series
class FGameplayGraphSeries : public FGraphSeries
{
public:
	// Custom overload for auto zoom
	void UpdateAutoZoom(const FTimingTrackViewport& InViewport, const FGameplayGraphTrack& InTrack, int32 InActiveSeriesIndex);

	// Check whether this series has any events with the current view
	bool IsDrawn() const { return IsVisible() && (Events.Num() > 0 || (LinePoints.Num() > 0 && LinePoints[0].Num() > 0) || Points.Num() > 0 || Boxes.Num() > 0); }

	// Helper for auto-zoom and drawing
	void ComputePosition(const FTimingTrackViewport& InViewport, const FGameplayGraphTrack& InTrack, int32 InActiveSeriesIndex, float& OutTopY, float& OutBottomY) const;

public:
	float CurrentMin;
	float CurrentMax;
};

// Track holding common functionality for gameplay-related graph tracks
class FGameplayGraphTrack : public TGameplayTrackMixin<FGraphTrack>
{
	INSIGHTS_DECLARE_RTTI(FGameplayGraphTrack, TGameplayTrackMixin<FGraphTrack>)

public:
	/** Whether to draw labels */
	static constexpr EGraphOptions ShowLabelsOption = EGraphOptions::FirstCustomOption;

public:
	FGameplayGraphTrack(const FGameplaySharedData& InGameplaySharedData, uint64 InObjectID, const FText& InName);

	virtual void Draw(const ITimingTrackDrawContext& Context) const override;
	virtual void PreUpdate(const ITimingTrackUpdateContext& Context) override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;

	// Get all variants at the specified time/frame
	virtual void GetVariantsAtTime(double InTime, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const {}
	virtual void GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const;

	// Get the requested track size scale
	float GetRequestedTrackSizeScale() const { return RequestedTrackSizeScale; }

	// Get the number of active series (i.e. visible)
	int32 GetNumActiveSeries() const { return NumActiveSeries; }

	// Get the layout we are displaying out graphs as
	EGameplayGraphLayout GetLayout() const { return Layout; }

protected:
	// Add all the series to this track
	virtual void AddAllSeries() {}

	// Update a single series' data (from PreUpdate())
	virtual void UpdateSeries(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport) {}

	// Update a single series' bounds
	// @return true from this if there are graph events in the current viewport
	virtual bool UpdateSeriesBounds(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport) { return false; }

private:
	// Helper for PreUpdate to update the track height
	void UpdateTrackHeight(const ITimingTrackUpdateContext& Context);

	// Helper to update and zoom series
	void UpdateSeriesInternal(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport, int32 InActiveSeriesIndex);

protected:
	const FGameplaySharedData& GameplaySharedData;

	/** The track size we want to be displayed at */
	float RequestedTrackSizeScale;

	/** The size of the border between tracks */
	float BorderY;

	/** The current number of active series */
	int32 NumActiveSeries;

	/** The current series layout */
	EGameplayGraphLayout Layout;
};
