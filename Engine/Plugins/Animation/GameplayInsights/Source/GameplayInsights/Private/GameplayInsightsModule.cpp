// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayInsightsModule.h"
#include "Features/IModularFeatures.h"
#include "Insights/ITimingViewExtender.h"
#include "Containers/Ticker.h"
#include "Modules/ModuleManager.h"
#include "Framework/Docking/LayoutExtender.h"
#include "SAnimGraphSchematicView.h"
#include "Insights/IUnrealInsightsModule.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformApplicationMisc.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "Trace/StoreClient.h"
#include "Stats/Stats.h"
#include "SMontageView.h"
#include "AnimCurvesTrack.h"
#include "InertializationsTrack.h"
#include "BlendWeightsTrack.h"
#include "MontagesTrack.h"
#include "NotifiesTrack.h"
#include "PoseWatchTrack.h"
#include "PropertiesTrack.h"
#include "ExternalMorphTrack.h"
#include "PropertyWatchManager.h"

#if WITH_EDITOR
#include "IAnimationBlueprintEditorModule.h"
#include "ToolMenus.h"
#include "Engine/Selection.h"
#include "SubobjectEditorMenuContext.h"
#include "GameplayInsightsStyle.h"
#include "SSubobjectInstanceEditor.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "ObjectPropertyTrace.h"
#endif

#if WITH_ENGINE
#include "Engine/Engine.h"
#endif

#define LOCTEXT_NAMESPACE "GameplayInsightsModule"

const FName GameplayInsightsTabs::DocumentTab("DocumentTab");

void FGameplayInsightsModule::StartupModule()
{
	LLM_SCOPE_BYNAME(TEXT("Insights/GameplayInsights"));

	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &GameplayTraceModule);
	IModularFeatures::Get().RegisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &GameplayTimingViewExtender);

	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("GameplayInsights"), 0.0f, [this](float DeltaTime)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FGameplayInsightsModule_TickVisualizers);

		GameplayTimingViewExtender.TickVisualizers(DeltaTime);
		return true;
	});

	IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	UnrealInsightsModule.OnMajorTabCreated().AddLambda([this](const FName& InMajorTabId, TSharedPtr<FTabManager> InTabManager)
	{
		if (InMajorTabId == FInsightsManagerTabs::TimingProfilerTabId)
		{
			WeakTimingProfilerTabManager = InTabManager;
		}
	});

#if WITH_EDITOR
	// register rewind debugger track creators
	static FAnimGraphSchematicTrackCreator AnimGraphSchematicTrackCreator;
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &AnimGraphSchematicTrackCreator);
	static RewindDebugger::FAnimationCurvesTrackCreator AnimationCurvesTrackCreator;
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &AnimationCurvesTrackCreator);
	static RewindDebugger::FInertializationsTrackCreator InertializationsTrackCreator;
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &InertializationsTrackCreator);
	static RewindDebugger::FBlendWeightsTrackCreator BlendWeightsTrackCreator;
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &BlendWeightsTrackCreator);
	static RewindDebugger::FMontagesTrackCreator MontagesTrackCreator;
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &MontagesTrackCreator);
	static RewindDebugger::FNotifiesTrackCreator NotifiesTrackCreator;
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &NotifiesTrackCreator);
	static RewindDebugger::FPoseWatchesTrackCreator PoseWatchesTrackCreator;
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &PoseWatchesTrackCreator);
	static RewindDebugger::FPropertiesTrackCreator PropertyTrackCreator;
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &PropertyTrackCreator);
	static RewindDebugger::FExternalMorphSetGroupTrackCreator ExternalMorphSetGroupTrackCreator;
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &ExternalMorphSetGroupTrackCreator);

	FPropertyWatchManager::Initialize();
	
	if (!IsRunningCommandlet())
	{
		IAnimationBlueprintEditorModule& AnimationBlueprintEditorModule = FModuleManager::LoadModuleChecked<IAnimationBlueprintEditorModule>("AnimationBlueprintEditor");
		CustomDebugObjectHandle = AnimationBlueprintEditorModule.OnGetCustomDebugObjects().AddLambda([this](const IAnimationBlueprintEditor& InAnimationBlueprintEditor, TArray<FCustomDebugObject>& OutDebugList)
		{
			GameplayTimingViewExtender.GetCustomDebugObjects(InAnimationBlueprintEditor, OutDebugList);
		});

		const float DPIScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(10.0f, 10.0f);

		TSharedRef<FTabManager::FLayout> MajorTabsLayout = FTabManager::NewLayout("GameplayInsightsMajorLayout_v1.0")
		->AddArea
		(
			FTabManager::NewArea(1280.f * DPIScaleFactor, 720.0f * DPIScaleFactor)
			->Split
			(
				FTabManager::NewStack()
				->AddTab(FInsightsManagerTabs::TimingProfilerTabId, ETabState::ClosedTab)
			)
		);
	}

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FGameplayInsightsModule::RegisterMenus));

#else
	FOnRegisterMajorTabExtensions& TimingProfilerExtension = UnrealInsightsModule.OnRegisterMajorTabExtension(FInsightsManagerTabs::TimingProfilerTabId);
	TimingProfilerExtension.AddRaw(this, &FGameplayInsightsModule::RegisterTimingProfilerLayoutExtensions);
#endif
}

void FGameplayInsightsModule::ShutdownModule()
{
	LLM_SCOPE_BYNAME(TEXT("Insights/GameplayInsights"));

#if WITH_EDITOR
	IAnimationBlueprintEditorModule* AnimationBlueprintEditorModule = FModuleManager::GetModulePtr<IAnimationBlueprintEditorModule>("AnimationBlueprintEditor");
	if(AnimationBlueprintEditorModule)
	{
		AnimationBlueprintEditorModule->OnGetCustomDebugObjects().Remove(CustomDebugObjectHandle);
	}
	FPropertyWatchManager::Shutdown();
#else
	IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	FOnRegisterMajorTabExtensions& TimingProfilerLayoutExtension = UnrealInsightsModule.OnRegisterMajorTabExtension(FInsightsManagerTabs::TimingProfilerTabId);
	TimingProfilerLayoutExtension.RemoveAll(this);
#endif

	FTSTicker::RemoveTicker(TickerHandle);

	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &GameplayTraceModule);
	IModularFeatures::Get().UnregisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &GameplayTimingViewExtender);
}

TSharedRef<SDockTab> FGameplayInsightsModule::SpawnTimingProfilerDocumentTab(const FTabManager::FSearchPreference& InSearchPreference)
{
	TSharedRef<SDockTab> NewTab = SNew(SDockTab);
	TSharedPtr<FTabManager> TimingProfilerTabManager = WeakTimingProfilerTabManager.Pin();
	if(TimingProfilerTabManager.IsValid())
	{
		TimingProfilerTabManager->InsertNewDocumentTab(GameplayInsightsTabs::DocumentTab, InSearchPreference, NewTab);
	}
	return NewTab;
}

void FGameplayInsightsModule::RegisterTimingProfilerLayoutExtensions(FInsightsMajorTabExtender& InOutExtender)
{
	InOutExtender.GetLayoutExtender().ExtendLayout(FTimingProfilerTabs::TimersID, ELayoutExtensionPosition::Before, FTabManager::FTab(GameplayInsightsTabs::DocumentTab, ETabState::ClosedTab));
}

#if WITH_EDITOR
void FGameplayInsightsModule::RegisterMenus()
{
	FToolMenuOwnerScoped OwnerScoped(this);

#if OBJECT_PROPERTY_TRACE_ENABLED
	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("Kismet.SubobjectEditorContextMenu");

		FToolMenuSection& Section = Menu->AddSection("GameplayInsights", LOCTEXT("GameplayInsights", "Gameplay Insights"));

		auto GetCheckState = [](const TSharedPtr<SSubobjectEditor>& InSubobjectEditor)
		{
			if (InSubobjectEditor->GetNumSelectedNodes() > 0 && FObjectPropertyTrace::IsEnabled())
			{
				TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = InSubobjectEditor->GetSelectedNodes();
				int32 TotalObjectCount = SelectedNodes.Num();
				int32 RegisteredObjectCount = 0;

				for (FSubobjectEditorTreeNodePtrType SubobjectNode : SelectedNodes)
				{
					const UObject* SelectedComponent = SubobjectNode->GetObject();
					if (FObjectPropertyTrace::IsObjectRegistered(SelectedComponent))
					{
						RegisteredObjectCount++;
					}
					else
					{
						break;
					}
				}

				if (RegisteredObjectCount == TotalObjectCount)
				{
					return ECheckBoxState::Checked;
				}
				else
				{
					return ECheckBoxState::Unchecked;
				}
			}

			return ECheckBoxState::Unchecked;
		};

		FToolUIAction Action;
		Action.ExecuteAction = FToolMenuExecuteAction::CreateLambda([&GetCheckState](const FToolMenuContext& InContext)
		{
			if (FObjectPropertyTrace::IsEnabled())
			{
				USubobjectEditorMenuContext* ContextObject = InContext.FindContext<USubobjectEditorMenuContext>();
				TSharedPtr<SSubobjectEditor> SubobjectEditor = ContextObject ? ContextObject->SubobjectEditor.Pin() : nullptr;

				if (SubobjectEditor.IsValid() && StaticCastSharedPtr<SSubobjectInstanceEditor>(SubobjectEditor))
				{					
					ECheckBoxState CheckState = GetCheckState(SubobjectEditor);

					for(FSubobjectEditorTreeNodePtrType Node : SubobjectEditor->GetSelectedNodes())
					{
						const UObject* SelectedComponent = Node->GetObject();
						if(CheckState == ECheckBoxState::Unchecked)
						{
							FObjectPropertyTrace::RegisterObject(SelectedComponent);
						}
						else
						{
							FObjectPropertyTrace::UnregisterObject(SelectedComponent);
						}
					}
				}
			}
		});
		Action.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([](const FToolMenuContext& InContext)
		{
			if (FObjectPropertyTrace::IsEnabled())
			{
				USubobjectEditorMenuContext* ContextObject = InContext.FindContext<USubobjectEditorMenuContext>();
				if (ContextObject && ContextObject->SubobjectEditor.IsValid() && StaticCastSharedPtr<SSubobjectInstanceEditor>(ContextObject->SubobjectEditor.Pin()))
				{
					TSharedPtr<SSubobjectEditor> SubobjectEditor = ContextObject->SubobjectEditor.Pin();
					if (SubobjectEditor.IsValid())
					{
						return SubobjectEditor->GetNumSelectedNodes() > 0;
					}
				}
			}

			return false;
		});
		Action.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([&GetCheckState](const FToolMenuContext& InContext)
		{
			USubobjectEditorMenuContext* ContextObject = InContext.FindContext<USubobjectEditorMenuContext>();
			if (ContextObject && StaticCastSharedPtr<SSubobjectInstanceEditor>(ContextObject->SubobjectEditor.Pin()))
			{
				TSharedPtr<SSubobjectEditor> SubobjectEditor = ContextObject->SubobjectEditor.Pin();

				if (SubobjectEditor.IsValid())
				{
					return GetCheckState(SubobjectEditor);
				}
			}

			return ECheckBoxState::Unchecked;
		});
		Action.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateLambda([](const FToolMenuContext& InContext)
		{
			if (FObjectPropertyTrace::IsEnabled())
			{
				USubobjectEditorMenuContext* ContextObject = InContext.FindContext<USubobjectEditorMenuContext>();
				if (ContextObject && ContextObject->SubobjectEditor.IsValid() && StaticCastSharedPtr<SSubobjectInstanceEditor>(ContextObject->SubobjectEditor.Pin()))
				{
					return true;
				}
			}

			return false;
		});

		FToolMenuEntry& Entry = Section.AddMenuEntry(
			"TraceComponentProperties",
			LOCTEXT("TraceComponentProperties", "Trace Component Properties"),
			LOCTEXT("TraceComponentPropertiesTooltip", "Trace the properties of this component to be viewed in Insights"),
			FSlateIcon(),
			Action,
			EUserInterfaceActionType::ToggleButton);
	}
#endif
}

void FGameplayInsightsModule::EnableObjectPropertyTrace(UObject* Object, bool bEnable)
{
#if OBJECT_PROPERTY_TRACE_ENABLED
	if (bEnable)
	{
		FObjectPropertyTrace::RegisterObject(Object);
	}
	else
	{
		FObjectPropertyTrace::UnregisterObject(Object);
	}
#endif
}

bool FGameplayInsightsModule::IsObjectPropertyTraceEnabled(UObject* Object)
{
#if OBJECT_PROPERTY_TRACE_ENABLED
	return FObjectPropertyTrace::IsObjectRegistered(Object);
#else
	return false;
#endif
}

#endif

IMPLEMENT_MODULE(FGameplayInsightsModule, GameplayInsights);

#undef LOCTEXT_NAMESPACE
