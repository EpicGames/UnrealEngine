// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "GameplayTraceModule.h"
#include "GameplayTimingViewExtender.h"
#include "Framework/Docking/TabManager.h"
#include "IGameplayInsightsModule.h"
#include "Trace/StoreService.h"

struct FInsightsMajorTabExtender;

namespace UE 
{
namespace Trace 
{
	class FStoreClient;
}
}

class FGameplayInsightsModule : public IGameplayInsightsModule
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

#if WITH_EDITOR
	virtual void EnableObjectPropertyTrace(UObject* Object, bool bEnable = true) override;
	virtual bool IsObjectPropertyTraceEnabled(UObject* Object) override;
	virtual void StartTrace() override {} ;
#endif

	// Spawn a document tab
	TSharedRef<SDockTab> SpawnTimingProfilerDocumentTab(const FTabManager::FSearchPreference& InSearchPreference);

protected:
	void RegisterTimingProfilerLayoutExtensions(FInsightsMajorTabExtender& InOutExtender);
#if WITH_EDITOR
	void RegisterMenus();
#endif

private:
	FGameplayTraceModule GameplayTraceModule;

	FGameplayTimingViewExtender GameplayTimingViewExtender;

	FTSTicker::FDelegateHandle TickerHandle;

#if WITH_EDITOR
	FDelegateHandle CustomDebugObjectHandle;
#endif

	TWeakPtr<FTabManager> WeakTimingProfilerTabManager;
};
