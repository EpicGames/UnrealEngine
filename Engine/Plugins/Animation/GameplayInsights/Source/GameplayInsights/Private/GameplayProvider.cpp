// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayProvider.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/PackageName.h"
#include "Model/AsyncEnumerateTask.h"
#include "Styling/SlateIconFinder.h"

FName FGameplayProvider::ProviderName("GameplayProvider");

#define LOCTEXT_NAMESPACE "GameplayProvider"

#if WITH_EDITOR
#include "Engine/Blueprint.h"
#include "Engine/BlueprintCore.h"

// This is copied from EditorUtilitySubsystem (where it is not public)
// Should probably be somewhere shared
static UClass* FindBlueprintClass(const FString& TargetNameRaw)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	if (AssetRegistry.IsLoadingAssets())
	{
		AssetRegistry.SearchAllAssets(true);
	}

	FString TargetName = TargetNameRaw;
	TargetName.RemoveFromEnd(TEXT("_C"), ESearchCase::CaseSensitive);

	FARFilter Filter;
	Filter.bRecursiveClasses = true;
	Filter.ClassPaths.Add(UBlueprintCore::StaticClass()->GetClassPathName());

	// We enumerate all assets to find any blueprints who inherit from native classes directly - or
	// from other blueprints.
	UClass* FoundClass = nullptr;
	AssetRegistry.EnumerateAssets(Filter, [&FoundClass, TargetName](const FAssetData& AssetData)
		{
			if ((AssetData.AssetName.ToString() == TargetName) || (AssetData.GetObjectPathString() == TargetName))
			{
				if (UBlueprint* BP = Cast<UBlueprint>(AssetData.GetAsset()))
				{
					FoundClass = BP->GeneratedClass;
					return false;
				}
			}

			return true;
		});

	return FoundClass;
}

// This is copied from EditorUtilitySubsystem (where it is not public)
// Should probably be somewhere shared
static UStruct* FindTypeByPathName(const FString& RawTargetPathName, const bool bCheckBlueprintClasses)
{
	FString TargetName = RawTargetPathName;

	// Check native classes and loaded assets first before resorting to the asset registry
	bool bIsValidClassName = true;
	if (TargetName.IsEmpty() || TargetName.Contains(TEXT(" ")))
	{
		bIsValidClassName = false;
	}
	else if (!FPackageName::IsShortPackageName(TargetName))
	{
		if (TargetName.Contains(TEXT(".")))
		{
			// Convert type'path' to just path (will return the full string if it doesn't have ' in it)
			TargetName = FPackageName::ExportTextPathToObjectPath(TargetName);

			FString PackageName;
			FString ObjectName;
			TargetName.Split(TEXT("."), &PackageName, &ObjectName);

			constexpr bool bIncludeReadOnlyRoots = true;
			FText Reason;
			if (!FPackageName::IsValidLongPackageName(PackageName, bIncludeReadOnlyRoots, &Reason))
			{
				bIsValidClassName = false;
			}
		}
		else
		{
			bIsValidClassName = false;
		}
	}

	UStruct* ResultClass = nullptr;
	if (bIsValidClassName)
	{
		ResultClass = FindObject<UStruct>(nullptr, *TargetName);
	}

	// If we still haven't found anything yet, try the asset registry for blueprints that match the requirements
	if (ResultClass == nullptr && bCheckBlueprintClasses)
	{
		ResultClass = FindBlueprintClass(TargetName);
	}

	return ResultClass;
}
#endif

namespace TraceServices
{
thread_local FProviderLock::FThreadLocalState GGameplayProviderLockState;
}

FGameplayProvider::FGameplayProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
	, TransformTimeline(InSession.GetLinearAllocator())
	, EndPlayEvent(nullptr)
	, PawnPossession(InSession.GetLinearAllocator())
	, ObjectLifetimes(InSession.GetLinearAllocator())
	, ObjectRecordingLifetimes(InSession.GetLinearAllocator())
{
}

bool FGameplayProvider::ReadObjectEventsTimeline(const RewindDebugger::FObjectId& InObjectId, TFunctionRef<void(const ObjectEventsTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToEventTimelines.Find(InObjectId);
	if (IndexPtr != nullptr)
	{
		if (*IndexPtr < static_cast<uint32>(EventTimelines.Num()))
		{
			Callback(*EventTimelines[*IndexPtr]);
			return true;
		}
	}

	return false;
}

bool FGameplayProvider::ReadObjectEvent(const RewindDebugger::FObjectId& InObjectId, uint64 InMessageId, TFunctionRef<void(const FObjectEventMessage&)> Callback) const
{
	ReadAccessCheck();

	return ReadObjectEventsTimeline(InObjectId, [&Callback, &InMessageId](const ObjectEventsTimeline& InTimeline)
		{
			if (InMessageId < InTimeline.GetEventCount())
			{
				Callback(InTimeline.GetEvent(InMessageId));
			}
		});
}

bool FGameplayProvider::ReadObjectPropertiesTimeline(const uint64 InObjectId, TFunctionRef<void(const ObjectPropertiesTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToPropertiesStorage.Find(InObjectId);
	if (IndexPtr != nullptr)
	{
		if (*IndexPtr < static_cast<uint32>(ObjectIdToPropertiesStorage.Num()))
		{
			Callback(*PropertiesStorage[*IndexPtr]->Timeline);
			return true;
		}
	}

	return false;
}

bool FGameplayProvider::ReadObjectPropertiesStorage(const uint64 InObjectId, const FObjectPropertiesMessage& InMessage, TFunctionRef<void(const TConstArrayView<FObjectPropertyValue>&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToPropertiesStorage.Find(InObjectId);
	if (IndexPtr != nullptr)
	{
		if (*IndexPtr < static_cast<uint32>(ObjectIdToPropertiesStorage.Num()))
		{
			const TSharedRef<FObjectPropertiesStorage> Storage = PropertiesStorage[*IndexPtr];
			Callback(MakeArrayView(Storage->Values.GetData() + InMessage.PropertyValueStartIndex, InMessage.PropertyValueEndIndex - InMessage.PropertyValueStartIndex));
			return true;
		}
	}

	return false;
}

void FGameplayProvider::EnumerateObjectPropertyValues(const uint64 InObjectId, const FObjectPropertiesMessage& InMessage, TFunctionRef<void(const FObjectPropertyValue&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToPropertiesStorage.Find(InObjectId);
	if (IndexPtr != nullptr)
	{
		if (*IndexPtr < static_cast<uint32>(ObjectIdToPropertiesStorage.Num()))
		{
			TSharedRef<FObjectPropertiesStorage> Storage = PropertiesStorage[*IndexPtr];
			for (int64 ValueIndex = InMessage.PropertyValueStartIndex; ValueIndex < InMessage.PropertyValueEndIndex; ++ValueIndex)
			{
				Callback(Storage->Values[ValueIndex]);
			}
		}
	}
}

void FGameplayProvider::EnumerateObjects(TFunctionRef<void(const FObjectInfo&)> Callback) const
{
	ReadAccessCheck();

	for (const FObjectInfo& ObjectInfo : ObjectInfos)
	{
		Callback(ObjectInfo);
	}
}

void FGameplayProvider::EnumerateObjects(double StartTime, double EndTime, TFunctionRef<void(const FObjectInfo&)> Callback) const
{
	ReadAccessCheck();

	ObjectLifetimes.EnumerateEvents(StartTime, EndTime,
		[this, Callback](double InStartTime, double InEndTime, uint32 InDepth, const FObjectExistsMessage& ExistsMessage)
		{
			checkSlow(ObjectIdToIndexMap.Contains(ExistsMessage.ObjectId));
			if (const int32* ObjectInfoIndex = ObjectIdToIndexMap.Find(ExistsMessage.ObjectId))
			{
				Callback(ObjectInfos[*ObjectInfoIndex]);
			}
			return TraceServices::EEventEnumerate::Continue;
		});
}

void FGameplayProvider::EnumerateWorlds(TFunctionRef<void(const FWorldInfo&)> Callback) const
{
	ReadAccessCheck();

	for (const FWorldInfo& WorldInfo : WorldInfos)
	{
		Callback(WorldInfo);
	}
}

const FObjectPropertyValue* FGameplayProvider::FindPropertyValueFromStorageIndex(const uint64 InObjectId, int64 InStorageIndex) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToPropertiesStorage.Find(InObjectId);
	if (IndexPtr != nullptr)
	{
		if (*IndexPtr < static_cast<uint32>(ObjectIdToPropertiesStorage.Num()))
		{
			const TSharedRef<FObjectPropertiesStorage> Storage = PropertiesStorage[*IndexPtr];

			if (Storage->Values.IsValidIndex(InStorageIndex))
			{
				return &Storage->Values[InStorageIndex];
			}
		}
	}

	return nullptr;
}

void FGameplayProvider::EnumerateSubobjects(const RewindDebugger::FObjectId& InObjectId, TFunctionRef<void(const RewindDebugger::FObjectId& SubObjectId)> Callback) const
{
	ReadAccessCheck();
	TArray<RewindDebugger::FObjectId, TInlineAllocator<32>> SubObjectIds;
	ObjectHierarchy.MultiFind(InObjectId, SubObjectIds);

	for (const RewindDebugger::FObjectId& SubObjectId : SubObjectIds)
	{
		Callback(SubObjectId);
	}
}


const FClassInfo* FGameplayProvider::FindClassInfo(uint64 InClassId) const
{
	ReadAccessCheck();

	const int32* ClassIndex = ClassIdToIndexMap.Find(InClassId);
	if (ClassIndex != nullptr)
	{
		return &ClassInfos[*ClassIndex];
	}

	return nullptr;
}

const UStruct* FGameplayProvider::FindType(uint64 InTypeId, bool bSearchBlueprints) const
{
#if WITH_EDITOR
	if (const FClassInfo* ClassInfo = FindClassInfo(InTypeId))
	{
		return FindTypeByPathName(ClassInfo->PathName, bSearchBlueprints);
	}
	return nullptr;
#else
	return nullptr;
#endif
}

const FClassInfo* FGameplayProvider::FindClassInfo(const TCHAR* InClassPath) const
{
	ReadAccessCheck();

	const int32* ClassIndex = ClassPathNameToIndexMap.Find(InClassPath);
	if (ClassIndex != nullptr)
	{
		return &ClassInfos[*ClassIndex];
	}

	return nullptr;
}

bool FGameplayProvider::IsSubClassOf(uint64 InSubClassId, uint64 InParentClassId) const
{
	ReadAccessCheck();

	uint64 ClassId = InSubClassId;
	while (true)
	{
		if (ClassId == InParentClassId)
		{
			return true;
		}
		else
		{
			const FClassInfo& ClassInfo = GetClassInfo(ClassId);
			if (ClassInfo.SuperId != 0)
			{
				ClassId = ClassInfo.SuperId;
			}
			else
			{
				return false;
			}
		}
	}
}

const FObjectInfo* FGameplayProvider::FindObjectInfo(uint64 InObjectId) const
{
	return FindObjectInfo(RewindDebugger::FObjectId{InObjectId});
}

const FObjectInfo* FGameplayProvider::FindObjectInfo(const RewindDebugger::FObjectId& InObjectId) const
{
	ReadAccessCheck();

	const int32* ObjectIndex = ObjectIdToIndexMap.Find(InObjectId);
	if (ObjectIndex != nullptr)
	{
		return &ObjectInfos[*ObjectIndex];
	}

	return nullptr;
}

const FWorldInfo* FGameplayProvider::FindWorldInfo(const uint64 InObjectId) const
{
	ReadAccessCheck();

	const int32* WorldIndex = WorldIdToIndexMap.Find(InObjectId);
	if (WorldIndex != nullptr)
	{
		return &WorldInfos[*WorldIndex];
	}

	return nullptr;
}

const FWorldInfo* FGameplayProvider::FindWorldInfoFromObject(const uint64 InObjectId) const
{
	if (const FClassInfo* WorldClass = FindClassInfo(TEXT("/Script/Engine.World")))
	{
		// Traverse outer chain until we find a world
		const FObjectInfo* ObjectInfo = FindObjectInfo(InObjectId);
		while (ObjectInfo != nullptr)
		{
			if (ObjectInfo->ClassId == WorldClass->Id)
			{
				return FindWorldInfo(ObjectInfo->GetUObjectId());
			}

			ObjectInfo = FindObjectInfo(ObjectInfo->GetOuterId());
		}
	}

	return nullptr;
}

bool FGameplayProvider::IsWorld(const uint64 InObjectId) const
{
	if (const FClassInfo* WorldClass = FindClassInfo(TEXT("/Script/Engine.World")))
	{
		if (const FObjectInfo* ObjectInfo = FindObjectInfo(InObjectId))
		{
			return ObjectInfo->ClassId == WorldClass->Id;
		}
	}

	return false;
}

const FClassInfo& FGameplayProvider::GetClassInfo(uint64 InClassId) const
{
	if (const FClassInfo* ClassInfo = FindClassInfo(InClassId))
	{
		return *ClassInfo;
	}

	static FText UnknownText(LOCTEXT("Unknown", "Unknown"));
	static FClassInfo DefaultClassInfo = { 0, 0, *UnknownText.ToString(), *UnknownText.ToString() };
	return DefaultClassInfo;
}

const FClassInfo& FGameplayProvider::GetClassInfoFromObject(const uint64 InObjectId) const
{
	if (const FObjectInfo* ObjectInfo = FindObjectInfo(InObjectId))
	{
		if (const FClassInfo* ClassInfo = FindClassInfo(ObjectInfo->ClassId))
		{
			return *ClassInfo;
		}
	}

	static FText UnknownText(LOCTEXT("Unknown", "Unknown"));
	static FClassInfo DefaultClassInfo = { 0, 0, *UnknownText.ToString(), *UnknownText.ToString() };
	return DefaultClassInfo;
}

const FObjectInfo& FGameplayProvider::GetObjectInfo(uint64 InObjectId) const
{
	return GetObjectInfo(RewindDebugger::FObjectId{InObjectId});
}

const FObjectInfo& FGameplayProvider::GetObjectInfo(const RewindDebugger::FObjectId& InObjectId) const
{
	if (const FObjectInfo* ObjectInfo = FindObjectInfo(InObjectId))
	{
		return *ObjectInfo;
	}

	static FText UnknownText(LOCTEXT("Unknown", "Unknown"));
	static FObjectInfo DefaultObjectInfo = {
		RewindDebugger::FObjectId()
		, RewindDebugger::FObjectId()
		, /*ClassId*/0
		, *UnknownText.ToString()
		, *UnknownText.ToString()
		, EObjectInfoFlags::None 
	};
	return DefaultObjectInfo;
}

const TCHAR* FGameplayProvider::GetPropertyName(uint32 InPropertyStringId) const
{
	ReadAccessCheck();
	if (const TCHAR* const* FoundString = PropertyStrings.Find(InPropertyStringId))
	{
		return *FoundString;
	}

	static FText UnknownText(LOCTEXT("Unknown", "Unknown"));
	return *UnknownText.ToString();
}

void FGameplayProvider::AppendClass(uint64 InClassId, uint64 InSuperId, const TCHAR* InClassName, const TCHAR* InClassPathName)
{
	EditAccessCheck();

	bHasAnyData = true;

	if (ClassIdToIndexMap.Find(InClassId) == nullptr)
	{
		const TCHAR* NewClassName = Session.StoreString(InClassName);
		const TCHAR* NewClassPathName = Session.StoreString(InClassPathName);

		FClassInfo NewClassInfo;
		NewClassInfo.Id = InClassId;
		NewClassInfo.SuperId = InSuperId;
		NewClassInfo.Name = NewClassName;
		NewClassInfo.PathName = NewClassPathName;

		int32 NewClassInfoIndex = ClassInfos.Add(NewClassInfo);
		ClassIdToIndexMap.Add(InClassId, NewClassInfoIndex);
		ClassPathNameToIndexMap.Add(NewClassPathName, NewClassInfoIndex);
	}
}

void FGameplayProvider::AppendObject(const RewindDebugger::FObjectId& InObjectId, const RewindDebugger::FObjectId& InOuterId, uint64 InClassId, const TCHAR* InObjectName, const TCHAR* InObjectPathName, EObjectInfoFlags Flags)
{
	EditAccessCheck();

	bHasAnyData = true;

	if (ObjectIdToIndexMap.Find(InObjectId) == nullptr)
	{
		const TCHAR* NewObjectName = Session.StoreString(InObjectName);
		const TCHAR* NewObjectPathName = Session.StoreString(InObjectPathName);

		ObjectIdToIndexMap.Add(InObjectId, ObjectInfos.Add({ InObjectId, InOuterId, InClassId, NewObjectName, NewObjectPathName, Flags }));

		ObjectHierarchy.AddUnique(InOuterId, InObjectId);
	}
}

void FGameplayProvider::AppendObjectTransform(uint64 InObjectId, double InProfileTime, double InRecordingTime, const FTransform& Transform)
{
	EditAccessCheck();

	bHasAnyData = true;

	FObjectTransformMessage Message;
	Message.ObjectId = InObjectId;
	Message.RecordingTime = InRecordingTime;
	Message.Transform = Transform;

	TransformTimeline.AppendEvent(InProfileTime, Message);
}

void FGameplayProvider::AppendObjectLifetimeBegin(const RewindDebugger::FObjectId& InObjectId, double InProfileTime, double InRecordingTime)
{
	EditAccessCheck();
	bHasAnyData = true;

	if (InObjectId.IsSet())
	{
		FObjectExistsMessage Message;
		Message.ObjectId = InObjectId;
		ActiveObjectLifetimes.Add(InObjectId, ObjectLifetimes.AppendBeginEvent(InProfileTime, Message));
		ActiveObjectRecordingLifetimes.Add(InObjectId, ObjectRecordingLifetimes.AppendBeginEvent(InRecordingTime, Message));
	}
}

void FGameplayProvider::AppendObjectLifetimeEnd(const RewindDebugger::FObjectId& InObjectId, double InProfileTime, double InRecordingTime)
{
	EditAccessCheck();
	bHasAnyData = true;

	if (const uint64* FoundIndex = ActiveObjectLifetimes.Find(InObjectId))
	{
		ObjectLifetimes.EndEvent(*FoundIndex, InProfileTime);
		ActiveObjectLifetimes.Remove(InObjectId);
	}

	if (const uint64* FoundIndex = ActiveObjectRecordingLifetimes.Find(InObjectId))
	{
		if (ObjectRecordingLifetimes.GetEventEndTime(*FoundIndex) > InRecordingTime)
		{
			ObjectRecordingLifetimes.EndEvent(*FoundIndex, InRecordingTime);
		}
		// do not remove from ActiveObjectRecordingLifetimes - lifetimes can be queried by Object Id in GetObjectRecordingLifetime
	}

	if (const int32* ObjectInfoIndex = ObjectIdToIndexMap.Find(InObjectId))
	{
		OnObjectEndPlayDelegate.Broadcast(InObjectId.GetMainId(), InProfileTime, ObjectInfos[*ObjectInfoIndex]);
	}
}

void FGameplayProvider::AppendObjectEvent(const RewindDebugger::FObjectId& InObjectId, double InTime, const TCHAR* InEventName)
{
	EditAccessCheck();

	bHasAnyData = true;

	// Important events need some extra routing
	if (EndPlayEvent == nullptr)
	{
		EndPlayEvent = Session.StoreString(TEXT("EndPlay"));
	}

	TSharedPtr<TraceServices::TPointTimeline<FObjectEventMessage>> Timeline;
	uint32* IndexPtr = ObjectIdToEventTimelines.Find(InObjectId);
	if (IndexPtr != nullptr)
	{
		Timeline = EventTimelines[*IndexPtr];
	}
	else
	{
		Timeline = MakeShared<TraceServices::TPointTimeline<FObjectEventMessage>>(Session.GetLinearAllocator());
		ObjectIdToEventTimelines.Add(InObjectId, EventTimelines.Num());
		EventTimelines.Add(Timeline.ToSharedRef());
	}

	FObjectEventMessage Message;
	Message.ObjectId = InObjectId;
	Message.Name = Session.StoreString(InEventName);

	// For now, we don't have a need to notify EndPlay on child element so let's avoid it
	if (Message.Name == EndPlayEvent)
	{
		if (const int32* ObjectInfoIndex = ObjectIdToIndexMap.Find(InObjectId))
		{
			OnObjectEndPlayDelegate.Broadcast(InObjectId.GetMainId(), InTime, ObjectInfos[*ObjectInfoIndex]);
		}
	}

	Timeline->AppendEvent(InTime, Message);

	{
		TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
		Session.UpdateDurationSeconds(InTime);
	}
}

void FGameplayProvider::AppendPawnPossess(uint64 InControllerId, uint64 InPawnId, double InTime)
{
	EditAccessCheck();
	bHasAnyData = true;

	// End any active controller attachment interval for this controller
	if (const uint64* FoundIndex = ActivePawnPossession.Find(InControllerId))
	{
		PawnPossession.EndEvent(*FoundIndex, InTime);
		ActivePawnPossession.Remove(InControllerId);
	}

	if (InPawnId)
	{
		FPawnPossessMessage Message;
		Message.ControllerId = InControllerId;
		Message.PawnId = InPawnId;
		ActivePawnPossession.Add(InControllerId, PawnPossession.AppendBeginEvent(InTime, Message));
	}
}

uint64 FGameplayProvider::FindPossessingController(uint64 PawnId, double Time) const
{
	ReadAccessCheck();
	uint64 ControllerId = 0;
	PawnPossession.EnumerateEvents(Time, Time, [&ControllerId, PawnId](double StartTime, double EndTime, const FPawnPossessMessage Message)
		{
			if (Message.PawnId == PawnId)
			{
				ControllerId = Message.ControllerId;
				return TraceServices::EEventEnumerate::Stop;
			}
			return TraceServices::EEventEnumerate::Continue;
		});
	return ControllerId;
}

bool FGameplayProvider::GetObjectTransform(uint64 InObjectId, double InStartTime, double InEndTime, FTransform& OutTransform) const
{
	ReadAccessCheck();
	bool bSuccess = false;
	TransformTimeline.EnumerateEvents(InStartTime, InEndTime, [&OutTransform, InObjectId, &bSuccess](bool bStart, double Time, const FObjectTransformMessage& TransformMessage)
		{
			if (TransformMessage.ObjectId == InObjectId)
			{
				bSuccess = true;
				OutTransform = TransformMessage.Transform;
			}

			return TraceServices::EEventEnumerate::Continue;
		});
	return bSuccess;
}

TRange<double> FGameplayProvider::GetObjectTraceLifetime(const RewindDebugger::FObjectId& InObjectId) const
{
	ReadAccessCheck();

	if (const uint64* FoundIndex = ActiveObjectLifetimes.Find(InObjectId))
	{
		return TRange<double>(ObjectLifetimes.GetEventStartTime(*FoundIndex), ObjectLifetimes.GetEventEndTime(*FoundIndex));
	}
	else
	{
		return TRange<double>(0, 0);
	}
}

TRange<double> FGameplayProvider::GetObjectRecordingLifetime(const RewindDebugger::FObjectId& InObjectId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGameplayProvider::GetObjectRecordingLifetime);
	ReadAccessCheck();

	if (const uint64 *FoundIndex = ActiveObjectRecordingLifetimes.Find(InObjectId))
	{
		return TRange<double>(ObjectRecordingLifetimes.GetEventStartTime(*FoundIndex), ObjectRecordingLifetimes.GetEventEndTime(*FoundIndex));
	}
	else
	{
		return TRange<double>(0, 0);
	}
}

void FGameplayProvider::ReadViewTimeline(TFunctionRef<void(const IGameplayProvider::ViewTimeline&)> Callback) const
{
	ReadAccessCheck();

	if (ViewTimeline.IsValid())
	{
		Callback(*ViewTimeline);
	}
}

double FGameplayProvider::GetRecordingDuration() const
{
	ReadAccessCheck();
	return RecordingDuration;
}

FSlateIcon FGameplayProvider::FindIconForClass(uint64 ClassId) const
{
	FSlateIcon Icon;
#if WITH_EDITOR
	// traverse class hierarchy until we find a class that exists, and return the icon for that class
	// loaded trace data may contain classes from different projects than this editor is compiled against or loading
	// , but we will still find a Character icon for example for subclasses of Character
	while (ClassId)
	{
		// don't search blueprints because it's very slow and icons are associated with native classes
		if (const UStruct* FoundType = FindType(ClassId, false))
		{
			Icon = FSlateIconFinder::FindIconForClass(FoundType);
			break;
		}

		const FClassInfo& ClassInfo = GetClassInfo(ClassId);
		ClassId = ClassInfo.SuperId;
	}

	if (!Icon.IsSet())
	{
		Icon = FSlateIconFinder::FindIconForClass(UObject::StaticClass());
	}
#endif
	return Icon;
}

void FGameplayProvider::AppendView(uint64 InPlayerId, double InTime, const FVector& InPosition, const FRotator& InRotation, float InFov, float InAspectRatio)
{
	EditAccessCheck();

	if (!ViewTimeline.IsValid())
	{
		ViewTimeline = MakeShared<TraceServices::TPointTimeline<FViewMessage>>(Session.GetLinearAllocator());
	}

	bHasAnyData = true;

	FViewMessage Message;
	Message.PlayerId = InPlayerId;
	Message.Position = InPosition;
	Message.Rotation = InRotation;
	Message.Fov = InFov;
	Message.AspectRatio = InAspectRatio;

	ViewTimeline->AppendEvent(InTime, Message);

	{
		TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
		Session.UpdateDurationSeconds(InTime);
	}
}

void FGameplayProvider::AppendWorld(uint64 InWorldId, int32 InPIEInstanceId, uint8 InType, uint8 InNetMode, bool bInIsSimulating)
{
	EditAccessCheck();

	bHasAnyData = true;

	if (WorldIdToIndexMap.Find(InWorldId) == nullptr)
	{
		FWorldInfo NewWorldInfo;
		NewWorldInfo.Id = InWorldId;
		NewWorldInfo.PIEInstanceId = InPIEInstanceId;
		NewWorldInfo.Type = static_cast<FWorldInfo::EType>(InType);
		NewWorldInfo.NetMode = static_cast<FWorldInfo::ENetMode>(InNetMode);
		NewWorldInfo.bIsSimulating = bInIsSimulating;

		int32 NewWorldInfoIndex = WorldInfos.Add(NewWorldInfo);
		WorldIdToIndexMap.Add(InWorldId, NewWorldInfoIndex);
	}
}

void FGameplayProvider::AppendRecordingInfo(uint64 InWorldId, double InProfileTime, uint32 InRecordingIndex, uint32 InFrameIndex, double InElapsedTime)
{
	EditAccessCheck();

	if (!TargetType.IsSet())
	{
		TargetType = EBuildTargetType::Unknown;

		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);
		Session.EnumerateMetadata(
			[&TargetType = TargetType](const TraceServices::FTraceSessionMetadata& Metadata)
			{
				if (Metadata.Name == FName("TargetType"))
				{
					TargetType = static_cast<EBuildTargetType>(Metadata.Int64Value);
				}
			});
	}

	if (const int32* WorldIndex = WorldIdToIndexMap.Find(InWorldId))
	{
		if (WorldInfos.IsValidIndex(*WorldIndex))
		{
			// We should put each world into its own timeline in the future, to support frame stepping in sync with server frames
			if (WorldInfos[*WorldIndex].NetMode == FWorldInfo::ENetMode::Standalone
				|| WorldInfos[*WorldIndex].NetMode == FWorldInfo::ENetMode::Client
				|| (TargetType == EBuildTargetType::Server
					&& WorldInfos[*WorldIndex].NetMode == FWorldInfo::ENetMode::DedicatedServer))
			{
				FRecordingInfoMessage NewRecordingInfo;
				NewRecordingInfo.WorldId = InWorldId;
				NewRecordingInfo.ProfileTime = InProfileTime;
				NewRecordingInfo.RecordingIndex = InRecordingIndex;
				NewRecordingInfo.FrameIndex = InFrameIndex;
				NewRecordingInfo.ElapsedTime = InElapsedTime;
				RecordingDuration = FMath::Max(RecordingDuration, InElapsedTime);

				if (TSharedRef<TraceServices::TPointTimeline<FRecordingInfoMessage>>* ExistingRecording = Recordings.Find(InRecordingIndex))
				{
					(*ExistingRecording)->AppendEvent(InProfileTime, NewRecordingInfo);
				}
				else
				{
					TSharedPtr<TraceServices::TPointTimeline<FRecordingInfoMessage>> NewRecording = MakeShared<TraceServices::TPointTimeline<FRecordingInfoMessage>>(Session.GetLinearAllocator());
					NewRecording->AppendEvent(InProfileTime, NewRecordingInfo);
					Recordings.Add(InRecordingIndex, NewRecording.ToSharedRef());
				}
			}
		}
	}
}


const FGameplayProvider::RecordingInfoTimeline* FGameplayProvider::GetRecordingInfo(uint32 RecordingId) const
{
	ReadAccessCheck();

	if (const TSharedRef<TraceServices::TPointTimeline<FRecordingInfoMessage>>* Recording = Recordings.Find(RecordingId))
	{
		return &(*Recording).Get();
	}

	return nullptr;
}

void FGameplayProvider::AppendClassPropertyStringId(uint32 InStringId, const FStringView& InString)
{
	EditAccessCheck();

	bHasAnyData = true;

	const TCHAR* StoredString = Session.StoreString(InString);

	PropertyStrings.Add(InStringId, StoredString);
}

void FGameplayProvider::AppendPropertiesStart(const uint64 InObjectId, double InTime, uint64 InEventId, double InRecordingTime)
{
	EditAccessCheck();

	bHasAnyData = true;
	bHasObjectProperties = true;

	TSharedPtr<FObjectPropertiesStorage> Storage;
	uint32* IndexPtr = ObjectIdToPropertiesStorage.Find(InObjectId);
	if (IndexPtr != nullptr)
	{
		Storage = PropertiesStorage[*IndexPtr];
	}
	else
	{
		Storage = MakeShared<FObjectPropertiesStorage>();
		Storage->Timeline = MakeShared<TraceServices::TIntervalTimeline<FObjectPropertiesMessage>>(Session.GetLinearAllocator());
		ObjectIdToPropertiesStorage.Add(InObjectId, PropertiesStorage.Num());
		PropertiesStorage.Add(Storage.ToSharedRef());
	}

	Storage->OpenEventId = InEventId;
	Storage->OpenStartTime = InTime;

	FObjectPropertiesMessage& Message = Storage->OpenEvent;
	Message.PropertyValueStartIndex = Storage->Values.Num();
	Message.PropertyValueEndIndex = Storage->Values.Num();
	Message.ProfileTime = InTime;
	Message.ElapsedTime = InRecordingTime;
}

void FGameplayProvider::AppendPropertiesEnd(const uint64 InObjectId, double InTime)
{
	EditAccessCheck();

	bHasAnyData = true;
	bHasObjectProperties = true;

	TSharedPtr<FObjectPropertiesStorage> Storage;
	uint32* IndexPtr = ObjectIdToPropertiesStorage.Find(InObjectId);
	if (IndexPtr != nullptr)
	{
		Storage = PropertiesStorage[*IndexPtr];
	}
	else
	{
		Storage = MakeShared<FObjectPropertiesStorage>();
		Storage->Timeline = MakeShared<TraceServices::TIntervalTimeline<FObjectPropertiesMessage>>(Session.GetLinearAllocator());
		ObjectIdToPropertiesStorage.Add(InObjectId, PropertiesStorage.Num());
		PropertiesStorage.Add(Storage.ToSharedRef());
	}

	if (Storage->OpenEventId != 0)
	{
		uint64 EventIndex = Storage->Timeline->AppendBeginEvent(Storage->OpenStartTime, Storage->OpenEvent);
		Storage->Timeline->EndEvent(EventIndex, InTime);

		Storage->OpenEventId = 0;
	}
}

void FGameplayProvider::AppendPropertyValue(const uint64 InObjectId, double InTime, uint64 InEventId, int32 InParentId, uint32 InTypeStringId, uint32 InNameId, uint32 InParentNameId, const FStringView& InValue)
{
	EditAccessCheck();

	bHasAnyData = true;
	bHasObjectProperties = true;

	TSharedPtr<FObjectPropertiesStorage> Storage;
	if (const uint32* IndexPtr = ObjectIdToPropertiesStorage.Find(InObjectId))
	{
		Storage = PropertiesStorage[*IndexPtr];
	}
	else
	{
		Storage = MakeShared<FObjectPropertiesStorage>();
		Storage->Timeline = MakeShared<TraceServices::TIntervalTimeline<FObjectPropertiesMessage>>(Session.GetLinearAllocator());
		ObjectIdToPropertiesStorage.Add(InObjectId, PropertiesStorage.Num());
		PropertiesStorage.Add(Storage.ToSharedRef());
	}

	if (Storage->OpenEventId == InEventId)
	{
		FObjectPropertyValue& Message = Storage->Values.AddDefaulted_GetRef();

		Message.Value = Session.StoreString(InValue);
		Message.ValueAsFloat = FCString::Atof(Message.Value);
		Message.ParentId = InParentId;
		Message.TypeStringId = InTypeStringId;
		Message.NameId = InNameId;
		Message.ParentNameId = InParentNameId;

		Storage->OpenEvent.PropertyValueEndIndex = Storage->Values.Num();
	}
}

bool FGameplayProvider::HasAnyData() const
{
	ReadAccessCheck();

	return bHasAnyData;
}

bool FGameplayProvider::HasObjectProperties() const
{
	ReadAccessCheck();

	return bHasObjectProperties;
}

#undef LOCTEXT_NAMESPACE
