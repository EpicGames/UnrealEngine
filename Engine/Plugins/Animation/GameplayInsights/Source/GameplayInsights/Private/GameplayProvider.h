// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IGameplayProvider.h"
#include "Common/ProviderLock.h"
#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "Math/TransformVectorized.h"
#include "Model/IntervalTimeline.h"
#include "Model/PointTimeline.h"
#include "Templates/SharedPointer.h"

namespace RewindDebugger
{
struct FObjectId;
}

namespace TraceServices
{
class IAnalysisSession;
extern thread_local FProviderLock::FThreadLocalState GGameplayProviderLockState;
}

class FGameplayProvider : public IGameplayProvider
{
public:
	static FName ProviderName;

	explicit FGameplayProvider(TraceServices::IAnalysisSession& InSession);

	//~ IProvider
	virtual void BeginRead() const override
	{
		Lock.BeginRead(TraceServices::GGameplayProviderLockState);
	}

	virtual void EndRead() const override
	{
		Lock.EndRead(TraceServices::GGameplayProviderLockState);
	}

	virtual void ReadAccessCheck() const override
	{
		Lock.ReadAccessCheck(TraceServices::GGameplayProviderLockState);
	}

	//~ IEditableProvider
	virtual void BeginEdit() const override
	{
		Lock.BeginWrite(TraceServices::GGameplayProviderLockState);
	}

	virtual void EndEdit() const override
	{
		Lock.EndWrite(TraceServices::GGameplayProviderLockState);
	}

	virtual void EditAccessCheck() const override
	{
		Lock.WriteAccessCheck(TraceServices::GGameplayProviderLockState);
	}

	//~ IGameplayProvider
	virtual bool ReadObjectEventsTimeline(const RewindDebugger::FObjectId& InObjectId, TFunctionRef<void(const ObjectEventsTimeline&)> Callback) const override;
	virtual bool ReadObjectEvent(const RewindDebugger::FObjectId& InObjectId, uint64 InMessageId, TFunctionRef<void(const FObjectEventMessage&)> Callback) const override;
	virtual bool ReadObjectPropertiesTimeline(uint64 InObjectId, TFunctionRef<void(const ObjectPropertiesTimeline&)> Callback) const override;
	virtual bool ReadObjectPropertiesStorage(uint64 InObjectId, const FObjectPropertiesMessage& InMessage, TFunctionRef<void(const TConstArrayView<FObjectPropertyValue>&)> Callback) const override;
	virtual void EnumerateObjectPropertyValues(uint64 InObjectId, const FObjectPropertiesMessage& InMessage, TFunctionRef<void(const FObjectPropertyValue&)> Callback) const override;
	virtual void EnumerateObjects(TFunctionRef<void(const FObjectInfo&)> Callback) const override;
	virtual void EnumerateObjects(double StartTime, double EndTime, TFunctionRef<void(const FObjectInfo&)> Callback) const override;
	virtual void EnumerateWorlds(TFunctionRef<void(const FWorldInfo&)> Callback) const override;
	virtual void EnumerateSubobjects(const RewindDebugger::FObjectId& InObjectId, TFunctionRef<void(const RewindDebugger::FObjectId& SubObjectId)> Callback) const override;
	virtual const FObjectPropertyValue* FindPropertyValueFromStorageIndex(uint64 InObjectId, int64 InStorageIndex) const override;
	virtual const FClassInfo* FindClassInfo(uint64 InClassId) const override;
	virtual const UStruct* FindType(uint64 InTypeId, bool bSearchBlueprints = true) const override;
	virtual const FClassInfo* FindClassInfo(const TCHAR* InClassPath) const override;
	virtual bool IsSubClassOf(uint64 InSubClassId, uint64 InParentClassId) const override;
	virtual const FObjectInfo* FindObjectInfo(const uint64 InObjectId) const override;
	virtual const FObjectInfo* FindObjectInfo(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual const FWorldInfo* FindWorldInfo(uint64 InObjectId) const override;
	virtual const FWorldInfo* FindWorldInfoFromObject(uint64 InObjectId) const override;
	virtual bool IsWorld(uint64 InObjectId) const override;
	virtual const FClassInfo& GetClassInfo(uint64 InClassId) const override;
	virtual const FClassInfo& GetClassInfoFromObject(uint64 InObjectId) const override;
	virtual const FObjectInfo& GetObjectInfo(uint64 InObjectId) const override;
	virtual const FObjectInfo& GetObjectInfo(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual FOnObjectEndPlay& OnObjectEndPlay() override
	{
		return OnObjectEndPlayDelegate;
	}
	virtual const TCHAR* GetPropertyName(uint32 InPropertyStringId) const override;
	virtual const RecordingInfoTimeline* GetRecordingInfo(uint32 RecordingId) const override; 
	virtual void ReadViewTimeline(TFunctionRef<void(const ViewTimeline&)> Callback) const override;
	virtual double GetRecordingDuration() const override;
	virtual FSlateIcon FindIconForClass(uint64 ClassId) const override;

	/** Add a class message */
	void AppendClass(uint64 InClassId, uint64 InSuperId, const TCHAR* InClassName, const TCHAR* InClassPathName);

	/** Add an object message */
	void AppendObject(const RewindDebugger::FObjectId& InObjectId, const RewindDebugger::FObjectId& InOuterId, uint64 InClassId, const TCHAR* InObjectName, const TCHAR* InObjectPathName, EObjectInfoFlags Flags);

	/** Add an object transform */
	void AppendObjectTransform(uint64 InObjectId, double InProfileTime, double InRecordingTime, const FTransform& Transform);

	/** Add an object create message */
	void AppendObjectLifetimeBegin(const RewindDebugger::FObjectId& InObjectId, double InProfileTime, double InRecordingTime);

	/** Add an object destroy message */
	void AppendObjectLifetimeEnd(const RewindDebugger::FObjectId& InObjectId, double InProfileTime, double InRecordingTime);	

	/** Add an object event message */
	void AppendObjectEvent(const RewindDebugger::FObjectId& InObjectId, double InTime, const TCHAR* InEvent);

	/** Add a Controller Attach message */
	void AppendPawnPossess(uint64 InControllerId, uint64 InPawnId, double InTime);

	/** Add a view message */
	void AppendView(uint64 InPlayerId, double InTime, const FVector& InPosition, const FRotator& InRotation, float InFov, float InAspectRatio);

	/** Add a world message */
	void AppendWorld(uint64 InWorldId, int32 InPIEInstanceId, uint8 InType, uint8 InNetMode, bool bInIsSimulating);

	/** Add a recording info message */
	void AppendRecordingInfo(uint64 InWorldId, double InProfileTime, uint32 InRecordingIndex, uint32 InFrameIndex, double InElapsedTime);

	/** Add a class property string ID message */
	void AppendClassPropertyStringId(uint32 InStringId, const FStringView& InString);

	/** Add a 'properties start' message */
	void AppendPropertiesStart(uint64 InObjectId, double InTime, uint64 InEventId, double InRecordingTime);

	/** Add a 'properties end' message */
	void AppendPropertiesEnd(uint64 InObjectId, double InTime);

	/** Add a property value message */
	void AppendPropertyValue(uint64 InObjectId, double InTime, uint64 InEventId, int32 InParentId, uint32 InTypeStringId, uint32 InNameId, uint32 InParentNameId, const FStringView& InValue);

	/** Check whether we have any data */
	bool HasAnyData() const;

	/** Check whether we have any object property data */
	bool HasObjectProperties() const;

	/** Search PawnPossession timeline to find the Controller Object Id for a Pawn (or 0 if none)*/
	virtual uint64 FindPossessingController(uint64 Pawn, double Time) const override;

	/** find the Transform of an object within a frame that Starts at InStartTime, and ends at InEndTime */
	virtual bool GetObjectTransform(uint64 InObjectId, double InStartTime, double InEndTime, FTransform& OutTransform) const override;

	/** find the Trace time range for which an object existed */
	virtual TRange<double> GetObjectTraceLifetime(const RewindDebugger::FObjectId& InObjectId) const override;

	/** find the Recording time range for which an object existed */
	virtual TRange<double> GetObjectRecordingLifetime(const RewindDebugger::FObjectId& InObjectId) const override;

private:
	mutable TraceServices::FProviderLock Lock;
	TraceServices::IAnalysisSession& Session;

	/** All class info, grow only for stable indices */
	TArray<FClassInfo> ClassInfos;

	/** All object info, grow only for stable indices */
	TArray<FObjectInfo> ObjectInfos;

	/** All world info, grow only for stable indices */
	TArray<FWorldInfo> WorldInfos;

	/** Classes that are in use. Map from Id to ClassInfo index */
	TMap<uint64, int32> ClassIdToIndexMap;

	/** Objects that are in use. Map from Id to ObjectInfo index */
	TMap<RewindDebugger::FObjectId, int32> ObjectIdToIndexMap;

	/** Worlds that are in use. Map from Id to WorldInfo index */
	TMap<uint64, int32> WorldIdToIndexMap;

	/** Map from object Ids to timeline index */
	TMap<RewindDebugger::FObjectId, uint32> ObjectIdToEventTimelines;
	TMap<uint64, uint32> ObjectIdToTransformTimeline;
	TMap<uint64, uint32> ObjectIdToPropertiesStorage;

	struct FPawnPossessMessage
	{
		uint64 ControllerId = 0;
		uint64 PawnId = 0;
	};

	struct FObjectTransformMessage
	{
		uint64 ObjectId = 0;
		double RecordingTime;
		FTransform Transform;
	};

	struct FObjectExistsMessage
	{
		RewindDebugger::FObjectId ObjectId;
	};

	struct FObjectPropertiesStorage
	{
		double OpenStartTime;
		uint64 OpenEventId;
		FObjectPropertiesMessage OpenEvent;
		TSharedPtr<TraceServices::TIntervalTimeline<FObjectPropertiesMessage>> Timeline;
		TArray<FObjectPropertyValue> Values;
	};

	/** Message storage */
	TArray<TSharedRef<TraceServices::TPointTimeline<FObjectEventMessage>>> EventTimelines;
	TraceServices::TPointTimeline<FObjectTransformMessage> TransformTimeline;
	TArray<TSharedRef<FObjectPropertiesStorage>> PropertiesStorage;
	TSharedPtr<TraceServices::TPointTimeline<FViewMessage>> ViewTimeline;

	/** Map of class path name to ClassInfo index */
	TMap<FStringView, int32> ClassPathNameToIndexMap;

	/** EndPlay event text */
	const TCHAR* EndPlayEvent;

	/** Delegate fired when an object receives an end play event */
	FOnObjectEndPlay OnObjectEndPlayDelegate;

	/** Map from string ID to stored string */
	TMap<uint32, const TCHAR*> PropertyStrings;

	/** Map of RecordingInfo Timelines by RecordingId - Each timeline is a mapping from Gameplay Elapsed time to Profiler Elapsed Time for one recording session */
	TMap<uint32, TSharedRef<TraceServices::TPointTimeline<FRecordingInfoMessage>>> Recordings;

	// Timeline containing intervals where a controller is attached to a pawn
	TraceServices::TIntervalTimeline<FPawnPossessMessage> PawnPossession;

	// Timeline containing intervals where an object exists
	TraceServices::TIntervalTimeline<FObjectExistsMessage> ObjectLifetimes;

	// Timeline containing intervals where an object exists - In Rewind Debugger recording time units, to avoid excessive conversions
	TraceServices::TIntervalTimeline<FObjectExistsMessage> ObjectRecordingLifetimes;

	// map from controller id to index in the PawnPossession timeline, for lookup when ending events
	TMap<uint64, uint64> ActivePawnPossession;

	// map from object id to index in ObjectLifetimes, for lookup when objects are destroyed
	TMap<RewindDebugger::FObjectId, uint64> ActiveObjectLifetimes;

	// map from object id to index in ObjectRecordingLifetimes, for lookup
	TMap<RewindDebugger::FObjectId, uint64> ActiveObjectRecordingLifetimes;

	/** MultiMap from parent object to sub objects, to avoid slow reverse lookups */
	TMultiMap<RewindDebugger::FObjectId, RewindDebugger::FObjectId> ObjectHierarchy;

	/** The target type specified in the current analysis session */
	TOptional<EBuildTargetType> TargetType;

	/** Whether we have any data */
	bool bHasAnyData = false;

	/** The elapsed world time */
	double RecordingDuration = 0.0;

	/** Whether we have any object properties */
	bool bHasObjectProperties = false;
};
