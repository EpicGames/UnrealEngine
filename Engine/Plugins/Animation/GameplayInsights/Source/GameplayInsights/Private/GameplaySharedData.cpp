// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplaySharedData.h"

#include "Algo/Sort.h"
#include "Common/ProviderLock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"

// TraceServices
#include "TraceServices/Model/AnalysisSession.h"

// TraceInsights
#include "Insights/ITimingViewSession.h"

#include "GameplayInsightsModule.h"
#include "GameplayProvider.h"
#include "ObjectEventsTrack.h"
#include "ObjectPropertiesTrack.h"
#include "SGameplayTrackTree.h"
#include "STrackVariantValueView.h"

#define LOCTEXT_NAMESPACE "GameplaySharedData"

FGameplaySharedData::FGameplaySharedData()
	: AnalysisSession(nullptr)
	, TimingViewSession(nullptr)
	, bObjectTracksDirty(false)
	, bObjectTracksEnabled(false)
	, bObjectPropertyTracksEnabled(true)
{
}

void FGameplaySharedData::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession)
{
	TimingViewSession = &InTimingViewSession;

	ObjectTracks.Reset();
}

void FGameplaySharedData::OnEndSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession)
{
	ObjectTracks.Reset();

	TimingViewSession = nullptr;
}

TSharedRef<FObjectEventsTrack> FGameplaySharedData::GetObjectEventsTrackForId(UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession, const FObjectInfo& InObjectInfo)
{
	const FGameplayProvider* GameplayProvider = InAnalysisSession.ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
	check(GameplayProvider);

	const uint64 ObjectId = InObjectInfo.GetUObjectId();
	TSharedPtr<FObjectEventsTrack> LeafObjectEventsTrack = ObjectTracks.FindRef(ObjectId);
	if(!LeafObjectEventsTrack.IsValid())
	{
		LeafObjectEventsTrack = MakeShared<FObjectEventsTrack>(*this, ObjectId, InObjectInfo.Name);
		LeafObjectEventsTrack->SetVisibilityFlag(bObjectTracksEnabled);
		ObjectTracks.Add(ObjectId, LeafObjectEventsTrack);

		InTimingViewSession.AddScrollableTrack(LeafObjectEventsTrack);
		InvalidateObjectTracksOrder();
	}

	// Fill the outer chain
	if(InObjectInfo.GetOuterId().IsSet())
	{
		TSharedPtr<FObjectEventsTrack> ObjectEventsTrack = LeafObjectEventsTrack;
		uint64 OuterId = InObjectInfo.GetOuterUObjectId();
		while(const FObjectInfo* OuterInfo = GameplayProvider->FindObjectInfo(OuterId))
		{
			TSharedPtr<FObjectEventsTrack> OuterEventsTrack = ObjectTracks.FindRef(OuterId);
			if(!OuterEventsTrack.IsValid())
			{
				OuterEventsTrack = MakeShared<FObjectEventsTrack>(*this, OuterId, OuterInfo->Name);
				OuterEventsTrack->SetVisibilityFlag(bObjectTracksEnabled);
				ObjectTracks.Add(OuterId, OuterEventsTrack);

				InTimingViewSession.AddScrollableTrack(OuterEventsTrack);
				InvalidateObjectTracksOrder();
			}

			// setup hierarchy
			if(ObjectEventsTrack->GetGameplayTrack().GetParentTrack() == nullptr)
			{
				OuterEventsTrack->GetGameplayTrack().AddChildTrack(ObjectEventsTrack->GetGameplayTrack());
			}

			ObjectEventsTrack = OuterEventsTrack;
			OuterId = OuterInfo->GetOuterUObjectId();
		}
	}

	return LeafObjectEventsTrack.ToSharedRef();
}

static void UpdateTrackOrderRecursive(TSharedRef<FBaseTimingTrack> InTrack, int32& InOutOrder)
{
	if(InTrack->Is<FObjectEventsTrack>())
	{
		TSharedRef<FObjectEventsTrack> ObjectEventsTrack = StaticCastSharedRef<FObjectEventsTrack>(InTrack);

		// recurse down object-track children, then non-object track leaf tracks to set 
		// overall ordering based on depth-first traversal of the hierarchy

		ObjectEventsTrack->SetOrder(InOutOrder++);

		for(FGameplayTrack* ChildTrack : ObjectEventsTrack->GetGameplayTrack().GetChildTracks())
		{
			if(ChildTrack->GetTimingTrack()->Is<FObjectEventsTrack>())
			{
				ChildTrack->SetIndent(ObjectEventsTrack->GetGameplayTrack().GetIndent() + 1);
				UpdateTrackOrderRecursive(StaticCastSharedRef<FObjectEventsTrack>(ChildTrack->GetTimingTrack()), InOutOrder);
			}
		}

		for(FGameplayTrack* ChildTrack : ObjectEventsTrack->GetGameplayTrack().GetChildTracks())
		{
			if(!ChildTrack->GetTimingTrack()->Is<FObjectEventsTrack>())
			{
				ChildTrack->SetIndent(ObjectEventsTrack->GetGameplayTrack().GetIndent() + 1);
				ChildTrack->GetTimingTrack()->SetOrder(InOutOrder++);
			}
		}
	}
}

void FGameplaySharedData::MakeTrackAndAncestorsVisible(const TSharedRef<FObjectEventsTrack>& InObjectEventsTrack, bool bInVisible)
{
	TSharedPtr<FObjectEventsTrack> CurrentTrack = InObjectEventsTrack;
	while(CurrentTrack.IsValid())
	{
		CurrentTrack->SetVisibilityFlag(bInVisible);

		FGameplayTrack* ParentGameplayTrack = CurrentTrack->GetGameplayTrack().GetParentTrack();
		TSharedPtr<FBaseTimingTrack> BaseParentTrack = ParentGameplayTrack != nullptr ? ParentGameplayTrack->GetTimingTrack() : TSharedPtr<FBaseTimingTrack>();
		if(BaseParentTrack.IsValid())
		{
			check(BaseParentTrack->Is<FObjectEventsTrack>());
		}
		CurrentTrack = StaticCastSharedPtr<FObjectEventsTrack>(BaseParentTrack);
	}

	InvalidateObjectTracksOrder();
}

void FGameplaySharedData::Tick(UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	AnalysisSession = &InAnalysisSession;

	const FGameplayProvider* GameplayProvider = InAnalysisSession.ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);

	if(GameplayProvider)
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

		if(GameplayProvider->HasAnyData())
		{
			// Add a track for each tracked object
			GameplayProvider->EnumerateObjects([this, &InTimingViewSession, &InAnalysisSession, &GameplayProvider](const FObjectInfo& InObjectInfo)
			{
				if(bObjectTracksEnabled || bObjectPropertyTracksEnabled)
				{
					TSharedPtr<FObjectEventsTrack> ObjectEventsTrack;

					GameplayProvider->ReadObjectEventsTimeline(InObjectInfo.GetId(), [this, &InTimingViewSession, &InAnalysisSession, &InObjectInfo, &ObjectEventsTrack](const IGameplayProvider::ObjectEventsTimeline& InTimeline)
					{
						if(bObjectTracksEnabled)
						{
							ObjectEventsTrack = GetObjectEventsTrackForId(InTimingViewSession, InAnalysisSession, InObjectInfo);
						}
					});

					GameplayProvider->ReadObjectPropertiesTimeline(InObjectInfo.GetUObjectId(), [this, &InObjectInfo, &ObjectEventsTrack, &InTimingViewSession, &InAnalysisSession](const IGameplayProvider::ObjectPropertiesTimeline& InTimeline)
					{
						if(!ObjectEventsTrack.IsValid())
						{
							ObjectEventsTrack = GetObjectEventsTrackForId(InTimingViewSession, InAnalysisSession, InObjectInfo);
						}

						auto FindObjectProperties = [](const FBaseTimingTrack& InTrack)
						{
							return InTrack.Is<FObjectPropertiesTrack>();
						};

						TSharedPtr<FObjectPropertiesTrack> ExistingObjectPropertiesTrack = StaticCastSharedPtr<FObjectPropertiesTrack>(ObjectEventsTrack->GetGameplayTrack().FindChildTrack(InObjectInfo.GetUObjectId(), FindObjectProperties));
						if(!ExistingObjectPropertiesTrack.IsValid())
						{
							TSharedPtr<FObjectPropertiesTrack> ObjectPropertiesTrack = MakeShared<FObjectPropertiesTrack>(*this, InObjectInfo.GetUObjectId(), InObjectInfo.Name);
							ObjectPropertiesTrack->SetVisibilityFlag(bObjectPropertyTracksEnabled);
							ObjectPropertyTracks.Add(ObjectPropertiesTrack.ToSharedRef());

							InTimingViewSession.AddScrollableTrack(ObjectPropertiesTrack);
							InvalidateObjectTracksOrder();

							ObjectEventsTrack->GetGameplayTrack().AddChildTrack(ObjectPropertiesTrack->GetGameplayTrack());

							MakeTrackAndAncestorsVisible(ObjectEventsTrack.ToSharedRef(), true);
						}
					});
				}
			});
		}

		if(bObjectTracksDirty)
		{
			SortTracks();
			InTimingViewSession.InvalidateScrollableTracksOrder();
			bObjectTracksDirty = false;
		}
	}
}

void FGameplaySharedData::ExtendFilterMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("GameplayTracks", LOCTEXT("GameplayTracksSection", "Gameplay Tracks"));
	{
		InMenuBuilder.AddSubMenu(
			LOCTEXT("ToggleGameplayTracks", "Gameplay Tracks"),
			LOCTEXT("ToggleGameplayTracks_Tooltip", "Show/hide individual gameplay tracks"),
			FNewMenuDelegate::CreateLambda([this](FMenuBuilder& InSubMenuBuilder)
			{ 
				InSubMenuBuilder.AddWidget(
					SNew(SBox)
					.MaxDesiredHeight(300.0f)
					.MinDesiredWidth(300.0f)
					.MaxDesiredWidth(300.0f)
					[
						SNew(SGameplayTrackTree, *this)
					],
					FText(), true);
			})
		);

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleEventTracks", "Event Tracks"),
			LOCTEXT("ToggleEventTracks_Tooltip", "Show/hide the gameplay event tracks"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FGameplaySharedData::ToggleGameplayTracks),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FGameplaySharedData::AreGameplayTracksEnabled)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("TogglePropertyTracks", "Property Tracks"),
			LOCTEXT("TogglePropertyTracks_Tooltip", "Show/hide the object property tracks"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FGameplaySharedData::ToggleObjectPropertyTracks),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FGameplaySharedData::AreObjectPropertyTracksEnabled)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	InMenuBuilder.EndSection();
}

void FGameplaySharedData::SortTracks()
{
	int32 Order = 10000;

	// Find current roots
	RootTracks.Reset();

	for(auto ObjectTrackPair : ObjectTracks)
	{
		if(ObjectTrackPair.Value->GetGameplayTrack().GetParentTrack() == nullptr)
		{
			RootTracks.Add(ObjectTrackPair.Value.ToSharedRef());
		}
	}

	// Sort roots alphabetically
	Algo::Sort(RootTracks, [](TSharedPtr<FBaseTimingTrack> InTrack0, TSharedPtr<FBaseTimingTrack> InTrack1)
	{
		return InTrack0->GetName() < InTrack1->GetName();
	});

	// update ordering/indent
	for(TSharedPtr<FBaseTimingTrack> RootTrack : RootTracks)
	{
		UpdateTrackOrderRecursive(RootTrack.ToSharedRef(), Order);
	}

	OnTracksChangedDelegate.Broadcast();
}

void FGameplaySharedData::ToggleGameplayTracks()
{
	bObjectTracksEnabled = !bObjectTracksEnabled;

	for(auto ObjectTrackPair : ObjectTracks)
	{
		ObjectTrackPair.Value->SetVisibilityFlag(bObjectTracksEnabled);
	}
}

bool FGameplaySharedData::AreGameplayTracksEnabled() const
{
	return bObjectTracksEnabled;
}

void FGameplaySharedData::ToggleObjectPropertyTracks()
{
	bObjectPropertyTracksEnabled = !bObjectPropertyTracksEnabled;

	for(TSharedRef<FObjectPropertiesTrack>& ObjectPropertyTrack : ObjectPropertyTracks)
	{
		ObjectPropertyTrack->SetVisibilityFlag(bObjectPropertyTracksEnabled);
	}
}

bool FGameplaySharedData::AreObjectPropertyTracksEnabled() const
{
	return bObjectPropertyTracksEnabled;
}

void FGameplaySharedData::EnumerateObjectTracks(TFunctionRef<void(const TSharedRef<FObjectEventsTrack>&)> InCallback) const
{
	for(const auto& TrackPair : ObjectTracks)
	{
		InCallback(TrackPair.Value.ToSharedRef());
	}
}

TSharedPtr<SDockTab> FGameplaySharedData::FindDocumentTab(const TArray<TWeakPtr<SDockTab>>& InWeakDocumentTabs, TFunction<bool(const TSharedRef<SDockTab>&)> InSearchFunction)
{
	for(TWeakPtr<SDockTab> WeakTab : InWeakDocumentTabs)
	{
		TSharedPtr<SDockTab> PinnedTab = WeakTab.Pin();
		if(PinnedTab.IsValid())
		{
			if(InSearchFunction(PinnedTab.ToSharedRef()))
			{
				return PinnedTab;
			}
		}
	}

	return nullptr;
}

void FGameplaySharedData::OpenTrackVariantsTab(const FGameplayTrack& InGameplayTrack) const
{
	if(TimingViewSession && AnalysisSession)
	{
		FGameplayInsightsModule& GameplayInsightsModule = FModuleManager::GetModuleChecked<FGameplayInsightsModule>("GameplayInsights");
		TSharedRef<SDockTab> Tab = GameplayInsightsModule.SpawnTimingProfilerDocumentTab(
			FSearchForTab([this, &InGameplayTrack]()
			{
				return FindDocumentTab(WeakTrackVariantsDocumentTabs, [&InGameplayTrack](const TSharedRef<SDockTab>& InDockTab)
				{
					return StaticCastSharedRef<STrackVariantValueView>(InDockTab->GetContent())->GetTimingTrack() == InGameplayTrack.GetTimingTrack();
				});
			})
		);

		Tab->SetContent(SNew(STrackVariantValueView, InGameplayTrack.GetTimingTrack(), *TimingViewSession, *AnalysisSession));
		WeakTrackVariantsDocumentTabs.Add(Tab);
		Tab->SetLabel(FText::FromString(InGameplayTrack.GetTimingTrack()->GetName()));
	}
}

#undef LOCTEXT_NAMESPACE
