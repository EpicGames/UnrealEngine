// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ViewModels/TimingEventsTrack.h"
#include "Framework/Docking/TabManager.h"

namespace TraceServices { class IAnalysisSession; }
namespace UE::Insights::Timing { class ITimingViewSession; }
class FObjectEventsTrack;
class FObjectPropertiesTrack;
class FSkeletalMeshPoseTrack;
class FAnimationTickRecordsTrack;
struct FObjectInfo;
class FGameplayTrack;

class FGameplaySharedData
{
public:
	FGameplaySharedData();

	void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession);
	void OnEndSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession);
	void Tick(UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession);
	void ExtendFilterMenu(FMenuBuilder& InMenuBuilder);

	// Helper function. Builds object track hierarchy on-demand and returns a track for the supplied object info.
	TSharedRef<FObjectEventsTrack> GetObjectEventsTrackForId(UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession, const FObjectInfo& InObjectInfo);

	// Helper function to make tracks visible when children are first added
	void MakeTrackAndAncestorsVisible(const TSharedRef<FObjectEventsTrack>& InObjectEventsTrack, bool bInVisible);

	// Check whether gameplay tacks are enabled
	bool AreGameplayTracksEnabled() const;

	// Invalidate object trcks order, so they get re-sorted next tick
	void InvalidateObjectTracksOrder() { bObjectTracksDirty = true; }

	// Get the last cached analysis session
	const TraceServices::IAnalysisSession& GetAnalysisSession() const { return *AnalysisSession; }

	// Get the timing view session
	UE::Insights::Timing::ITimingViewSession& GetTimingViewSession() { return *TimingViewSession; }

	// Check the validity of the analysis session
	bool IsAnalysisSessionValid() const { return AnalysisSession != nullptr; }

	// Enumerate object tracks
	void EnumerateObjectTracks(TFunctionRef<void(const TSharedRef<FObjectEventsTrack>&)> InCallback) const;

	// Get the root tracks
	const TArray<TSharedRef<FBaseTimingTrack>>& GetRootTracks() const { return RootTracks; }

	// Delegate fired when tracks change
	FSimpleMulticastDelegate& OnTracksChanged() { return OnTracksChangedDelegate; }

	/** Search predicate wrapper for an open document */
	struct FSearchForTab : public FTabManager::FSearchPreference
	{
		FSearchForTab(TFunction<TSharedPtr<SDockTab>(void)> InSearchFunction)
			: SearchFunction(InSearchFunction)
		{
		}

		virtual TSharedPtr<SDockTab> Search(const FTabManager& Manager, FName PlaceholderId, const TSharedRef<SDockTab>& UnmanagedTab) const override
		{
			return SearchFunction();
		}

		TFunction<TSharedPtr<SDockTab>(void)> SearchFunction;
	};

	// Helper function to find an existing document tab
	static TSharedPtr<SDockTab> FindDocumentTab(const TArray<TWeakPtr<SDockTab>>& InWeakDocumentTabs, TFunction<bool(const TSharedRef<SDockTab>&)> InSearchFunction);

	/** Open a tab to see a track's variants */
	void OpenTrackVariantsTab(const FGameplayTrack& InGameplayTrack) const;

private:
	// Re-sort tracks if track ordering has changed
	void SortTracks();

	// UI handlers
	void ToggleGameplayTracks();
	void ToggleObjectPropertyTracks();
	bool AreObjectPropertyTracksEnabled() const;

private:
	// Track for each tracked object, mapped from Object ID -> track
	TMap<uint64, TSharedPtr<FObjectEventsTrack>> ObjectTracks;
	TArray<TSharedRef<FObjectPropertiesTrack>> ObjectPropertyTracks;

	// The root tracks
	TArray<TSharedRef<FBaseTimingTrack>> RootTracks;

	// Cached analysis session, set in Tick()
	const TraceServices::IAnalysisSession* AnalysisSession;

	// Cached timing view session, set in OnBeginSession/OnEndSession
	UE::Insights::Timing::ITimingViewSession* TimingViewSession;

	// Delegate fired when tracks change
	FSimpleMulticastDelegate OnTracksChangedDelegate;

	// All the documents we have spawned
	mutable TArray<TWeakPtr<SDockTab>> WeakTrackVariantsDocumentTabs;

	// Dirty flag for adding object tracks, used to trigger re-sorting
	bool bObjectTracksDirty;

	// Whether all of our object tracks are enabled
	bool bObjectTracksEnabled;

	// Whether all of our object property tracks are enabled
	bool bObjectPropertyTracksEnabled;
};
