// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTimingViewExtender.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/IUnrealInsightsModule.h"
#include "GameplaySharedData.h"
#include "AnimationSharedData.h"
#include "UObject/WeakObjectPtr.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#if WITH_EDITOR
#include "SGameplayInsightsTransportControls.h"
#include "Widgets/Layout/SBorder.h"
#endif

#if WITH_ENGINE
#include "Engine/World.h"
#include "Editor/EditorEngine.h"
#endif

#define LOCTEXT_NAMESPACE "GameplayTimingViewExtender"

void FGameplayTimingViewExtender::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	if (InSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if(PerSessionData == nullptr)
	{
		PerSessionData = &PerSessionDataMap.Add(&InSession);
		PerSessionData->GameplaySharedData = new FGameplaySharedData();
		PerSessionData->AnimationSharedData = new FAnimationSharedData(*PerSessionData->GameplaySharedData);

		PerSessionData->GameplaySharedData->OnBeginSession(InSession);
		PerSessionData->AnimationSharedData->OnBeginSession(InSession);

#if WITH_EDITOR
		InSession.AddOverlayWidget(
			SNew(SOverlay)
			.Visibility_Lambda([PerSessionData](){ return PerSessionData->GameplaySharedData->IsAnalysisSessionValid() && PerSessionData->AnimationSharedData->ShouldShowTransportControls()  ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed; })
			+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Bottom)
			.Padding(20.0f)
			[
				SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				[
					SAssignNew(PerSessionData->TransportControls, SGameplayInsightsTransportControls, *PerSessionData->GameplaySharedData)
				]
			]);
#endif
	}
	else
	{
		PerSessionData->GameplaySharedData->OnBeginSession(InSession);
		PerSessionData->AnimationSharedData->OnBeginSession(InSession);
	}
}

void FGameplayTimingViewExtender::OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	if (InSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if(PerSessionData != nullptr)
	{
		PerSessionData->GameplaySharedData->OnEndSession(InSession);
		PerSessionData->AnimationSharedData->OnEndSession(InSession);

		delete PerSessionData->GameplaySharedData;
		PerSessionData->GameplaySharedData = nullptr;
		delete PerSessionData->AnimationSharedData;
		PerSessionData->AnimationSharedData = nullptr;
#if WITH_EDITOR
		PerSessionData->TransportControls.Reset();
#endif
	}

	PerSessionDataMap.Remove(&InSession);
}

void FGameplayTimingViewExtender::Tick(UE::Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	if (InSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if(PerSessionData != nullptr)
	{
		PerSessionData->GameplaySharedData->Tick(InSession, InAnalysisSession);
		PerSessionData->AnimationSharedData->Tick(InSession, InAnalysisSession);
	}
}

void FGameplayTimingViewExtender::ExtendFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder)
{
	if (InSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if(PerSessionData != nullptr)
	{
		PerSessionData->GameplaySharedData->ExtendFilterMenu(InMenuBuilder);
		PerSessionData->AnimationSharedData->ExtendFilterMenu(InMenuBuilder);
	}
}

#if WITH_ENGINE

UWorld* FGameplayTimingViewExtender::GetWorldToVisualize()
{
	UWorld* World = nullptr;

#if WITH_EDITOR
	UEditorEngine* EditorEngine = Cast<UEditorEngine>(GEngine);
	if (GIsEditor && EditorEngine != nullptr && World == nullptr)
	{
		// lets use PlayWorld during PIE/Simulate and regular world from editor otherwise, to draw debug information
		World = EditorEngine->PlayWorld != nullptr ? ToRawPtr(EditorEngine->PlayWorld) : EditorEngine->GetEditorWorldContext().World();
	}

#endif
	if (!GIsEditor && World == nullptr)
	{
		World = GEngine->GetWorld();
	}

	return World;
}

#endif

void FGameplayTimingViewExtender::TickVisualizers(float DeltaTime)
{
#if WITH_ENGINE
	UWorld* WorldToVisualize = GetWorldToVisualize();
	if(WorldToVisualize)
	{
		for(auto& PerSessionData : PerSessionDataMap)
		{
			PerSessionData.Value.AnimationSharedData->DrawPoses(WorldToVisualize);
		}
	}
#endif
}

#if WITH_EDITOR

void FGameplayTimingViewExtender::GetCustomDebugObjects(const IAnimationBlueprintEditor& InAnimationBlueprintEditor, TArray<FCustomDebugObject>& OutDebugList)
{
	for(auto& PerSessionData : PerSessionDataMap)
	{
		PerSessionData.Value.AnimationSharedData->GetCustomDebugObjects(InAnimationBlueprintEditor, OutDebugList);
	}
}

#endif

#undef LOCTEXT_NAMESPACE
