// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ITimingViewExtender.h"

namespace UE::Insights::Timing { class ITimingViewSession; }
class UWorld;
class FGameplaySharedData;
class FAnimationSharedData;
class IAnimationBlueprintEditor;
struct FCustomDebugObject;
class SGameplayInsightsTransportControls;

class FGameplayTimingViewExtender : public UE::Insights::Timing::ITimingViewExtender
{
public:
	// Insights::ITimingViewExtender interface
	virtual void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void Tick(UE::Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;
	virtual void ExtendFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) override;

#if WITH_EDITOR
	// Gets a world to perform visualizations within, depending on context
	static UWorld* GetWorldToVisualize();

	// Get custom debug objects for integration with anim blueprint debugging
	void GetCustomDebugObjects(const IAnimationBlueprintEditor& InAnimationBlueprintEditor, TArray<FCustomDebugObject>& OutDebugList);
#endif

	// Tick the visualizers
	void TickVisualizers(float DeltaTime);

private:
	struct FPerSessionData
	{
		// Shared data
		FGameplaySharedData* GameplaySharedData;
		FAnimationSharedData* AnimationSharedData;
#if WITH_EDITOR
		TSharedPtr<SGameplayInsightsTransportControls> TransportControls;
#endif
	};

	// The data we host per-session
	TMap<UE::Insights::Timing::ITimingViewSession*, FPerSessionData> PerSessionDataMap;
};
