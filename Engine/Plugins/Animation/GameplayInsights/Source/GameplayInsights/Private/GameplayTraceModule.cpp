// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTraceModule.h"
#include "AnimationAnalyzer.h"
#include "AnimationProvider.h"
#include "GameplayProvider.h"
#include "GameplayAnalyzer.h"

FName FGameplayTraceModule::ModuleName("GameplayTrace");

void FGameplayTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("Gameplay");
}

void FGameplayTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedPtr<FGameplayProvider> GameplayProvider = MakeShared<FGameplayProvider>(InSession);
	InSession.AddProvider(FGameplayProvider::ProviderName, GameplayProvider, GameplayProvider);
	TSharedPtr<FAnimationProvider> AnimationProvider = MakeShared<FAnimationProvider>(InSession, *GameplayProvider);
	InSession.AddProvider(FAnimationProvider::ProviderName, AnimationProvider);

	InSession.AddAnalyzer(new FAnimationAnalyzer(InSession, *AnimationProvider));
	InSession.AddAnalyzer(new FGameplayAnalyzer(InSession, *GameplayProvider));
}

void FGameplayTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(TEXT("Object"));
	OutLoggers.Add(TEXT("Animation"));
}

void FGameplayTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{

}

const IAnimationProvider* ReadAnimationProvider(const TraceServices::IAnalysisSession& Session)
{
	return Session.ReadProvider<IAnimationProvider>(FAnimationProvider::ProviderName);
}

