// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraceServices/ModuleService.h"

class IAnimationProvider;

class FGameplayTraceModule : public TraceServices::IModule
{
public:
	// TraceServices::IModule interface
	virtual void GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo) override;
	virtual void OnAnalysisBegin(TraceServices::IAnalysisSession& Session) override;
	virtual void GetLoggers(TArray<const TCHAR *>& OutLoggers) override;
	virtual void GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory) override;
	virtual const TCHAR* GetCommandLineArgument() override { return TEXT("objecttrace"); }

private:
	static FName ModuleName;
};

const IAnimationProvider* ReadAnimationProvider(const TraceServices::IAnalysisSession& Session);
