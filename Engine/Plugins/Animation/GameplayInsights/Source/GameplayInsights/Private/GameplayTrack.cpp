// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTrack.h"

#include "Algo/Sort.h"
#include "Common/ProviderLock.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameplayProvider.h"
#include "GameplaySharedData.h"
#include "IGameplayProvider.h"

// TraceServices
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Frames.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"

// TraceInsights
#include "Insights/ViewModels/ITimingViewDrawHelper.h"
#include "Insights/ViewModels/TimingEventsTrack.h"
#include "Insights/ViewModels/TimingTrackViewport.h"

INSIGHTS_IMPLEMENT_RTTI(FGameplayTimingEventsTrack);

#define LOCTEXT_NAMESPACE "GameplayTrack"

RewindDebugger::FObjectId FGameplayTrack::GetAssociatedObjectId() const
{
	return RewindDebugger::FObjectId{ ObjectId };
}

void FGameplayTrack::AddChildTrack(FGameplayTrack& InChildTrack)
{
	check(InChildTrack.Parent == nullptr);
	InChildTrack.Parent = this;

	Children.Add(&InChildTrack);

	Algo::Sort(Children, [](FGameplayTrack* InTrack0, FGameplayTrack* InTrack1)
	{
		return InTrack0->GetTimingTrack()->GetName() < InTrack1->GetTimingTrack()->GetName();
	});	
}

TSharedPtr<FBaseTimingTrack> FGameplayTrack::FindChildTrack(uint64 InObjectId, TFunctionRef<bool(const FBaseTimingTrack& InTrack)> Callback) const
{
	for(const FGameplayTrack* ChildTrack : Children)
	{
		if( ChildTrack != nullptr &&
			ChildTrack->ObjectId == InObjectId && 
			Callback(ChildTrack->GetTimingTrack().Get()))
		{
			return ChildTrack->GetTimingTrack();
		}
	}

	return nullptr;
}

static FORCEINLINE bool IntervalsIntersect(float Min1, float Max1, float Min2, float Max2)
{
	return !(Max2 < Min1 || Max1 < Min2);
}

void FGameplayTrack::DrawHeaderForTimingTrack(const ITimingTrackDrawContext& InContext, const FBaseTimingTrack& InTrack, bool bUsePreallocatedLayers) const
{
	const float X = (float)Indent * GameplayTrackConstants::IndentSize;
	const float Y = InTrack.GetPosY();
	const float H = InTrack.GetHeight();
	const float TrackNameH = H > 7.0f ? 12.0f : H;
	const int32 HeaderBackgroundLayerId = bUsePreallocatedLayers ? InContext.GetHelper().GetHeaderBackgroundLayerId() : InContext.GetDrawContext().LayerId++;
	const int32 HeaderTextLayerId = bUsePreallocatedLayers ? InContext.GetHelper().GetHeaderTextLayerId() : InContext.GetDrawContext().LayerId++;

	if (H > 0.0f &&
		Y + H > InContext.GetViewport().GetTopOffset() &&
		Y < InContext.GetViewport().GetHeight() - InContext.GetViewport().GetBottomOffset())
	{
		// Draw a horizontal line between timelines.
		InContext.GetDrawContext().DrawBox(HeaderBackgroundLayerId, X, Y, InContext.GetViewport().GetWidth(), 1.0f, InContext.GetHelper().GetWhiteBrush(), InContext.GetHelper().GetEdgeColor());

		if(H > 7.0f)
		{
			const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			const float NameWidth = FontMeasureService->Measure(InTrack.GetName(), InContext.GetHelper().GetEventFont()).X;
			InContext.GetDrawContext().DrawBox(HeaderBackgroundLayerId, X, Y + 1.0f, NameWidth + 4.0f, TrackNameH, InContext.GetHelper().GetWhiteBrush(), InContext.GetHelper().GetEdgeColor());
			InContext.GetDrawContext().DrawText(HeaderTextLayerId, X + 2.0f, Y, InTrack.GetName(), InContext.GetHelper().GetEventFont(), InContext.GetHelper().GetTrackNameTextColor(InTrack));
		}
		else
		{
			InContext.GetDrawContext().DrawBox(HeaderBackgroundLayerId, X, Y + 1.0f, H, H, InContext.GetHelper().GetWhiteBrush(), InContext.GetHelper().GetEdgeColor());
		}
	}

	// Draw lines connecting to parent
	if(Parent && Parent->GetTimingTrack()->IsVisible())
	{
		const float ParentX = (float)Parent->GetIndent() * GameplayTrackConstants::IndentSize;
		const float ParentY = FMath::Max(InContext.GetViewport().GetTopOffset(), Parent->GetTimingTrack()->GetPosY());

		if (IntervalsIntersect(ParentY, Y, InContext.GetViewport().GetTopOffset(), InContext.GetViewport().GetHeight() - InContext.GetViewport().GetBottomOffset()))
		{
			InContext.GetDrawContext().DrawBox(HeaderBackgroundLayerId, ParentX, Y + (TrackNameH * 0.5f), X - ParentX, 1.0f, InContext.GetHelper().GetWhiteBrush(), InContext.GetHelper().GetEdgeColor());
			InContext.GetDrawContext().DrawBox(HeaderBackgroundLayerId, ParentX, ParentY, 1.0f, (Y - ParentY) + (TrackNameH * 0.5f), InContext.GetHelper().GetWhiteBrush(), InContext.GetHelper().GetEdgeColor());
		}
	}
}

FText FGameplayTrack::GetWorldPostFix(const FWorldInfo& InWorldInfo)
{
	switch (InWorldInfo.Type)
	{
	default:
		return LOCTEXT("Unknown", "Unknown");
	case FWorldInfo::EType::None:
		return LOCTEXT("None", "None");
	case FWorldInfo::EType::PIE:
	{
		switch (InWorldInfo.NetMode)
		{
		case FWorldInfo::ENetMode::Client:
			return FText::Format(LOCTEXT("ClientIndexFormat", "Client {0}"), InWorldInfo.PIEInstanceId - 1);
		case FWorldInfo::ENetMode::DedicatedServer:
			return LOCTEXT("ServerPostfix", "Server");
		case FWorldInfo::ENetMode::ListenServer:
			return LOCTEXT("ListenServerPostfix", "Listen Server");
		case FWorldInfo::ENetMode::Standalone:
			return InWorldInfo.bIsSimulating ? LOCTEXT("SimulateInEditorPostfix", "Simulate") : LOCTEXT("PlayInEditorPostfix", "PIE");
		}
	}
	case FWorldInfo::EType::Editor:
	case FWorldInfo::EType::EditorPreview:
		return LOCTEXT("EditorPostfix", "Editor");
	case FWorldInfo::EType::Game:
	case FWorldInfo::EType::GamePreview:
	case FWorldInfo::EType::GameRPC:
		return LOCTEXT("GamePostfix", "Game");
	case FWorldInfo::EType::Inactive:
		return LOCTEXT("InactivePostfix", "Inactive");
	}
}

FText FGameplayTrack::GetWorldName(const TraceServices::IAnalysisSession& InAnalysisSession) const
{
	const FGameplayProvider* GameplayProvider = InAnalysisSession.ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
	if (GameplayProvider)
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

		if(const FWorldInfo* WorldInfo = GameplayProvider->FindWorldInfoFromObject(ObjectId))
		{
			const FObjectInfo& WorldObjectInfo = GameplayProvider->GetObjectInfo(WorldInfo->Id);
			return FText::Format(LOCTEXT("WorldNameFormat", "{0} ({1})"), FText::FromString(WorldObjectInfo.Name), GetWorldPostFix(*WorldInfo));
		}
	}

	return LOCTEXT("UnknownWorld", "Unknown");
}

void FGameplayTimingEventsTrack::GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const
{ 
	GetVariantsAtTime(InFrame.StartTime, OutVariants);
}

void FGameplayTimingEventsTrack::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("View", LOCTEXT("ViewHeader", "View"));
	{
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ViewProperties", "View Properties"),
			LOCTEXT("ViewProperties_Tooltip", "Open a window to view the properties of this track. You can scrub the timeline to see properties change in real-time."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ GameplaySharedData.OpenTrackVariantsTab(GetGameplayTrack()); })
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();
}

#undef LOCTEXT_NAMESPACE
