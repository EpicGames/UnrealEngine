// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ViewModels/TimingEventsTrack.h"

namespace GameplayTrackConstants
{
	constexpr float IndentSize = 12.0f;
}

struct FVariantTreeNode;
class FGameplaySharedData;
struct FWorldInfo;
namespace TraceServices
{
	class IAnalysisSession;
	struct FFrame;
}

namespace RewindDebugger
{
	struct FObjectId;
}

// Provides parent/child hierarchy structure and the owning object Id
// Designed as a compositional member of outer timing tracks
class FGameplayTrack
{
public:
	FGameplayTrack(FBaseTimingTrack& InTimingTrack, uint64 InObjectId)
		: TimingTrack(InTimingTrack)
		, ObjectId(InObjectId)
		, Parent(nullptr)
		, Indent(0)
	{
	}

	// Get the object ID for this track
	uint64 GetObjectId() const { return ObjectId; }

	RewindDebugger::FObjectId GetAssociatedObjectId() const;

	/** Get the parent track */
	FGameplayTrack* GetParentTrack() { return Parent; }

	/** Get child tracks */
	TArrayView<FGameplayTrack*> GetChildTracks() { return Children; }

	/** Add a child track */
	void AddChildTrack(FGameplayTrack& InChildTrack);

	/** Find a child track using the specified callback */
	TSharedPtr<FBaseTimingTrack> FindChildTrack(uint64 InObjectId, TFunctionRef<bool(const FBaseTimingTrack& InTrack)> Callback) const;

	/** Helper to draw the name for a timing track (uses indentation etc.) */
	void DrawHeaderForTimingTrack(const ITimingTrackDrawContext& InContext, const FBaseTimingTrack& InTrack, bool bUsePreallocatedLayers) const;

	/** Access the outer timing track */
	TSharedRef<FBaseTimingTrack> GetTimingTrack() const { return TimingTrack.AsShared(); }

	/** Access indent */
	void SetIndent(uint32 InIndent) { Indent = InIndent; }

	/** Access indent */
	uint32 GetIndent() const { return Indent; }

	/** Get the world name for this object */
	FText GetWorldName(const TraceServices::IAnalysisSession& InAnalysisSession) const;

	/** Helper function to find a postfix for a world (e.g. "Client 0", "Server" etc.) */
	static FText GetWorldPostFix(const FWorldInfo& InWorldInfo);

private:
	/** Outer timing track */
	FBaseTimingTrack& TimingTrack;

	// The object ID for this track
	uint64 ObjectId;

	/** Parent track */
	FGameplayTrack* Parent;

	/** Our child tracks */
	TArray<FGameplayTrack*> Children;

	/** Our indent when drawn in a tree */
	uint32 Indent;
};

template <class Base>
class TGameplayTrackMixin : public Base
{
	//INSIGHTS_DECLARE_RTTI(TGameplayTrackMixin, Base) -- Commented to skip this class in the Simple RTTI class hierarchy. It will get same type name as its Base class.

public:
	TGameplayTrackMixin(uint64 InObjectId, const FText& InName)
		: Base(InName.ToString())
		, GameplayTrack(*this, InObjectId)
	{
	}

	/** Access the underlying gameplay track */
	FGameplayTrack& GetGameplayTrack() { return GameplayTrack; }

	/** Access the underlying gameplay track */
	const FGameplayTrack& GetGameplayTrack() const { return GameplayTrack; }

private:
	FGameplayTrack GameplayTrack;
};

// Common base class for timing-event tracks
class FGameplayTimingEventsTrack : public TGameplayTrackMixin<FTimingEventsTrack>
{
	INSIGHTS_DECLARE_RTTI(FGameplayTimingEventsTrack, TGameplayTrackMixin<FTimingEventsTrack>)

public:
	FGameplayTimingEventsTrack(const FGameplaySharedData& InGameplaySharedData, uint64 InObjectId, const FText& InName)
		: TGameplayTrackMixin<FTimingEventsTrack>(InObjectId, InName)
		, GameplaySharedData(InGameplaySharedData)
	{}

	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;

	/** Get all variants at the specified time/frame */
	virtual void GetVariantsAtTime(double InTime, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const {}
	virtual void GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const;

protected:
	const FGameplaySharedData& GameplaySharedData;
};

//template <class Base>
//INSIGHTS_IMPLEMENT_RTTI(TGameplayTrackMixin<Base>) // Note: All templated classes will return same type name as FName(TEXT"TGameplayTrackMixin<Base>") !!!
