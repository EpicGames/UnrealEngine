// Copyright Epic Games, Inc. All Rights Reserved.

#include "InertializationsTrack.h"
#include "Common/ProviderLock.h"
#include "SCurveTimelineView.h"
#include "IRewindDebugger.h"
#include "GameplayProvider.h"
#include "AnimationProvider.h"
#include "Styling/SlateIconFinder.h"
#include "SInertializationDetailsView.h"

#if WITH_EDITOR
#include "Animation/AnimInstance.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "IAnimationBlueprintEditor.h"
#include "Animation/AnimBlueprint.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ObjectTrace.h"
#endif

#define LOCTEXT_NAMESPACE "InertializationsTrack"

namespace RewindDebugger
{

FInertializationsTrack::FInertializationsTrack(uint64 InObjectId) : ObjectId(InObjectId)
{
	Icon = FSlateIcon("EditorStyle", "AnimGraph.Attribute.InertialBlending.Icon", "AnimGraph.Attribute.InertialBlending.Icon");
}

TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> FInertializationsTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(Children.GetData()), Children.Num());
}

bool FInertializationsTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FInertializationsTrack::UpdateInternal);
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();

	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();

	bool bChanged = false;
	
	// convert time range to from rewind debugger times to profiler times
	TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	double StartTime = TraceTimeRange.GetLowerBoundValue();
	double EndTime = TraceTimeRange.GetUpperBoundValue();
	
	// count number of unique inertialization nodes in the current time range
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if (AnimationProvider)
	{
		TArray<TPair<int32,EInertializationType>, TInlineAllocator<8>> Nodes;
		{
			TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
			AnimationProvider->EnumerateInertializationNodes(ObjectId,  [&Nodes] (uint32 NodeId, EInertializationType Type)
			{
				Nodes.Add({ NodeId, Type });
			});
		}

		if (Children.Num() != Nodes.Num())
		{
			bChanged = true;
		}

		Children.SetNum(Nodes.Num());
		for(int32 NodeIdx = 0; NodeIdx < Nodes.Num(); NodeIdx++)
		{
			if (!Children[NodeIdx].IsValid() || Children[NodeIdx]->GetNodeId() != Nodes[NodeIdx].Key)
			{
				Children[NodeIdx] = MakeShared<FInertializationTrack>(ObjectId, Nodes[NodeIdx].Key,
					Nodes[NodeIdx].Value == EInertializationType::DeadBlending ? LOCTEXT("DeadBlending","DeadBlending") : LOCTEXT("Inertialization","Inertialization"));
				bChanged = true;
			}

			if (Children[NodeIdx]->Update())
			{
				bChanged = true;
			}
		}
	}

	return bChanged;
}

TSharedPtr<SWidget> FInertializationsTrack::GetDetailsViewInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	return SNew(SInertializationDetailsView, ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession())
				.CurrentTime_Lambda([RewindDebugger]{ return RewindDebugger->CurrentTraceTime(); });
}


FInertializationTrack::FInertializationTrack(uint64 InObjectId, int32 InNodeId, const FText& Name)
	: NodeId(InNodeId)
	, CurveName(Name)
	, ObjectId(InObjectId)
{
	Icon = FSlateIcon("EditorStyle", "AnimGraph.Attribute.InertialBlending.Icon", "AnimGraph.Attribute.InertialBlending.Icon");
	SetIsExpanded(false);
}

TSharedPtr<SCurveTimelineView::FTimelineCurveData> FInertializationTrack::GetCurveData() const
{
	if (!CurveData.IsValid())
	{
		CurveData = MakeShared<SCurveTimelineView::FTimelineCurveData>();
	}
	
	CurvesUpdateRequested++;
	
	return CurveData;
}

bool FInertializationTrack::UpdateInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	// compute curve points
	//
	TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	double StartTime = TraceTimeRange.GetLowerBoundValue();
	double EndTime = TraceTimeRange.GetUpperBoundValue();

	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if (CurvesUpdateRequested > 10 && AnimationProvider)
	{
		auto& CurvePoints = CurveData->Points;
		CurvePoints.SetNum(0,EAllowShrinking::No);
		
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadInertializationTimeline(ObjectId, [AnalysisSession, StartTime, EndTime, &CurvePoints, this](const IAnimationProvider::InertializationTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(StartTime, EndTime, [&CurvePoints, AnalysisSession, this](double InStartTime, double InEndTime, uint32 InDepth, const FInertializationMessage& InMessage)
			{
				if (InMessage.NodeId == NodeId)
				{
					CurvePoints.Add({ InMessage.RecordingTime,InMessage.Weight });
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
		
		CurvesUpdateRequested = 0;
	}

	return false;
}

static FLinearColor MakeInertializationCurveColor(uint32 InSeed, bool bInLine = false)
{
	FRandomStream Stream(InSeed);
	const uint8 Hue = (uint8)(Stream.FRand() * 255.0f);
	const uint8 SatVal = bInLine ? 196 : 128;
	return FLinearColor::MakeFromHSV8(Hue, SatVal, SatVal);
}

TSharedPtr<SWidget> FInertializationTrack::GetTimelineViewInternal()
{
	FLinearColor Color;
	Color = MakeInertializationCurveColor(CityHash32(reinterpret_cast<char*>(&NodeId), 8));
	Color.A = 0.5f;

	FLinearColor CurveColor = Color;
	CurveColor.R *= 0.5;
	CurveColor.G *= 0.5;
	CurveColor.B *= 0.5;

	TSharedPtr<SCurveTimelineView> CurveTimelineView = SNew(SCurveTimelineView)
		.TrackName(GetDisplayNameInternal())
		.FillColor(Color)
		.CurveColor(CurveColor)
		.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
		.RenderFill(true)
		.CurveData_Raw(this, &FInertializationTrack::GetCurveData);

	CurveTimelineView->SetFixedRange(0, 1);

	return CurveTimelineView;
}

TSharedPtr<SWidget> FInertializationTrack::GetDetailsViewInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	TSharedPtr<SInertializationDetailsView> InertializationDetailsView = SNew(SInertializationDetailsView, ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession())
				.CurrentTime_Lambda([RewindDebugger]{ return RewindDebugger->CurrentTraceTime(); });
	InertializationDetailsView->SetFilter(NodeId);
	return InertializationDetailsView;
}

bool FInertializationTrack::HandleDoubleClickInternal()
{

#if WITH_EDITOR
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
	{
		if (const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName))
		{
			FString AnimInstanceClassPathName;
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
				if (const FObjectInfo* AnimInstanceInfo = GameplayProvider->FindObjectInfo(ObjectId))
				{
					if (const FClassInfo* AnimInstanceClassInfo = GameplayProvider->FindClassInfo(AnimInstanceInfo->ClassId))
					{
						AnimInstanceClassPathName = AnimInstanceClassInfo->PathName;
					}
				}
			}

			if (!AnimInstanceClassPathName.IsEmpty())
			{
				TSoftObjectPtr<UAnimBlueprintGeneratedClass> InstanceClass;
				InstanceClass = FSoftObjectPath(AnimInstanceClassPathName);

				if (InstanceClass.LoadSynchronous())
				{
					if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(InstanceClass.Get()->ClassGeneratedBy))
					{
						GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AnimBlueprint);

						if (UObject* SelectedInstance = FObjectTrace::GetObjectFromId(ObjectId))
						{
							AnimBlueprint->SetObjectBeingDebugged(SelectedInstance);
						}

						if (IAnimationBlueprintEditor* AnimBlueprintEditor = static_cast<IAnimationBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AnimBlueprint, true)))
						{
							int32 AnimNodeIndex = InstanceClass.Get()->GetAnimNodeProperties().Num() - NodeId - 1;
							TWeakObjectPtr<const UEdGraphNode>* GraphNode = InstanceClass.Get()->AnimBlueprintDebugData.NodePropertyIndexToNodeMap.Find(AnimNodeIndex);
							if (GraphNode != nullptr && GraphNode->Get())
							{
								AnimBlueprintEditor->JumpToHyperlink(GraphNode->Get());
							}
						}

						return true;
					}
				}
			}
		}
	}
#endif

	return false;
}

#if WITH_ENGINE
FName FInertializationsTrackCreator::GetTargetTypeNameInternal() const
{
	return UAnimInstance::StaticClass()->GetFName();
}

static const FName InertializationsName("Inertializations");
FName FInertializationsTrackCreator::GetNameInternal() const
{
	return InertializationsName;
}

void FInertializationsTrackCreator::GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const 
{
	Types.Add({InertializationsName, LOCTEXT("Inertializations", "Inertializations")});
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FInertializationsTrackCreator::CreateTrackInternal(const FObjectId& InObjectId) const
{
 	return MakeShared<RewindDebugger::FInertializationsTrack>(InObjectId.GetMainId());
}

bool FInertializationsTrackCreator::HasDebugInfoInternal(const FObjectId& InObjectId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FInertializationsTrack::HasDebugInfoInternal);
	const TraceServices::IAnalysisSession* AnalysisSession = IRewindDebugger::Instance()->GetAnalysisSession();
	if (const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName))
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
		if (AnimationProvider->ReadInertializationTimeline(InObjectId.GetMainId(), [](const IAnimationProvider::InertializationTimeline& InTimeline){}))
		{
			return true;
		}
	}

	return false;
}
#endif // WITH_ENGINE

}
#undef LOCTEXT_NAMESPACE
