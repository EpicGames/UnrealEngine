// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAnimationProvider.h"
#include "IRewindDebugger.h"
#include "RewindDebuggerTrack.h"
#include "SCurveTimelineView.h"
#include "Textures/SlateIcon.h"
#include "IRewindDebuggerTrackCreator.h"

namespace RewindDebugger
{

class FInertializationTrack : public FRewindDebuggerTrack
{
public:
	
	FInertializationTrack(uint64 InObjectId, int32 NodeId, const FText& Name);
	int32 GetNodeId() const { return NodeId; }

private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	
	virtual FName GetNameInternal() const override { return "Inertialization"; }
	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FText GetDisplayNameInternal() const override { return CurveName; }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }
	virtual bool HandleDoubleClickInternal() override;

	TSharedPtr<SCurveTimelineView::FTimelineCurveData> GetCurveData() const;
	
	int32 NodeId = INDEX_NONE;

	FSlateIcon Icon;
	FText CurveName;
	uint64 ObjectId = INDEX_NONE;

	mutable TSharedPtr<SCurveTimelineView::FTimelineCurveData> CurveData;
	mutable int CurvesUpdateRequested = 0;
};

class FInertializationsTrack : public FRewindDebuggerTrack
{
public:
	FInertializationsTrack(uint64 InObjectId);

private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;

	virtual FName GetNameInternal() const override { return "Inertializations"; }
	virtual FText GetDisplayNameInternal() const override { return NSLOCTEXT("RewindDebugger", "Inertializations Track Name", "Inertializations"); }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }

	virtual FSlateIcon GetIconInternal() override { return Icon; }

	FSlateIcon Icon;
	uint64 ObjectId = INDEX_NONE;
	TArray<TSharedPtr<FInertializationTrack>> Children;
};

#if WITH_ENGINE
class FInertializationsTrackCreator : public IRewindDebuggerTrackCreator
{
public:
	virtual FName GetTargetTypeNameInternal() const;
	virtual FName GetNameInternal() const override;
	virtual void GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<FRewindDebuggerTrack> CreateTrackInternal(const FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const FObjectId& InObjectId) const override;
};
#endif // WITH_ENGINE

}