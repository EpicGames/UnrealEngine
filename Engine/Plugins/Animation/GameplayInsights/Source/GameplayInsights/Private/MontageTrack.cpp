// Copyright Epic Games, Inc. All Rights Reserved.

#include "MontageTrack.h"
#include "Common/ProviderLock.h"
#include "GameplayProvider.h"
#include "AnimationProvider.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "AnimationSharedData.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "Templates/Invoke.h"
#include "Modules/ModuleManager.h"
#include "Insights/ViewModels/GraphTrackBuilder.h"
#include "VariantTreeNode.h"
#include "TraceServices/Model/Frames.h"

#define LOCTEXT_NAMESPACE "MontageTrack"

INSIGHTS_IMPLEMENT_RTTI(FMontageTrack)

FMontageTrack::FMontageTrack(const FAnimationSharedData& InSharedData, uint64 InObjectID, const TCHAR* InName)
	: FGameplayGraphTrack(InSharedData.GetGameplaySharedData(), InObjectID, FText::Format(LOCTEXT("TrackNameFormat", "Montage - {0}"), FText::FromString(FString(InName))))
	, SharedData(InSharedData)
{
	EnableOptions(ShowLabelsOption);
	Layout = EGameplayGraphLayout::Stack;
}

void FMontageTrack::AddAllSeries()
{
	struct FSeriesDescription
	{
		FText Name;
		FText Description;
		FMontageSeries::ESeriesType Type;
		bool bEnabled;
	};

	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
	const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);

	if(AnimationProvider && GameplayProvider)
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);

		AnimationProvider->EnumerateMontageIds(GetGameplayTrack().GetObjectId(), [this, &AnimationProvider, &GameplayProvider](uint64 InMontageId)
		{
			if(!AllSeries.ContainsByPredicate([InMontageId](const TSharedPtr<FGraphSeries>& InSeries){ return StaticCastSharedPtr<FMontageSeries>(InSeries)->MontageId == InMontageId; }))
			{
				static const FSeriesDescription SeriesDescriptions[] =
				{
					{
						LOCTEXT("SeriesNameWeight", "Weight"),
						LOCTEXT("SeriesDescWeight", "The final effective weight that this montage was played at"),
						FMontageSeries::ESeriesType::Weight,
						true
					},
					{
						LOCTEXT("SeriesNamePosition", "Position"),
						LOCTEXT("SeriesDescPosition", "The playback position of this montage"),
						FMontageSeries::ESeriesType::Position,
						false
					},
				};
				auto MakeCurveSeriesColor = [](uint32 Hash, bool bInLine)
				{
					FRandomStream Stream(Hash);
					const uint8 Hue = (uint8)Stream.RandHelper(255);
					const uint8 SatVal = bInLine ? 196 : 128;
					return FLinearColor::MakeFromHSV8(Hue, SatVal, SatVal);
				};

				const FObjectInfo& MontageInfo = GameplayProvider->GetObjectInfo(InMontageId);

				auto AddSeries = [this, GameplayProvider, MakeCurveSeriesColor](const FSeriesDescription& InSeriesDescription, const TCHAR* InAssetName, uint64 InMontageId)
				{
					FString SeriesName = FText::Format(LOCTEXT("SeriesNameFormat", "{0} - {1}"), FText::FromString(InAssetName), InSeriesDescription.Name).ToString();

					TSharedRef<FMontageSeries> Series = MakeShared<FMontageSeries>();

					Series->SetName(SeriesName);
					Series->SetDescription(InSeriesDescription.Description.ToString());
					uint32 Hash = GetTypeHash(SeriesName) % 255;
					const FLinearColor LineColor = MakeCurveSeriesColor(Hash, true);
					const FLinearColor FillColor = MakeCurveSeriesColor(Hash, false);
					Series->SetColor(LineColor, LineColor, FillColor);

					Series->MontageId = InMontageId;
					Series->Type = InSeriesDescription.Type;
					Series->SetVisibility(true);
					Series->SetBaselineY(25.0f);
					Series->SetScaleY(20.0f);
					Series->EnableAutoZoom();
					AllSeries.Add(Series);
				};

				for (const FSeriesDescription& SeriesDescription : SeriesDescriptions)
				{
					AddSeries(SeriesDescription, MontageInfo.Name, InMontageId);
				}
			}
		});
	}
}

template<typename ProjectionType>
bool FMontageTrack::UpdateSeriesBoundsHelper(FMontageSeries& InSeries, const FTimingTrackViewport& InViewport, ProjectionType Projection)
{
	bool bFoundEvents = false;

	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if (AnimationProvider)
	{
		InSeries.CurrentMin = 0.0;
		InSeries.CurrentMax = 0.0;

		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadMontageTimeline(GetGameplayTrack().GetObjectId(), [&bFoundEvents, &InViewport, &InSeries, &Projection](const FAnimationProvider::AnimMontageTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(InViewport.GetStartTime(), InViewport.GetEndTime(), [&bFoundEvents, &InSeries, &Projection](double InStartTime, double InEndTime, uint32 InDepth, const FAnimMontageMessage& InMessage)
			{
				if (InMessage.MontageId == InSeries.MontageId )
				{
					const float Value = Invoke(Projection, InMessage);
					InSeries.CurrentMin = FMath::Min(InSeries.CurrentMin, Value);
					InSeries.CurrentMax = FMath::Max(InSeries.CurrentMax, Value);
					bFoundEvents = true;
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}

	return bFoundEvents;
}


bool FMontageTrack::UpdateSeriesBounds(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport)
{
	FMontageSeries& MontageSeries = *static_cast<FMontageSeries*>(&InSeries);
	switch (MontageSeries.Type)
	{
	case FMontageSeries::ESeriesType::Weight:
		return UpdateSeriesBoundsHelper(MontageSeries, InViewport, &FAnimMontageMessage::Weight);
	case FMontageSeries::ESeriesType::Position:
		return UpdateSeriesBoundsHelper(MontageSeries, InViewport, &FAnimMontageMessage::Position);
	}
	return false;
}

template<typename ProjectionType>
void FMontageTrack::UpdateSeriesHelper(FMontageSeries& InSeries, const FTimingTrackViewport& InViewport, ProjectionType Projection)
{
	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if (AnimationProvider)
	{
		FGraphTrackBuilder Builder(*this, InSeries, InViewport);

		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadMontageTimeline(GetGameplayTrack().GetObjectId(), [this, &AnimationProvider, &Builder, &InViewport, &InSeries, &Projection](const FAnimationProvider::AnimMontageTimeline& InTimeline)
		{
			uint16 FrameCounter = 0;
			uint16 LastFrameWithMontage = 0;

			InTimeline.EnumerateEvents(InViewport.GetStartTime(), InViewport.GetEndTime(), [this, &FrameCounter, &LastFrameWithMontage, &Builder, &InSeries, &Projection](double InStartTime, double InEndTime, uint32 InDepth, const FAnimMontageMessage& InMessage)
			{
				FrameCounter = InMessage.FrameCounter;

				if (InMessage.MontageId == InSeries.MontageId)
				{
					Builder.AddEvent(InStartTime, InEndTime - InStartTime, Invoke(Projection, InMessage), LastFrameWithMontage == FrameCounter - 1);

					LastFrameWithMontage = InMessage.FrameCounter;
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}

void FMontageTrack::UpdateSeries(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport)
{
	FMontageSeries& MontageSeries = *static_cast<FMontageSeries*>(&InSeries);
	switch (MontageSeries.Type)
	{
	case FMontageSeries::ESeriesType::Weight:
		UpdateSeriesHelper(MontageSeries, InViewport, &FAnimMontageMessage::Weight);
		break;
	case FMontageSeries::ESeriesType::Position:
		UpdateSeriesHelper(MontageSeries, InViewport, &FAnimMontageMessage::Position);
		break;
	}
}

void FMontageTrack::InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const
{
	const FGraphTrackEvent& GraphTrackEvent = *static_cast<const FGraphTrackEvent*>(&HoveredTimingEvent);

	FTimingEventSearchParameters SearchParameters(HoveredTimingEvent.GetStartTime(), HoveredTimingEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch);

	FindMontageMessage(SearchParameters, [this, &Tooltip, &GraphTrackEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FAnimMontageMessage& InMessage)
	{
		Tooltip.ResetContent();

		Tooltip.AddTitle(GetName());

		Tooltip.AddNameValueTextLine(LOCTEXT("EventTime", "Time").ToString(), FText::AsNumber(InFoundStartTime).ToString());
		Tooltip.AddNameValueTextLine(LOCTEXT("EventWeight", "Weight").ToString(), FText::AsNumber(GraphTrackEvent.GetValue()).ToString());
		Tooltip.AddNameValueTextLine(LOCTEXT("EventDesiredWeight", "Desired Weight").ToString(), FText::AsNumber(InMessage.DesiredWeight).ToString());
		Tooltip.AddNameValueTextLine(LOCTEXT("EventPosition", "Position").ToString(), FText::AsNumber(InMessage.Position).ToString()); 

		const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
		if(GameplayProvider)
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			const FObjectInfo& MontageInfo = GameplayProvider->GetObjectInfo(InMessage.MontageId);
			Tooltip.AddNameValueTextLine(LOCTEXT("MontageName", "Montage").ToString(), MontageInfo.PathName);
		}

		const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
		if (AnimationProvider)
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
			const TCHAR* CurrentSectionName = AnimationProvider->GetName(InMessage.CurrentSectionNameId);
			Tooltip.AddNameValueTextLine(LOCTEXT("CurrentSectionName", "Current Section").ToString(), FText::FromString(CurrentSectionName).ToString());
			const TCHAR* NextSectionName = AnimationProvider->GetName(InMessage.NextSectionNameId);
			Tooltip.AddNameValueTextLine(LOCTEXT("NextSectionName", "Next Section").ToString(), FText::FromString(NextSectionName).ToString());
		}

		Tooltip.AddNameValueTextLine(LOCTEXT("EventWorld", "World").ToString(), GetGameplayTrack().GetWorldName(SharedData.GetAnalysisSession()).ToString());

		Tooltip.UpdateLayout();
	});
}

const TSharedPtr<const ITimingEvent> FMontageTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	FindMontageMessage(InSearchParameters, [this, &FoundEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FAnimMontageMessage& InFoundMessage)
	{
		FoundEvent = MakeShared<const FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth);
	});

	return FoundEvent;
}

void FMontageTrack::FindMontageMessage(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FAnimMontageMessage&)> InFoundPredicate) const
{
	TTimingEventSearch<FAnimMontageMessage>::Search(
		InParameters,

		[this](TTimingEventSearch<FAnimMontageMessage>::FContext& InContext)
		{
			const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

			if(AnimationProvider)
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);

				AnimationProvider->ReadMontageTimeline(GetGameplayTrack().GetObjectId(), [&InContext](const FAnimationProvider::AnimMontageTimeline& InTimeline)
				{
					InTimeline.EnumerateEvents(InContext.GetParameters().StartTime, InContext.GetParameters().EndTime, [&InContext](double InEventStartTime, double InEventEndTime, uint32 InDepth, const FAnimMontageMessage& InMessage)
					{
						InContext.Check(InEventStartTime, InEventEndTime, 0, InMessage);
						return TraceServices::EEventEnumerate::Continue;
					});
				});
			}
		},

		[&InParameters](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FAnimMontageMessage& InEvent)
		{
			// Match the start time exactly here
			return InFoundStartTime == InParameters.StartTime;
		},

		[&InFoundPredicate](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FAnimMontageMessage& InEvent)
		{
			InFoundPredicate(InFoundStartTime, InFoundEndTime, InFoundDepth, InEvent);
		},
		
		TTimingEventSearch<FAnimMontageMessage>::NoMatch);
}

void FMontageTrack::GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const 
{
	TSharedRef<FVariantTreeNode> Header = OutVariants.Add_GetRef(FVariantTreeNode::MakeHeader(LOCTEXT("MontagesHeader", "Montages"), 0));

	const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(SharedData.GetAnalysisSession());
	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
	const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);

	if(AnimationProvider && GameplayProvider)
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);

		AnimationProvider->ReadMontageTimeline(GetGameplayTrack().GetObjectId(), [this, &InFrame, &AnimationProvider, &GameplayProvider, &Header](const FAnimationProvider::AnimMontageTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, [this, &AnimationProvider, &GameplayProvider, &Header, &InFrame](double InStartTime, double InEndTime, uint32 InDepth, const FAnimMontageMessage& InMessage)
			{
				if(InStartTime >= InFrame.StartTime && InEndTime <= InFrame.EndTime)
				{
					const FObjectInfo& MontageInfo = GameplayProvider->GetObjectInfo(InMessage.MontageId);
					TSharedRef<FVariantTreeNode> MontageHeader = Header->AddChild(FVariantTreeNode::MakeObject(FText::FromString(MontageInfo.Name), InMessage.MontageId));

					MontageHeader->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("EventWeight", "Weight"), InMessage.Weight));
					MontageHeader->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("EventDesiredWeight", "Desired Weight"), InMessage.DesiredWeight));

					const TCHAR* CurrentSectionName = AnimationProvider->GetName(InMessage.CurrentSectionNameId);
					MontageHeader->AddChild(FVariantTreeNode::MakeString(LOCTEXT("CurrentSectionName", "Current Section"), CurrentSectionName));
					const TCHAR* NextSectionName = AnimationProvider->GetName(InMessage.NextSectionNameId);
					MontageHeader->AddChild(FVariantTreeNode::MakeString(LOCTEXT("NextSectionName", "Next Section"), NextSectionName));
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}

#undef LOCTEXT_NAMESPACE
