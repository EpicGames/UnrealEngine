// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayGraphTrack.h"

class FAnimationSharedData;
class FTimingEventSearchParameters;
struct FAnimMontageMessage;
class FMontageTrack;

class FMontageSeries : public FGameplayGraphSeries
{
public:
	enum class ESeriesType : uint32
	{
		Weight,
		Position,

		Count,
	};

	uint64 MontageId;
	ESeriesType Type;
};

class FMontageTrack : public FGameplayGraphTrack
{
	INSIGHTS_DECLARE_RTTI(FMontageTrack, FGameplayGraphTrack)

public:
	FMontageTrack(const FAnimationSharedData& InSharedData, uint64 InObjectID, const TCHAR* InName);

	virtual void InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const override;
	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;
	virtual void AddAllSeries() override;
	virtual bool UpdateSeriesBounds(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport) override;
	virtual void UpdateSeries(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport) override;
	virtual void GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const override;

private:
	// Helper function used to find a montage message
	void FindMontageMessage(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FAnimMontageMessage&)> InFoundPredicate) const;

	// Helper function for series bounds update
	template<typename ProjectionType>
	bool UpdateSeriesBoundsHelper(FMontageSeries& InSeries, const FTimingTrackViewport & InViewport, ProjectionType Projection);

	// Helper function for series update
	template<typename ProjectionType>
	void UpdateSeriesHelper(FMontageSeries& InSeries, const FTimingTrackViewport & InViewport, ProjectionType Projection);

private:
	/** The shared data */
	const FAnimationSharedData& SharedData;
};
