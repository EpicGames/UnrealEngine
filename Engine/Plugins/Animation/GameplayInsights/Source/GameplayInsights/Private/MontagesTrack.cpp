// Copyright Epic Games, Inc. All Rights Reserved.

#include "MontagesTrack.h"
#include "Common/ProviderLock.h"
#include "SCurveTimelineView.h"
#include "IRewindDebugger.h"
#include "GameplayProvider.h"
#include "AnimationProvider.h"
#include "Styling/SlateIconFinder.h"
#include "SMontageView.h"
#if WITH_EDITOR
#include "AnimPreviewInstance.h"
#include "Animation/AnimMontage.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Editor.h"
#include "IAnimationEditor.h"
#include "IPersonaToolkit.h"
#include "Subsystems/AssetEditorSubsystem.h"
#endif

#define LOCTEXT_NAMESPACE "MontagesTrack"

namespace RewindDebugger
{

FMontagesTrack::FMontagesTrack(uint64 InObjectId) : ObjectId(InObjectId)
{
#if WITH_EDITOR
	Icon = FSlateIconFinder::FindIconForClass(UAnimMontage::StaticClass());
#endif
}

TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> FMontagesTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(Children.GetData()), Children.Num());
}

bool FMontagesTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMontagesTrack::UpdateInternal);
	TArray<uint64> UniqueTrackIds;

	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();

	bool bChanged = false;
	
	// convert time range to from rewind debugger times to profiler times
	TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	double StartTime = TraceTimeRange.GetLowerBoundValue();
	double EndTime = TraceTimeRange.GetUpperBoundValue();
	
	// count number of unique animations in the current time range
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	int AnimationCount = 0;

	if (AnimationProvider)
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		UniqueTrackIds.SetNum(0, EAllowShrinking::No);

		AnimationProvider->ReadMontageTimeline(ObjectId, [&UniqueTrackIds, StartTime, EndTime](const FAnimationProvider::AnimMontageTimeline& InTimeline)
		{
			// this isn't very efficient, and it gets called every frame.  will need optimizing
			InTimeline.EnumerateEvents(StartTime, EndTime, [&UniqueTrackIds, StartTime, EndTime](double InStartTime, double InEndTime, uint32 InDepth, const FAnimMontageMessage& InMessage)
			{
				if (InEndTime > StartTime && InStartTime < EndTime)
				{
					UniqueTrackIds.AddUnique(InMessage.MontageId);
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});

		UniqueTrackIds.StableSort();
		
		AnimationCount = UniqueTrackIds.Num();

		if (Children.Num()!=AnimationCount)
			bChanged = true;
		
		Children.SetNum(AnimationCount);
		for(int i = 0; i < AnimationCount; i++)
		{
			if (!Children[i].IsValid() || Children[i].Get()->GetAssetId() != UniqueTrackIds[i])
			{
				Children[i] = MakeShared<FMontageTrack>(ObjectId, UniqueTrackIds[i]);
				bChanged = true;
			}

			if (Children[i]->Update())
			{
				bChanged = true;
			}
		}
	}

	return bChanged;
}

TSharedPtr<SWidget> FMontagesTrack::GetDetailsViewInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	return SNew(SMontageView, ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession())
		.CurrentTime_Lambda([RewindDebugger]{ return RewindDebugger->CurrentTraceTime(); });
}


FMontageTrack::FMontageTrack(uint64 InObjectId, uint64 InAssetId, FMontageTrack::ECurveType InCurveType) :
	 AssetId(InAssetId)
	, CurveType(InCurveType)
	, ObjectId(InObjectId)
{
	SetIsExpanded(false);
}

TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> FMontageTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(Children.GetData()), Children.Num());
}

TSharedPtr<SCurveTimelineView::FTimelineCurveData> FMontageTrack::GetCurveData() const
{
	if (!CurveData.IsValid())
	{
		CurveData = MakeShared<SCurveTimelineView::FTimelineCurveData>();
	}
	
	CurvesUpdateRequested++;
	
	return CurveData;
}

bool FMontageTrack::UpdateInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	// compute curve points
	//
	TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	double StartTime = TraceTimeRange.GetLowerBoundValue();
	double EndTime = TraceTimeRange.GetUpperBoundValue();

	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	int AnimationCount = 0;

	if(CurvesUpdateRequested > 10 && GameplayProvider && AnimationProvider)
	{
		auto& CurvePoints = CurveData->Points;
		CurvePoints.SetNum(0,EAllowShrinking::No);
		
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadMontageTimeline(ObjectId, [AnalysisSession, StartTime, EndTime, &CurvePoints, this](const FAnimationProvider::AnimMontageTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(StartTime, EndTime, [&CurvePoints, AnalysisSession, this](double InStartTime, double InEndTime, uint32 InDepth, const FAnimMontageMessage& InMessage)
			{
				if (InMessage.MontageId == AssetId)
				{
					float Weight = 0;
					switch (CurveType)
					{
						case ECurveType::BlendWeight:					Weight = InMessage.Weight; break;
						case ECurveType::DesiredWeight:					Weight = InMessage.DesiredWeight; break; 
						case ECurveType::Position:						Weight = InMessage.Position; break; 
					}
					
					CurvePoints.Add({ InMessage.RecordingTime,	Weight });
				}
								
				return TraceServices::EEventEnumerate::Continue;
			});
		});

		CurvesUpdateRequested = 0;
	}

	// update Icon:
	
	bool bChanged = false;

	if (CurveType == ECurveType::BlendWeight)
	{
		// Blend Weight track gets name/icon of animation
		if (CurveName.IsEmpty() && GameplayProvider)
		{
			TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
			if (const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(AssetId))
			{
				CurveName = FText::FromString(ObjectInfo->Name);
				bChanged = true;

				if (const UStruct* FoundType = GameplayProvider->FindType(ObjectInfo->ClassId))
				{
					Icon = FSlateIconFinder::FindIconForClass(FoundType);
				}

				if (Children.Num() == 0)
				{
					for (int i = static_cast<int>(ECurveType::BlendWeight) + 1; i <= static_cast<int>(ECurveType::Position); i++)
					{
						Children.Add(MakeShared<FMontageTrack>(ObjectId, AssetId, static_cast<ECurveType>(i)));
					}
				}
			}
		}
	}
	else
	{
		// other tracks get the curve name

		if (CurveName.IsEmpty())
		{
			switch (CurveType)
			{
				case ECurveType::BlendWeight:					break;
				case ECurveType::DesiredWeight:					CurveName = LOCTEXT("Desired Weight", "Desired Weight"); break;
				case ECurveType::Position:						CurveName = LOCTEXT("Position", "Position"); break;
			}

			Icon = FSlateIcon("EditorStyle", "AnimGraph.Attribute.Curves.Icon", "AnimGraph.Attribute.Curves.Icon");

			bChanged = true;
		}
	}

	for (auto& Child : Children)
	{
		bChanged |= Child->Update();
	}

	return bChanged;
}

static FLinearColor MakeMontageCurveColor(uint32 InSeed, bool bInLine = false)
{
	FRandomStream Stream(InSeed);
	const uint8 Hue = (uint8)(Stream.FRand() * 255.0f);
	const uint8 SatVal = bInLine ? 196 : 128;
	return FLinearColor::MakeFromHSV8(Hue, SatVal, SatVal);
}

TSharedPtr<SWidget> FMontageTrack::GetTimelineViewInternal()
{
	FLinearColor Color = FLinearColor::Black;
	switch(CurveType)
	{
	case ECurveType::BlendWeight:
		Color = MakeMontageCurveColor(CityHash32(reinterpret_cast<char*>(&AssetId), 8));
		Color.A = 0.5f;
		break;
	case ECurveType::DesiredWeight:
		Color = FLinearColor::MakeFromHSV8(0, 50, 50);
		break;
	case ECurveType::Position:
		Color = FLinearColor::MakeFromHSV8(0, 50, 50);
		break;
	}

	FLinearColor CurveColor = Color;
	CurveColor.R *= 0.5;
	CurveColor.G *= 0.5;
	CurveColor.B *= 0.5;

	TSharedPtr<SCurveTimelineView> CurveTimelineView = SNew(SCurveTimelineView)
		.FillColor(Color)
		.CurveColor(CurveColor)
		.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
		.RenderFill(CurveType == ECurveType::BlendWeight)
		.CurveData_Raw(this, &FMontageTrack::GetCurveData);

	if (CurveType == ECurveType::BlendWeight || CurveType == ECurveType::DesiredWeight)
	{
		CurveTimelineView->SetFixedRange(0, 1);
	}

	return CurveTimelineView;
}

TSharedPtr<SWidget> FMontageTrack::GetDetailsViewInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	TSharedPtr<SMontageView> MontageView = SNew(SMontageView, ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession())
			.CurrentTime_Lambda([RewindDebugger]{ return RewindDebugger->CurrentTraceTime(); });
	MontageView->SetAssetFilter(AssetId);
	return MontageView;
}
	
bool FMontageTrack::HandleDoubleClickInternal()
{
#if WITH_EDITOR
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
	{
		const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
		const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

		if (!GameplayProvider || !AnimationProvider)
		{
			return false;
		}

		bool bMessageFound = false;
		float PlaybackTime = 0;

		FString AssetPathName;
		{
			TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
			const FObjectInfo& AssetInfo = GameplayProvider->GetObjectInfo(GetAssetId());
			AssetPathName = AssetInfo.PathName;
		}

		const float CurrentTraceTime = RewindDebugger->CurrentTraceTime();

		bool bFrameFound;
		TraceServices::FFrame Frame;
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
			const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
			bFrameFound = FrameProvider.GetFrameFromTime(TraceFrameType_Game, CurrentTraceTime, Frame);
		}

		if (bFrameFound)
		{
			TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
			AnimationProvider->ReadMontageTimeline(ObjectId, [this, &bMessageFound, &PlaybackTime, &Frame](const FAnimationProvider::AnimMontageTimeline& InTimeline)
			{
				InTimeline.EnumerateEvents(Frame.StartTime, Frame.EndTime, [this, &bMessageFound, &PlaybackTime, &Frame](double InStartTime, double InEndTime, uint32 InDepth, const FAnimMontageMessage& InMessage)
				{
					if(InStartTime >= Frame.StartTime && InEndTime <= Frame.EndTime)
					{
						if (InMessage.MontageId == AssetId)
						{
							bMessageFound = true;
							PlaybackTime = InMessage.Position;
							return TraceServices::EEventEnumerate::Stop;
						}
					}
					return TraceServices::EEventEnumerate::Continue;
				});
			});
		}

		UObject* Asset = nullptr;
		FString PackagePathString = FPackageName::ObjectPathToPackageName(AssetPathName);

		UPackage* Package = LoadPackage(NULL, ToCStr(PackagePathString), LOAD_NoRedirects);
		if (Package)
		{
			Package->FullyLoad();
                
			FString AssetName = FPaths::GetBaseFilename(AssetPathName);
			Asset = FindObject<UObject>(Package, *AssetName);
		}
		else
		{
			// fallback for unsaved assets
			Asset = FindObject<UObject>(nullptr, *AssetPathName);
		}
                    	
		if (Asset != nullptr)
		{
			if (UAssetEditorSubsystem* AssetEditorSS = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
			{
				AssetEditorSS->OpenEditorForAsset(Asset);

				if (bMessageFound)
				{
					// if the asset is playing on the current frame, scrub to the appropriate time
					if (IAssetEditorInstance* Editor = AssetEditorSS->FindEditorForAsset(Asset, true))
					{
						if (Editor->GetEditorName()=="AnimationEditor")
						{
							IAnimationEditor* AnimationEditor = static_cast<IAnimationEditor*>(Editor);
							UDebugSkelMeshComponent* PreviewComponent = AnimationEditor->GetPersonaToolkit()->GetPreviewMeshComponent();
							PreviewComponent->PreviewInstance->SetPosition(PlaybackTime);
							PreviewComponent->PreviewInstance->SetPlaying(false);
						}
					}
				}
			}
		}

		

		return true;
	}
#endif
	return false;
}

static const FName MontagesName("Montages");
FName FMontagesTrack::GetNameInternal() const
{
	return MontagesName;
}

#if WITH_ENGINE
FName FMontagesTrackCreator::GetTargetTypeNameInternal() const
{
	return UAnimInstance::StaticClass()->GetFName();
}

FName FMontagesTrackCreator::GetNameInternal() const
{
	return MontagesName;
}
	
void FMontagesTrackCreator::GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const 
{
	Types.Add({MontagesName, LOCTEXT("Montages", "Montages")});
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FMontagesTrackCreator::CreateTrackInternal(const FObjectId& InObjectId) const
{
 	return MakeShared<RewindDebugger::FMontagesTrack>(InObjectId.GetMainId());
}

bool FMontagesTrackCreator::HasDebugInfoInternal(const FObjectId& InObjectId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMontagesTrack::HasDebugInfoInternal);
	const TraceServices::IAnalysisSession* AnalysisSession = IRewindDebugger::Instance()->GetAnalysisSession();
	bool bHasData = false;
	if (const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName))
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadMontageTimeline(InObjectId.GetMainId(), [&bHasData](const FAnimationProvider::AnimMontageTimeline& InTimeline)
		{
			bHasData = true;
		});
	}
	
	return bHasData;
}
#endif // WITH_ENGINE

}
#undef LOCTEXT_NAMESPACE
