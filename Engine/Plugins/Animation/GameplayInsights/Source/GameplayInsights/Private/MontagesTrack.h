// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAnimationProvider.h"
#include "IRewindDebugger.h"
#include "RewindDebuggerTrack.h"
#include "SCurveTimelineView.h"
#include "Textures/SlateIcon.h"
#include "IRewindDebuggerTrackCreator.h"

namespace RewindDebugger
{

class FMontageTrack : public FRewindDebuggerTrack
{
public:
	enum class ECurveType
	{
		BlendWeight,
		DesiredWeight,
		Position
	};
	
	FMontageTrack(uint64 InObjectId, uint64 AssetId, ECurveType CurveType = ECurveType::BlendWeight);

	uint64 GetAssetId() const { return AssetId; }

private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;
	virtual bool HandleDoubleClickInternal() override;

	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FName GetNameInternal() const override { return "Montage"; }
	virtual FText GetDisplayNameInternal() const override { return CurveName; }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }

	TSharedPtr<SCurveTimelineView::FTimelineCurveData> GetCurveData() const;
	
	uint64 AssetId;

	ECurveType CurveType;
	
	FSlateIcon Icon;
	FText CurveName;
	uint64 ObjectId;

	mutable TSharedPtr<SCurveTimelineView::FTimelineCurveData> CurveData;
	mutable int CurvesUpdateRequested = 0;
	
	TArray<TSharedPtr<FMontageTrack>> Children;
};

class FMontagesTrack : public FRewindDebuggerTrack
{
public:
	FMontagesTrack(uint64 InObjectId);

private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;

	virtual FName GetNameInternal() const override;
	virtual FText GetDisplayNameInternal() const override { return NSLOCTEXT("RewindDebugger", "Montages Track Name", "Montages"); }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }

	virtual FSlateIcon GetIconInternal() override { return Icon; }

	FSlateIcon Icon;
	uint64 ObjectId;
	TArray<TSharedPtr<FMontageTrack>> Children;
};

#if WITH_ENGINE
class FMontagesTrackCreator : public IRewindDebuggerTrackCreator
{
public:
	virtual FName GetTargetTypeNameInternal() const;
	virtual FName GetNameInternal() const override;
	virtual void GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<FRewindDebuggerTrack> CreateTrackInternal(const FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const FObjectId& InObjectId) const override;
};
#endif // WITH_ENGINE

}