// Copyright Epic Games, Inc. All Rights Reserved.

#include "NotifiesTrack.h"
#include "Common/ProviderLock.h"
#include "SCurveTimelineView.h"
#include "IRewindDebugger.h"
#include "GameplayProvider.h"
#include "AnimationProvider.h"
#include "SNotifiesView.h"
#if WITH_ENGINE
#include "Animation/AnimInstance.h"
#endif // WITH_ENGINE

#define LOCTEXT_NAMESPACE "NotifiesTrack"

namespace RewindDebugger
{

FNotifyTrack::FNotifyTrack(uint64 InObjectId, const FNotifyTrack::FNotifyTrackId& InNotifyTrackId) :
	ObjectId(InObjectId),
	NotifyTrackId(InNotifyTrackId)
{
	EventData = MakeShared<SEventTimelineView::FTimelineEventData>();
	Icon = FSlateIcon("EditorStyle", "Sequencer.Tracks.Event", "Sequencer.Tracks.Event");
}

TSharedPtr<SEventTimelineView::FTimelineEventData> FNotifyTrack::GetEventData() const
{
	if (!EventData.IsValid())
	{
		EventData = MakeShared<SEventTimelineView::FTimelineEventData>();
	}
	
	EventUpdateRequested++;
	
	return EventData;
}
	
static FLinearColor MakeNotifyColor(uint32 InSeed, bool bInLine = false)
{
	FRandomStream Stream(InSeed);
	const uint8 Hue = (uint8)(Stream.FRand() * 255.0f);
	const uint8 SatVal = bInLine ? 196 : 128;
	return FLinearColor::MakeFromHSV8(Hue, SatVal, SatVal);
}

static FNotifyTrack::ENotifyTrackType MakeTrackType(EAnimNotifyMessageType MessageType)
{
	switch (MessageType)
	{
		case EAnimNotifyMessageType::Begin: return FNotifyTrack::ENotifyTrackType::NotifyState;
		case EAnimNotifyMessageType::SyncMarker: return FNotifyTrack::ENotifyTrackType::SyncMarker;
		default: return FNotifyTrack::ENotifyTrackType::Notify;
	}
}
	
TSharedPtr<SWidget> FNotifyTrack::GetDetailsViewInternal() 
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	
	TSharedPtr<SNotifiesView> NotifiesView = SNew(SNotifiesView, ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession())
				.CurrentTime_Lambda([RewindDebugger]{ return RewindDebugger->CurrentTraceTime(); });
	NotifiesView->SetNotifyFilter(NotifyTrackId.NameId);
	return NotifiesView;
}

bool FNotifyTrack::UpdateInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	
	TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	double StartTime = TraceTimeRange.GetLowerBoundValue();
	double EndTime = TraceTimeRange.GetUpperBoundValue();
	
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
	
	if (EventUpdateRequested > 10 && AnimationProvider)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FNotifiesTrack::UpdateEventPointsInternal);
		EventUpdateRequested = 0;
		
		auto& EventPoints = EventData->EventPoints;
		EventPoints.SetNum(0,EAllowShrinking::No);
		EventData->EventWindows.SetNum(0);

		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);

		AnimationProvider->ReadNotifyTimeline(ObjectId, [this, StartTime, EndTime, AnimationProvider, AnalysisSession, &EventPoints](const FAnimationProvider::AnimNotifyTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(StartTime, EndTime, [this, StartTime, EndTime, AnimationProvider, AnalysisSession, &EventPoints](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNotifyMessage& InMessage)
			{
				if (InEndTime > StartTime && InStartTime < EndTime)
				{
					if (InMessage.NameId == NotifyTrackId.NameId && MakeTrackType(InMessage.NotifyEventType) == NotifyTrackId.Type)
					{
						SEventTimelineView::FTimelineEventData::FEventPoint Point;
						Point.Time = InMessage.RecordingTime;
						Point.Color = MakeNotifyColor(InMessage.NameId);
						EventPoints.Add(Point);
					}
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});

		AnimationProvider->EnumerateNotifyStateTimelines(ObjectId, [this, StartTime, EndTime, AnimationProvider, AnalysisSession, &EventPoints](uint64 InNotifyId, const FAnimationProvider::AnimNotifyTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(StartTime, EndTime, [this, StartTime, EndTime, AnimationProvider, AnalysisSession, &EventPoints](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNotifyMessage& InMessage)
			{
				if (InEndTime > StartTime && InStartTime < EndTime)
				{
					if (InMessage.NameId == NotifyTrackId.NameId && MakeTrackType(InMessage.NotifyEventType) == NotifyTrackId.Type)
					{
						SEventTimelineView::FTimelineEventData::FEventWindow Window;
						Window.TimeStart = InMessage.RecordingTime;
						Window.TimeEnd = InMessage.RecordingTime + InMessage.Duration;
						Window.Color = MakeNotifyColor(InMessage.NameId);
						EventData->EventWindows.Add(Window);
					}
				}
				
				return TraceServices::EEventEnumerate::Continue;
			});
		});

	}

	bool bChanged = false;

	if (TrackName.IsEmpty() && AnimationProvider)
    {
    	TrackName = FText::FromString(AnimationProvider->GetName(NotifyTrackId.NameId));
    	bChanged = true;
    }

	return bChanged;
	
}

TSharedPtr<SWidget> FNotifiesTrack::GetDetailsViewInternal() 
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	TSharedPtr<SNotifiesView> NotifiesView = SNew(SNotifiesView, ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession())
				.CurrentTime_Lambda([RewindDebugger]{ return RewindDebugger->CurrentTraceTime(); });
	return NotifiesView;
}

TSharedPtr<SWidget> FNotifyTrack::GetTimelineViewInternal()
{
	return SNew(SEventTimelineView)
		.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
		.EventData_Raw(this, &FNotifyTrack::GetEventData);
}

static const FName NotifiesName("Notifies");
#if WITH_ENGINE
FName FNotifiesTrackCreator::GetTargetTypeNameInternal() const
{
	return UAnimInstance::StaticClass()->GetFName();
}
	
FName FNotifiesTrackCreator::GetNameInternal() const
{
	return NotifiesName;
}

void FNotifiesTrackCreator::GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const
{
	Types.Add({ NotifiesName, LOCTEXT("Notifies", "Notifies") });
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FNotifiesTrackCreator::CreateTrackInternal(const FObjectId& InObjectId) const
{
	return MakeShared<RewindDebugger::FNotifiesTrack>(InObjectId.GetMainId());
}
#endif // WITH_ENGINE
		
FNotifiesTrack::FNotifiesTrack(uint64 InObjectId) :
	ObjectId(InObjectId)
{
	Icon = FSlateIcon("EditorStyle", "Sequencer.Tracks.Event", "Sequencer.Tracks.Event");
}


bool FNotifiesTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNotifiesTrack::UpdateInternal);
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	
	TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	double StartTime = TraceTimeRange.GetLowerBoundValue();
	double EndTime = TraceTimeRange.GetUpperBoundValue();
	
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
	
	bool bChanged = false;
	
	if (AnimationProvider)
	{
		TArray<FNotifyTrack::FNotifyTrackId> UniqueTrackIds;
		
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);

		AnimationProvider->ReadNotifyTimeline(ObjectId, [this, StartTime, EndTime, AnimationProvider, AnalysisSession, &UniqueTrackIds](const FAnimationProvider::AnimNotifyTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(StartTime, EndTime, [this, StartTime, EndTime, AnimationProvider, AnalysisSession, &UniqueTrackIds](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNotifyMessage& InMessage)
			{
				if (InMessage.NotifyEventType != EAnimNotifyMessageType::Tick)
				{
					UniqueTrackIds.AddUnique({InMessage.NameId, MakeTrackType(InMessage.NotifyEventType)});
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});

		AnimationProvider->EnumerateNotifyStateTimelines(ObjectId, [this, StartTime, EndTime, AnimationProvider, AnalysisSession, &UniqueTrackIds](uint64 InNotifyId, const FAnimationProvider::AnimNotifyTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(StartTime, EndTime, [this, StartTime, EndTime, AnimationProvider, AnalysisSession, &UniqueTrackIds](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNotifyMessage& InMessage)
			{
				if (InEndTime > StartTime && InStartTime < EndTime)
				{
					if (InMessage.NotifyEventType == EAnimNotifyMessageType::Begin)
					{
						UniqueTrackIds.AddUnique({InMessage.NameId, MakeTrackType(InMessage.NotifyEventType)});
					}
				}
				
				return TraceServices::EEventEnumerate::Continue;
			});
		});
		
		UniqueTrackIds.StableSort();
		const int32 TrackCount = UniqueTrackIds.Num();

		if (Children.Num() != TrackCount)
			bChanged = true;

		Children.SetNum(UniqueTrackIds.Num());
		for(int i = 0; i < TrackCount; i++)
		{
			if (!Children[i].IsValid() || !(Children[i].Get()->GetNotifyTrackId() == UniqueTrackIds[i]))
			{
				Children[i] = MakeShared<FNotifyTrack>(ObjectId, UniqueTrackIds[i]);
				bChanged = true;
			}

			if (Children[i]->Update())
			{
				bChanged = true;
			}
		}
	}
	
	return bChanged;
}

TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> FNotifiesTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(Children.GetData()), Children.Num());
}

#if WITH_ENGINE
bool FNotifiesTrackCreator::HasDebugInfoInternal(const FObjectId& InObjectId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNotifiesTrack::HasDebugInfoInternal);
	const TraceServices::IAnalysisSession* AnalysisSession = IRewindDebugger::Instance()->GetAnalysisSession();

	bool bHasData = false;
	if (const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName))
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadNotifyTimeline(InObjectId.GetMainId(), [&bHasData](const FAnimationProvider::AnimNotifyTimeline& InTimeline)
		{
			bHasData = true;
		});
	}
	return bHasData;
}
#endif // WITH_ENGINE
	
}

#undef LOCTEXT_NAMESPACE