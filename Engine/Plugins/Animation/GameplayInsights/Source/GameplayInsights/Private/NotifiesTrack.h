// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAnimationProvider.h"
#include "IRewindDebuggerTrackCreator.h"
#include "RewindDebuggerTrack.h"
#include "Textures/SlateIcon.h"
#include "SEventTimelineView.h"

namespace RewindDebugger
{

class FNotifyTrack : public FRewindDebuggerTrack
{
public:
	enum class ENotifyTrackType
	{
		SyncMarker,
		Notify,
		NotifyState
	};
	
	struct FNotifyTrackId
	{
		uint64 NameId;
		ENotifyTrackType Type;

		bool operator == (const FNotifyTrackId& Other) const
		{
			return NameId == Other.NameId && Type == Other.Type;
		}

		bool operator <(const FNotifyTrack::FNotifyTrackId& Other) const
		{
			if (Type == Other.Type)
			{
				return NameId < Other.NameId; 
			}
			
			return static_cast<int>(Type) < static_cast<int>(Other.Type);
		}		
	};

	FNotifyTrack(uint64 InObjectId, const FNotifyTrackId& TrackId);
	const FNotifyTrackId& GetNotifyTrackId() const { return NotifyTrackId; }
private:
	virtual bool UpdateInternal() override;
	TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FText GetDisplayNameInternal() const override { return TrackName; }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }

	TSharedPtr<SEventTimelineView::FTimelineEventData> GetEventData() const;
	
	mutable TSharedPtr<SEventTimelineView::FTimelineEventData> EventData;
	mutable int EventUpdateRequested = 0;
	
	uint64 ObjectId;
	FNotifyTrackId NotifyTrackId;
	FSlateIcon Icon;
	FText TrackName;
	
	enum class EFilterType
	{
		Notify,
		NotifyState,
		SyncMarker,
	};

	EFilterType FilterType = EFilterType::Notify;
};



// Parent track of all the curves
class FNotifiesTrack : public FRewindDebuggerTrack
{
public:
	FNotifiesTrack(uint64 InObjectId);

private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;

	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FName GetNameInternal() const override { return "Notifies"; }
	virtual FText GetDisplayNameInternal() const override { return NSLOCTEXT("RewindDebugger", "NotifiesTrackName", "Notifies"); }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;

	FSlateIcon Icon;
	uint64 ObjectId;

	TArray<TSharedPtr<FNotifyTrack>> Children;
};

#if WITH_ENGINE
class FNotifiesTrackCreator : public IRewindDebuggerTrackCreator
{
private:
	virtual FName GetTargetTypeNameInternal() const;
	virtual FName GetNameInternal() const override;
	virtual void GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<FRewindDebuggerTrack> CreateTrackInternal(const FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const FObjectId& InObjectId) const override;
};
#endif // WITH_ENGINE

}