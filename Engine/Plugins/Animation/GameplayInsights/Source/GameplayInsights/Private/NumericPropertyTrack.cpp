// Copyright Epic Games, Inc. All Rights Reserved.

#include "NumericPropertyTrack.h"
#include "PropertyHelpers.h"

namespace RewindDebugger
{
	///////////////////////////////////////////////////////////////////////////////
	// Numeric Property Track
	///////////////////////////////////////////////////////////////////////////////
	
	FNumericPropertyTrack::FNumericPropertyTrack(uint64 InObjectId, const TSharedPtr<FObjectPropertyInfo> & InObjectProperty, const TSharedPtr<FPropertyTrack> & InParentTrack) : FPropertyTrack(InObjectId, InObjectProperty, InParentTrack)
	{
	}

	bool FNumericPropertyTrack::CanBeCreated(const FObjectPropertyInfo& InObjectProperty)
	{
		const FString ValueString = { InObjectProperty.Property.Value };
		return !ValueString.IsEmpty() && ValueString.IsNumeric();
	}
	
	TSharedPtr<SCurveTimelineView::FTimelineCurveData> FNumericPropertyTrack::GetCurveData() const
	{
		if (!CurveData.IsValid())
		{
			CurveData = MakeShared<SCurveTimelineView::FTimelineCurveData>();
		}

		CurvesUpdateRequested++;

		return CurveData;
	}
	
	bool FNumericPropertyTrack::UpdateInternal()
	{
		const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
		check(RewindDebugger)

		// Convert time range to from rewind debugger times to profiler times
		const TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
		const double StartTime = TraceTimeRange.GetLowerBoundValue();
		const double EndTime = TraceTimeRange.GetUpperBoundValue();

		// Root information
		
		// Get information providers)
		if (CurvesUpdateRequested > 10)
		{
			// Clear curve
			TArray<SCurveTimelineView::FTimelineCurveData::CurvePoint> & CurvePoints = CurveData->Points;
			CurvePoints.SetNum(0, EAllowShrinking::No);

			// Draw curve overtime
			ReadObjectPropertyValueOverTime(StartTime, EndTime, [&CurvePoints](const FObjectPropertyValue& InValue, uint32 InValueIndex, const FObjectPropertiesMessage& InMessage, const IGameplayProvider::ObjectPropertiesTimeline &  InTimeline , double InStart, double InEndTime)
			{
				CurvePoints.Add({InMessage.ElapsedTime, InValue.ValueAsFloat});
			});

			// Reset counter
			CurvesUpdateRequested = 0;
		}

		// Update child tracks.
		return FPropertyTrack::UpdateInternal();
	}

	TSharedPtr<SWidget> FNumericPropertyTrack::GetTimelineViewInternal()
	{
		const FLinearColor Color = FObjectPropertyHelpers::GetPropertyColor(ObjectProperty->Property);
		
		FLinearColor CurveColor = Color;
		CurveColor.R *= 0.25;
		CurveColor.G *= 0.25;
		CurveColor.B *= 0.25;

		TSharedPtr<SCurveTimelineView> CurveTimelineView = SNew(SCurveTimelineView)
		.FillColor(Color)
		.CurveColor(CurveColor)
		.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
		.RenderFill(false)
		.CurveData_Raw(this, &FNumericPropertyTrack::GetCurveData);

		return CurveTimelineView;
	}
}