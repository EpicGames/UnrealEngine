// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyTrack.h"
#include "SCurveTimelineView.h"

namespace RewindDebugger
{
	/**
	 * Track that displays a object's numeric property value overtime
	 */
	class FNumericPropertyTrack : public FPropertyTrack
	{
	public:

		BASE_PROPERTY_TRACK()
		
		/** Constructor */
		FNumericPropertyTrack(uint64 InObjectId, const TSharedPtr<FObjectPropertyInfo> & InObjectProperty, const TSharedPtr<FPropertyTrack> & InParentTrack);

		/** @return Property's timeline view curve data */
		TSharedPtr<SCurveTimelineView::FTimelineCurveData> GetCurveData() const;
	
	protected:
		
		/** Begin IRewindDebuggerTrack interface */
		virtual bool UpdateInternal() override;
		virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
		/** End IRewindDebuggerTrack interface */
		
		/** Curve data used to display value of property overtime */
		mutable TSharedPtr<SCurveTimelineView::FTimelineCurveData> CurveData;

		/** Keeps track of how many times the curve view has been requested to be drawn/computed */
		mutable int CurvesUpdateRequested = 0;
	};

}
