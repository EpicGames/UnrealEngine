// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectAsTraceIdProxyArchiveReader.h"

#if WITH_ENGINE

#include "Serialization/Archive.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Serialization/SerializedPropertyScope.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/UnrealType.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Misc/PackageName.h"
#include "ObjectTrace.h"
#include "IGameplayProvider.h"
#include "Common/ProviderLock.h"
#include "UObject/ConstructorHelpers.h"


/*----------------------------------------------------------------------------
	FObjectAsTraceIdProxyArchiveReader.
----------------------------------------------------------------------------*/

FObjectAsTraceIdProxyArchiveReader::FObjectAsTraceIdProxyArchiveReader( FArchive& InInnerArchive, const IGameplayProvider* InGameplayProvider)
	: FObjectAsTraceIdProxyArchive(InInnerArchive),
		GameplayProvider(InGameplayProvider)
{
}

FObjectAsTraceIdProxyArchiveReader::~FObjectAsTraceIdProxyArchiveReader() = default;


/**
 * Serialize the given UObject* as a Traced Object Id
 */
FArchive& FObjectAsTraceIdProxyArchiveReader::operator<<(UObject*& Obj)
{
	if (IsLoading())
	{
		uint64 Id = 0;
		InnerArchive << Id;

		const TCHAR* ClassPathName = nullptr;
		const TCHAR* ObjectPathName = nullptr;
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			if (const FClassInfo* ClassInfo = GameplayProvider->FindClassInfo(Id))
			{
				ClassPathName = ClassInfo->PathName;
			}
			else if (const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(Id))
			{
				ObjectPathName = ObjectInfo->PathName;
			}
		}

		if (ClassPathName != nullptr)
		{
			TSoftObjectPtr<UClass> Class;
			Class = FSoftObjectPath(ClassPathName);

			if (Class.LoadSynchronous())
			{
				Obj = Class.Get();
			}
		}
		else if (ObjectPathName != nullptr)
		{
			// look up the object by fully qualified pathname
			Obj = FindObject<UObject>(nullptr, ObjectPathName);
			// If we couldn't find it, and we want to load it, do that
			if (!Obj)
			{
				Obj = LoadObject<UObject>(nullptr, ObjectPathName);
			}
		}
		else
		{
			Obj = nullptr;
		}
	}
	else
	{
		Write(Obj);
	}
	return *this;
}


FArchive& FObjectAsTraceIdProxyArchiveReader::operator<<(FWeakObjectPtr& Obj)
{
	return FArchiveUObject::SerializeWeakObjectPtr(*this, Obj);
}

FArchive& FObjectAsTraceIdProxyArchiveReader::operator<<(FSoftObjectPtr& Value)
{
	if (IsLoading())
	{
		// Reset before serializing to clear the internal weak pointer. 
		Value.ResetWeakPtr();
	}
	*this << Value.GetUniqueID();
	return *this;
}

FArchive& FObjectAsTraceIdProxyArchiveReader::operator<<(FSoftObjectPath& Value)
{
	Value.SerializePath(*this);
	return *this;
}

FArchive& FObjectAsTraceIdProxyArchiveReader::operator<<(FObjectPtr& Obj)
{
	return FArchiveUObject::SerializeObjectPtr(*this, Obj);
}

#endif