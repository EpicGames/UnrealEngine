// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectEventsTrack.h"
#include "Common/ProviderLock.h"
#include "GameplayProvider.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "GameplaySharedData.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "TraceServices/Model/Frames.h"
#include "VariantTreeNode.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/UICommandInfo.h"

#if WITH_EDITOR
#include "ObjectPropertyTrace.h"
#endif

#define LOCTEXT_NAMESPACE "ObjectEventsTrack"

INSIGHTS_IMPLEMENT_RTTI(FObjectEventsTrack)

FObjectEventsTrack::FObjectEventsTrack(const FGameplaySharedData& InSharedData, uint64 InObjectID, const TCHAR* InName)
	: FGameplayTimingEventsTrack(InSharedData, InObjectID, FText::FromString(InName))
	, SharedData(InSharedData)
{
	SetName(MakeTrackName(InSharedData, InObjectID, InName).ToString());
}

void FObjectEventsTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);

	if(GameplayProvider)
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
		GameplayProvider->ReadObjectEventsTimeline(GetGameplayTrack().GetAssociatedObjectId(), [&Context, &Builder](const FGameplayProvider::ObjectEventsTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(Context.GetViewport().GetStartTime(), Context.GetViewport().GetEndTime(), [&Builder](double InStartTime, double InEndTime, uint32 InDepth, const FObjectEventMessage& InMessage)
			{
				Builder.AddEvent(InStartTime, InEndTime, 0, InMessage.Name);
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}


void FObjectEventsTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	DrawEvents(Context);
	GetGameplayTrack().DrawHeaderForTimingTrack(Context, *this, false);
}

void FObjectEventsTrack::InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const
{
	FTimingEventSearchParameters SearchParameters(HoveredTimingEvent.GetStartTime(), HoveredTimingEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch);

	FindObjectEvent(SearchParameters, [this, &Tooltip, &HoveredTimingEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FObjectEventMessage& InMessage)
	{
		Tooltip.ResetContent();

		Tooltip.AddTitle(FText::FromString(FString(InMessage.Name)).ToString());
		Tooltip.AddNameValueTextLine(LOCTEXT("EventTime", "Time").ToString(), FText::AsNumber(HoveredTimingEvent.GetStartTime()).ToString());
		Tooltip.AddNameValueTextLine(LOCTEXT("EventWorld", "World").ToString(), GetGameplayTrack().GetWorldName(SharedData.GetAnalysisSession()).ToString());

		Tooltip.UpdateLayout();
	});
}

const TSharedPtr<const ITimingEvent> FObjectEventsTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	FindObjectEvent(InSearchParameters, [this, &FoundEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FObjectEventMessage& InFoundMessage)
	{
		FoundEvent = MakeShared<const FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth);
	});

	return FoundEvent;
}

void FObjectEventsTrack::FindObjectEvent(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FObjectEventMessage&)> InFoundPredicate) const
{
	TTimingEventSearch<FObjectEventMessage>::Search(
		InParameters,

		// Search...
		[this](TTimingEventSearch<FObjectEventMessage>::FContext& InContext)
		{
			const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);

			if(GameplayProvider)
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
				GameplayProvider->ReadObjectEventsTimeline(GetGameplayTrack().GetAssociatedObjectId(), [&InContext](const FGameplayProvider::ObjectEventsTimeline& InTimeline)
				{
					InTimeline.EnumerateEvents(InContext.GetParameters().StartTime, InContext.GetParameters().EndTime, [&InContext](double InEventStartTime, double InEventEndTime, uint32 InDepth, const FObjectEventMessage& InMessage)
					{
						InContext.Check(InEventStartTime, InEventEndTime, 0, InMessage);
						return TraceServices::EEventEnumerate::Continue;
					});
				});
			}
		},

		// Found!
		[&InFoundPredicate](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FObjectEventMessage& InEvent)
		{
			InFoundPredicate(InFoundStartTime, InFoundEndTime, InFoundDepth, InEvent);
		});
}


FText FObjectEventsTrack::MakeTrackName(const FGameplaySharedData& InSharedData, uint64 InObjectID, const TCHAR* InName) const
{
	FText ClassName = LOCTEXT("UnknownClass", "Unknown");

	const FGameplayProvider* GameplayProvider = InSharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
	if (GameplayProvider)
	{
		bool bIsWorld = false;
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			if (const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(InObjectID))
			{
				if (const FClassInfo* ClassInfo = GameplayProvider->FindClassInfo(ObjectInfo->ClassId))
				{
					ClassName = FText::FromString(ClassInfo->Name);
				}
			}

			bIsWorld = GameplayProvider->IsWorld(InObjectID);
		}

		FText ObjectName;
		if (bIsWorld)
		{
			ObjectName = GetGameplayTrack().GetWorldName(InSharedData.GetAnalysisSession());
		}
		else
		{
			ObjectName = FText::FromString(InName);
		}

		return FText::Format(LOCTEXT("ObjectEventsTrackName", "{0} - {1}"), ClassName, ObjectName);
	}

	return ClassName;
}

void FObjectEventsTrack::GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const
{
	const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
	if (GameplayProvider)
	{
		TSharedRef<FVariantTreeNode> Header = OutVariants.Add_GetRef(FVariantTreeNode::MakeHeader(FText::FromString(GetName()), 0));

		// object events
		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
		GameplayProvider->ReadObjectEventsTimeline(GetGameplayTrack().GetAssociatedObjectId(), [&InFrame, &Header](const FGameplayProvider::ObjectEventsTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, [&Header](double InStartTime, double InEndTime, uint32 InDepth, const FObjectEventMessage& InMessage)
			{
				Header->AddChild(FVariantTreeNode::MakeFloat(FText::FromString(InMessage.Name), InStartTime));
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}

void FObjectEventsTrack::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	FGameplayTimingEventsTrack::BuildContextMenu(MenuBuilder);

#if WITH_EDITOR
#if OBJECT_PROPERTY_TRACE_ENABLED
	MenuBuilder.BeginSection("Trace", LOCTEXT("TraceHeader", "Trace"));
	{
		TWeakObjectPtr<UObject> WeakObject = nullptr;
		const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
		if (GameplayProvider)
		{
			FString ObjectPathName;
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
				if (const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(GetGameplayTrack().GetObjectId()))
				{
					ObjectPathName = ObjectInfo->PathName;
				}
			}

			if (!ObjectPathName.IsEmpty())
			{
				WeakObject = FindObject<UObject>(nullptr, *ObjectPathName);
			}
		}

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("TraceProperties", "Trace Object Properties"),
			LOCTEXT("TraceProperties_Tooltip", "Enable object property tracing for this object."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([WeakObject]()
				{ 
					if(UObject* Object = WeakObject.Get())
					{
						FObjectPropertyTrace::ToggleObjectRegistration(Object);
					}
				}),
				FCanExecuteAction::CreateLambda([WeakObject]()
				{
					return FObjectPropertyTrace::IsEnabled() && WeakObject.IsValid();
				}),
				FGetActionCheckState::CreateLambda([WeakObject]()
				{
					return FObjectPropertyTrace::IsEnabled() && FObjectPropertyTrace::IsObjectRegistered(WeakObject.Get()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();
#endif
#endif
}

#undef LOCTEXT_NAMESPACE
