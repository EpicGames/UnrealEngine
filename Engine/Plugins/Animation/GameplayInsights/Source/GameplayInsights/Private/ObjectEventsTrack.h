// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTrack.h"
#include "Insights/ViewModels/TimingEventsTrack.h"

class FGameplaySharedData;
struct FObjectEventMessage;
class FTimingEventSearchParameters;
struct FWorldInfo;
namespace TraceServices { struct FFrame; }

class FObjectEventsTrack : public FGameplayTimingEventsTrack
{
	INSIGHTS_DECLARE_RTTI(FObjectEventsTrack, FGameplayTimingEventsTrack)

public:
	FObjectEventsTrack(const FGameplaySharedData& InSharedData, uint64 InObjectID, const TCHAR* InName);

	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void Draw(const ITimingTrackDrawContext& Context) const override;
	virtual void InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const override;
	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;
	virtual void GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;

private:
	// Helper function used to find an object event
	void FindObjectEvent(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FObjectEventMessage&)> InFoundPredicate) const;

	// Helper function to build the track's name
	FText MakeTrackName(const FGameplaySharedData& InSharedData, uint64 InObjectID, const TCHAR* InName) const;

private:
	const FGameplaySharedData& SharedData;
};
