// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectPropertiesTrack.h"
#include "Common/ProviderLock.h"
#include "GameplayProvider.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "GameplaySharedData.h"
#include "PropertyHelpers.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "TraceServices/Model/Frames.h"
#include "VariantTreeNode.h"

#define LOCTEXT_NAMESPACE "ObjectPropertiesTrack"

INSIGHTS_IMPLEMENT_RTTI(FObjectPropertiesTrack)

FObjectPropertiesTrack::FObjectPropertiesTrack(const FGameplaySharedData& InSharedData, uint64 InObjectID, const TCHAR* InName)
	: FGameplayTimingEventsTrack(InSharedData, InObjectID, FText::FromString(InName))
	, SharedData(InSharedData)
{
	SetName(MakeTrackName(InSharedData, InObjectID, InName).ToString());
}

void FObjectPropertiesTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);

	if(GameplayProvider)
	{
		const FText PropertiesText(LOCTEXT("PropertiesEventLabel", "Properties"));
		const TCHAR* PropertiesString = *PropertiesText.ToString();

		// object events
		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
		GameplayProvider->ReadObjectPropertiesTimeline(GetGameplayTrack().GetObjectId(), [&Context, &Builder, &PropertiesString](const FGameplayProvider::ObjectPropertiesTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(Context.GetViewport().GetStartTime(), Context.GetViewport().GetEndTime(), [&Builder, &PropertiesString](double InStartTime, double InEndTime, uint32 InDepth, const FObjectPropertiesMessage& InMessage)
			{
				Builder.AddEvent(InStartTime, InEndTime, 0, PropertiesString);
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}


void FObjectPropertiesTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	DrawEvents(Context);
	GetGameplayTrack().DrawHeaderForTimingTrack(Context, *this, false);
}

void FObjectPropertiesTrack::InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const
{
	FTimingEventSearchParameters SearchParameters(HoveredTimingEvent.GetStartTime(), HoveredTimingEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch);

	FindObjectEvent(SearchParameters, [this, &Tooltip, &HoveredTimingEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FObjectPropertiesMessage& InMessage)
	{
		Tooltip.ResetContent();

		Tooltip.AddTitle(FText::Format(LOCTEXT("PropertiesEventFormat", "{0} Properties"), FText::AsNumber(InMessage.PropertyValueEndIndex - InMessage.PropertyValueStartIndex)).ToString());	
		Tooltip.AddNameValueTextLine(LOCTEXT("EventTime", "Time").ToString(), FText::AsNumber(HoveredTimingEvent.GetStartTime()).ToString());
		Tooltip.AddNameValueTextLine(LOCTEXT("EventWorld", "World").ToString(), GetGameplayTrack().GetWorldName(SharedData.GetAnalysisSession()).ToString());

		Tooltip.UpdateLayout();
	});
}

const TSharedPtr<const ITimingEvent> FObjectPropertiesTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	FindObjectEvent(InSearchParameters, [this, &FoundEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FObjectPropertiesMessage& InFoundMessage)
	{
		FoundEvent = MakeShared<const FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth);
	});

	return FoundEvent;
}

void FObjectPropertiesTrack::FindObjectEvent(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FObjectPropertiesMessage&)> InFoundPredicate) const
{
	TTimingEventSearch<FObjectPropertiesMessage>::Search(
		InParameters,

		// Search...
		[this](TTimingEventSearch<FObjectPropertiesMessage>::FContext& InContext)
		{
			const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);

			if(GameplayProvider)
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

				GameplayProvider->ReadObjectPropertiesTimeline(GetGameplayTrack().GetObjectId(), [&InContext](const FGameplayProvider::ObjectPropertiesTimeline& InTimeline)
				{
					InTimeline.EnumerateEvents(InContext.GetParameters().StartTime, InContext.GetParameters().EndTime, [&InContext](double InEventStartTime, double InEventEndTime, uint32 InDepth, const FObjectPropertiesMessage& InMessage)
					{
						InContext.Check(InEventStartTime, InEventEndTime, 0, InMessage);
						return TraceServices::EEventEnumerate::Continue;
					});
				});
			}
		},

		// Found!
		[&InFoundPredicate](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FObjectPropertiesMessage& InEvent)
		{
			InFoundPredicate(InFoundStartTime, InFoundEndTime, InFoundDepth, InEvent);
		});
}


FText FObjectPropertiesTrack::MakeTrackName(const FGameplaySharedData& InSharedData, uint64 InObjectID, const TCHAR* InName) const
{
	FText ClassName = LOCTEXT("UnknownClass", "Unknown");

	const FGameplayProvider* GameplayProvider = InSharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
	if(GameplayProvider)
	{
		bool bIsWorld = false;
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			bIsWorld = GameplayProvider->IsWorld(InObjectID);
		}

		FText ObjectName;
		if (bIsWorld)
		{
			ObjectName = GetGameplayTrack().GetWorldName(InSharedData.GetAnalysisSession());
		}
		else
		{
			ObjectName = FText::FromString(InName);
		}

		return FText::Format(LOCTEXT("ObjectPropertiesTrackName", "Properties - {0}"), ObjectName);
	}

	return ClassName;
}

void FObjectPropertiesTrack::GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const
{
	if(const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName))
	{
		TSharedRef<FVariantTreeNode> Header = OutVariants.Add_GetRef(FVariantTreeNode::MakeHeader(FText::FromString(GetName()), INDEX_NONE));
		TArray<TSharedPtr<FVariantTreeNode>> PropertyVariants;

		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
		GameplayProvider->ReadObjectPropertiesTimeline(GetGameplayTrack().GetObjectId(), [this, &InFrame, GameplayProvider, &Header, &PropertyVariants](const FGameplayProvider::ObjectPropertiesTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, [this, GameplayProvider, &Header, &PropertyVariants](double InStartTime, double InEndTime, uint32 InDepth, const FObjectPropertiesMessage& InMessage)
			{
				GameplayProvider->ReadObjectPropertiesStorage(GetGameplayTrack().GetObjectId(), InMessage, [GameplayProvider, &Header, &PropertyVariants](const TConstArrayView<FObjectPropertyValue> & InStorage)
				{
					for (int32 i = 0; i < InStorage.Num(); ++i)
					{
						PropertyVariants.Add(FObjectPropertyHelpers::GetVariantNodeFromProperty(i, *GameplayProvider, InStorage));

						// note assumes that order is parent->child in the properties array
						if(InStorage[i].ParentId != INDEX_NONE)
						{
							PropertyVariants[InStorage[i].ParentId]->AddChild(PropertyVariants.Last().ToSharedRef());
						}
						else
						{
							Header->AddChild(PropertyVariants.Last().ToSharedRef());
						}
					}
				});
				
				return TraceServices::EEventEnumerate::Stop;
			});
		});
	}
}

#undef LOCTEXT_NAMESPACE
