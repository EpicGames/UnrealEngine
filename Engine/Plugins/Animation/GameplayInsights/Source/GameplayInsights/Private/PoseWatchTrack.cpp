// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "PoseWatchTrack.h"
#include "Common/ProviderLock.h"
#include "SCurveTimelineView.h"
#include "IRewindDebugger.h"
#include "GameplayProvider.h"
#include "AnimationProvider.h"
#include "SNotifiesView.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimInstance.h"
#include "Engine/PoseWatchRenderData.h"
#include "Engine/PoseWatch.h"
#include "ObjectTrace.h"
#include "SPoseWatchCurvesView.h"

#define LOCTEXT_NAMESPACE "PoseWatchesTrack"

namespace RewindDebugger
{

FPoseWatchCurveTrack::FPoseWatchCurveTrack(uint64 InObjectId, uint32 InCurveId, uint64 InPoseWatchTrackId)
	: FAnimCurveTrack(InObjectId, InCurveId)
	, PostWatchTrackId(InPoseWatchTrackId)
{
}

void FPoseWatchCurveTrack::UpdateCurvePointsInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	// convert time range to from rewind debugger times to profiler times
	TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	double StartTime = TraceTimeRange.GetLowerBoundValue();
	double EndTime = TraceTimeRange.GetUpperBoundValue();
	
	auto& CurvePoints = CurveData->Points;
	CurvePoints.SetNum(0,EAllowShrinking::No);

	TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);

	AnimationProvider->ReadPoseWatchTimeline(ObjectId, [this, StartTime, EndTime, AnimationProvider, &CurvePoints](const FAnimationProvider::PoseWatchTimeline& InPoseWatchTimeline)
	{
		InPoseWatchTimeline.EnumerateEvents(StartTime, EndTime, [this, StartTime, EndTime, AnimationProvider, &CurvePoints](double InStartTime, double InEndTime, uint32 InDepth, const FPoseWatchMessage& InMessage)
		{
			if (InMessage.PoseWatchId == PostWatchTrackId)
			{
				if (InEndTime > StartTime && InStartTime < EndTime)
				{
					if(InMessage.bIsEnabled)
					{
						double Time = InMessage.RecordingTime;
						AnimationProvider->EnumeratePoseWatchCurves(InMessage, [this, Time, &CurvePoints](const FSkeletalMeshNamedCurve& InCurve)
						{
							if (InCurve.Id == CurveId)
							{
								CurvePoints.Add({Time,InCurve.Value});
							}
						});
					}
				}
			}
			return TraceServices::EEventEnumerate::Continue;
		});
	});
}

TSharedPtr<SWidget> FPoseWatchCurveTrack::GetDetailsViewInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	TSharedPtr<SPoseWatchCurvesView> PoseWatchCurvesView = SNew(SPoseWatchCurvesView, ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession())
				.CurrentTime_Lambda([RewindDebugger]{ return RewindDebugger->CurrentTraceTime(); });
	PoseWatchCurvesView->SetPoseWatchCurveFilter(PostWatchTrackId, CurveId);
	return PoseWatchCurvesView;
}

FPoseWatchTrack::FPoseWatchTrack(uint64 InObjectId, uint64 InPoseWatchTrackId, FColor InColor, uint32 InNameId)
	: ObjectId(InObjectId)
	, PoseWatchTrackId(InPoseWatchTrackId)
	, Color(InColor)
	, NameId(InNameId)
{
	EnabledSegments = MakeShared<SSegmentedTimelineView::FSegmentData>();
	Icon = UPoseWatchPoseElement::StaticGetIcon();
}

FText FPoseWatchTrack::GetDisplayNameInternal() const
{
	return TrackName;
}

TSharedPtr<SSegmentedTimelineView::FSegmentData> FPoseWatchTrack::GetSegmentData() const
{
	return EnabledSegments;
}

bool FPoseWatchTrack::UpdateInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	
	TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	double StartTime = TraceTimeRange.GetLowerBoundValue();
	double EndTime = TraceTimeRange.GetUpperBoundValue();
	
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	bool bChanged = false;

	if (AnimationProvider && EnabledSegments.IsValid())
	{
		EnabledSegments->Segments.SetNum(0, EAllowShrinking::No);

		struct FPoseWatchEnabledTime
		{
			double RecordingTime;
			bool bIsEnabled;

			bool operator <(const FPoseWatchEnabledTime& Other) const
			{
				return RecordingTime < Other.RecordingTime;
			}
		};

		TArray<FPoseWatchEnabledTime> RecordingTimes;

		TArray<uint64> UniqueCurveIds;

		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
	
			AnimationProvider->ReadPoseWatchTimeline(ObjectId, [this, StartTime, EndTime, &RecordingTimes, &UniqueCurveIds, AnimationProvider](const FAnimationProvider::PoseWatchTimeline& InPoseWatchTimeline)
			{
				InPoseWatchTimeline.EnumerateEvents(StartTime, EndTime, [this, StartTime, EndTime, &RecordingTimes, &UniqueCurveIds, AnimationProvider](double InStartTime, double InEndTime, uint32 InDepth, const FPoseWatchMessage& InMessage)
				{
					if (InMessage.PoseWatchId == PoseWatchTrackId)
					{
						Color = InMessage.Color;
						NameId = InMessage.NameId;
	
						FPoseWatchEnabledTime EnabledTime;
						EnabledTime.RecordingTime = InMessage.RecordingTime;
						EnabledTime.bIsEnabled = InMessage.bIsEnabled;
	
						RecordingTimes.Add(EnabledTime);
	
						AnimationProvider->EnumeratePoseWatchCurves(InMessage, [this, &UniqueCurveIds](const FSkeletalMeshNamedCurve& InCurve)
						{
							UniqueCurveIds.AddUnique(InCurve.Id);
						});
					}
					return TraceServices::EEventEnumerate::Continue;
				});
			});

			if(const TCHAR* FoundName = AnimationProvider->GetName(NameId))
			{
				bChanged = bChanged || TrackName.ToString().Compare(FoundName) != 0;
				TrackName = FText::FromString(FoundName);
			}
		}

		if (!RecordingTimes.IsEmpty())
		{
			RecordingTimes.Sort();
			double RangeStart = 0.0;

			bool bIsEnabled = RecordingTimes[0].bIsEnabled;
			for (int32 Index = 1; Index < RecordingTimes.Num(); ++Index)
			{
				// Going from disabled to enabled
				if (!bIsEnabled && RecordingTimes[Index].bIsEnabled)
				{
					RangeStart = RecordingTimes[Index].RecordingTime;
					bIsEnabled = true;
				}

				// Going from enabled to disabled
				else if (bIsEnabled && !RecordingTimes[Index].bIsEnabled)
				{
					double RangeEnd = RecordingTimes[Index].RecordingTime;
					bIsEnabled = false;
					EnabledSegments->Segments.Add(TRange<double>(RangeStart, RangeEnd));
				}
			}

			if (bIsEnabled)
			{
				// This region hasn't closed yet
				EnabledSegments->Segments.Add(TRange<double>(RangeStart, RecordingTimes.Last().RecordingTime));
			}
		}

		UniqueCurveIds.StableSort();
		const int32 TrackCount = UniqueCurveIds.Num();

		if (Children.Num() != TrackCount)
		{
			bChanged = true;
		}

		Children.SetNum(UniqueCurveIds.Num());
		for(int i = 0; i < TrackCount; i++)
		{
			if (!Children[i].IsValid() || !(Children[i].Get()->GetCurveId() == UniqueCurveIds[i]))
			{
				Children[i] = MakeShared<FPoseWatchCurveTrack>(ObjectId, UniqueCurveIds[i], PoseWatchTrackId);
				bChanged = true;
			}
			
			if (Children[i]->Update())
			{
				bChanged = true;
			}
		}
	}

	return bChanged;
}

TSharedPtr<SWidget> FPoseWatchTrack::GetTimelineViewInternal()
{
	const auto GetPoseWatchColorLambda = [this]() -> FLinearColor { return FLinearColor(Color); };

	const auto TimelineView = SNew(SSegmentedTimelineView)
		.FillColor_Lambda(GetPoseWatchColorLambda)
		.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
		.SegmentData_Raw(this, &FPoseWatchTrack::GetSegmentData);

	return TimelineView;
}

TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> FPoseWatchTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(Children.GetData()), Children.Num());
}

FName FPoseWatchesTrackCreator::GetTargetTypeNameInternal() const
{
	return UAnimInstance::StaticClass()->GetFName();
}
	
static const FName PoseWatchesName("PoseWatches");
FName FPoseWatchesTrackCreator::GetNameInternal() const
{
	return PoseWatchesName;
}
		
void FPoseWatchesTrackCreator::GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const 
{
	Types.Add({PoseWatchesName, LOCTEXT("Pose Watches", "Pose Watches")});
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FPoseWatchesTrackCreator::CreateTrackInternal(const FObjectId& InObjectId) const
{
	return MakeShared<RewindDebugger::FPoseWatchesTrack>(InObjectId.GetMainId());
}

		
FPoseWatchesTrack::FPoseWatchesTrack(uint64 InObjectId)
	: ObjectId(InObjectId)
{
	Icon = UPoseWatchPoseElement::StaticGetIcon();
}

bool FPoseWatchesTrack::UpdateInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	
	TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	double StartTime = TraceTimeRange.GetLowerBoundValue();
	double EndTime = TraceTimeRange.GetUpperBoundValue();
	
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
	
	bool bChanged = false;
	
	if (AnimationProvider)
	{
		struct FUniquePoseWatch
		{
			uint64 PoseWatchId;
			FColor Color;
			uint32 NameId;

			bool operator==(const FUniquePoseWatch& InOther) const
			{
				return PoseWatchId == InOther.PoseWatchId;
			}

			bool operator<(const FUniquePoseWatch& InOther) const
			{
				return PoseWatchId < InOther.PoseWatchId;
			}
		};
		
		TArray<FUniquePoseWatch> UniquePoseWatches;

		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
			AnimationProvider->ReadPoseWatchTimeline(ObjectId, [this, StartTime, EndTime, &UniquePoseWatches](const FAnimationProvider::PoseWatchTimeline& InPoseWatchTimeline)
			{
				InPoseWatchTimeline.EnumerateEvents(StartTime, EndTime, [this, &UniquePoseWatches](double InStartTime, double InEndTime, uint32 InDepth, const FPoseWatchMessage& InMessage)
				{
					UniquePoseWatches.AddUnique({ InMessage.PoseWatchId, FColor(InMessage.Color), InMessage.NameId });
					return TraceServices::EEventEnumerate::Continue;
				});
			});
		}

		UniquePoseWatches.StableSort();
		const int32 TrackCount = UniquePoseWatches.Num();

		if (Children.Num() != TrackCount)
			bChanged = true;

		Children.SetNum(UniquePoseWatches.Num());
		for(int i = 0; i < TrackCount; i++)
		{
			if (!Children[i].IsValid() || !(Children[i].Get()->GetPoseWatchTrackId() == UniquePoseWatches[i].PoseWatchId))
			{
				Children[i] = MakeShared<FPoseWatchTrack>(ObjectId, UniquePoseWatches[i].PoseWatchId, UniquePoseWatches[i].Color, UniquePoseWatches[i].NameId);
				bChanged = true;
			}

			if (Children[i]->Update())
			{
				bChanged = true;
			}
		}
	}
	
	return bChanged;
}

TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> FPoseWatchesTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(Children.GetData()), Children.Num());
}

bool FPoseWatchesTrackCreator::HasDebugInfoInternal(const FObjectId& InObjectId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPoseWatchesTrack::HasDebugInfoInternal);
	const TraceServices::IAnalysisSession* AnalysisSession = IRewindDebugger::Instance()->GetAnalysisSession();
	
	bool bHasData = false;
	if (const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName))
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadPoseWatchTimeline(InObjectId.GetMainId(), [&bHasData](const FAnimationProvider::PoseWatchTimeline& InTimeline)
		{
			bHasData = true;
		});
	}
	return bHasData;
}

}

#undef LOCTEXT_NAMESPACE

#endif