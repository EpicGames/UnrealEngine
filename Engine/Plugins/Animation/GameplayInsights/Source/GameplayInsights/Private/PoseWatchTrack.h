// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "IAnimationProvider.h"
#include "IRewindDebuggerTrackCreator.h"
#include "RewindDebuggerTrack.h"
#include "Textures/SlateIcon.h"
#include "SSegmentedTimelineView.h"
#include "AnimCurvesTrack.h"

class UPoseWatchPoseElement;
class UAnimBlueprintGeneratedClass;

namespace RewindDebugger
{

// Sub Track for a single curve in a pose watch
class FPoseWatchCurveTrack : public FAnimCurveTrack
{
public:
	FPoseWatchCurveTrack(uint64 InObjectId, uint32 InCurveId, uint64 InPoseWatchTrackId);

	uint64 GetPoseWatchTrackId() const { return PostWatchTrackId; }

private:
	virtual void UpdateCurvePointsInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual FName GetNameInternal() const override { return "PoseWatchAnimationCurve"; }
	
	uint64 PostWatchTrackId;
};

class FPoseWatchTrack : public FRewindDebuggerTrack
{
public:
	FPoseWatchTrack(uint64 InObjectId, const uint64 TrackId, FColor InColor, uint32 InNameId);
	uint64 GetPoseWatchTrackId() const { return PoseWatchTrackId; }

private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;

	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FText GetDisplayNameInternal() const override;
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;

	TSharedPtr<SSegmentedTimelineView::FSegmentData> GetSegmentData() const;
	
	uint64 ObjectId;
	uint64 PoseWatchTrackId;
	FSlateIcon Icon;
	FText TrackName;
	FColor Color;
	uint32 NameId;

	TSharedPtr<SSegmentedTimelineView::FSegmentData> EnabledSegments;

	TArray<TSharedPtr<FPoseWatchCurveTrack>> Children;
};


// Parent track of all the pose watches
class FPoseWatchesTrack : public FRewindDebuggerTrack
{
public:
	FPoseWatchesTrack(uint64 InObjectId);

private:
	virtual bool UpdateInternal() override;
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;

	virtual FName GetNameInternal() const override { return "PoseWatches"; }
	virtual FText GetDisplayNameInternal() const override { return NSLOCTEXT("RewindDebugger", "PoseWatchesTrackName", "Pose Watches"); }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }
	virtual FSlateIcon GetIconInternal() override { return Icon; }

	FSlateIcon Icon;
	uint64 ObjectId;

	TArray<TSharedPtr<FPoseWatchTrack>> Children;
};

#if WITH_ENGINE
class FPoseWatchesTrackCreator : public IRewindDebuggerTrackCreator
{
private:
	virtual FName GetTargetTypeNameInternal() const;
	virtual FName GetNameInternal() const override;
	virtual void GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<FRewindDebuggerTrack> CreateTrackInternal(const FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const FObjectId& InObjectId) const override;
};
#endif // WITH_ENGINE

}

#endif