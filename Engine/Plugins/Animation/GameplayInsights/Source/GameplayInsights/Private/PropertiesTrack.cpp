// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertiesTrack.h"

#include "AnimationProvider.h"
#include "Common/ProviderLock.h"
#include "GameplayProvider.h"
#include "IGameplayProvider.h"
#include "IRewindDebugger.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Frames.h"
#include "SObjectPropertiesView.h"
#include "SPropertyTrackView.h"
#include "VariantTreeNode.h"
#include "Styling/SlateIconFinder.h"
#include "PropertyTrack.h"
#include "PropertyHelpers.h"
#include "PropertyWatchManager.h"

#define LOCTEXT_NAMESPACE "FPropertiesTrack"

namespace RewindDebugger
{
	///////////////////////////////////////////////////////////////////////////////
	// Properties Track
	///////////////////////////////////////////////////////////////////////////////

	FPropertiesTrack::FPropertiesTrack(uint64 InObjectId) : ObjectId(InObjectId), bAreWatchedPropertiesDirty(false), TotalPropertyCountAtStart(0)
	{
		Icon = FSlateIconFinder::FindIconForClass(UObject::StaticClass());
	}

	void FPropertiesTrack::Initialize()
	{
		FPropertyWatchManager * PropertyWatchManager = FPropertyWatchManager::Instance();
		check(PropertyWatchManager)

		// Setup handlers so properties are added and deleted accordingly.
		PropertyWatchManager->OnPropertyWatched().AddSP(this, &FPropertiesTrack::OnPropertyWatched);
		PropertyWatchManager->OnPropertyUnwatched().AddSP(this, &FPropertiesTrack::OnPropertyUnwatched);

		// Add any watched properties for this object.
		for (const auto & WatchedProperty : PropertyWatchManager->GetWatchedProperties(ObjectId))
		{
			AddTrackedProperty(WatchedProperty);
		}
	}

	int64 FPropertiesTrack::GetTotalPropertyCountAtStart() const
	{
		return TotalPropertyCountAtStart;
	}

	void FPropertiesTrack::AddTrackedProperty(uint32 InPropertyNameId)
	{
		// Cache size before modification
		const int32 PrevSize = ChildrenTracks.Num();

		// Add property and its respective parents if possible.
		if (InPropertyNameId && !ChildrenTracks.Contains(InPropertyNameId))
		{
			const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
			check(RewindDebugger);
			const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession();

			// Get current visible range 
			const TRange<double> RecordingRange = RewindDebugger->GetCurrentTraceRange();
			const double StartTime = RecordingRange.GetLowerBoundValue();
			const double EndTime = RecordingRange.GetUpperBoundValue();

			// Create track for watched property if found in recording.
			FObjectPropertyInfo FoundProperty;
			if (Session && FObjectPropertyHelpers::FindPropertyValueFromNameId(InPropertyNameId, ObjectId, *Session, StartTime, EndTime, FoundProperty))
			{
				FPropertyWatchManager * PropertyWatchManager = FPropertyWatchManager::Instance();
				check(PropertyWatchManager)
				
				const TSharedPtr<FPropertyTrack> * PrevPropertyTrack = &ChildrenTracks.Add(InPropertyNameId, FPropertyTrack::Create(ObjectId, MakeShared<FObjectPropertyInfo>(FoundProperty.Name, FoundProperty.Property), nullptr));

				{
					uint32 NextNameId = FoundProperty.Property.ParentNameId;
					FObjectPropertyInfo FoundParentProperty {};
					while (NextNameId)
					{
						// Find parent property
						if (FObjectPropertyHelpers::FindPropertyValueFromNameId(NextNameId, ObjectId, *Session, StartTime, EndTime, FoundParentProperty))
						{
							const TSharedPtr<FPropertyTrack> * ParentPropertyTrack = ChildrenTracks.Find(NextNameId);

							// Ensure parent track exists.
							if (!ParentPropertyTrack)
							{
								ParentPropertyTrack = &ChildrenTracks.Add(NextNameId, FPropertyTrack::Create(ObjectId, MakeShared<FObjectPropertyInfo>(FoundParentProperty.Name, FoundParentProperty.Property), nullptr));
							}

							// Setup track relationships.
							(*PrevPropertyTrack)->SetParent(*ParentPropertyTrack);
							(*ParentPropertyTrack)->AddUniqueChild(*PrevPropertyTrack);
							PrevPropertyTrack = ParentPropertyTrack;

							// Ensure we are also watching the parent property.
							PropertyWatchManager->WatchProperty(ObjectId, NextNameId);
							
							// Get next id
							NextNameId = FoundParentProperty.Property.ParentNameId; 
						}
						else
						{
							UE_LOGF(LogRewindDebugger, Warning, "Failed to find parent property with named id {%d} while creating property watch track for {%ls}", FoundProperty.Property.ParentNameId, *FoundProperty.Name.ToString());
							break;
						}
					}
				}
			
				bAreWatchedPropertiesDirty = true;
			}
			else
			{
				UE_LOGF(LogRewindDebugger, Warning, "Failed to find property with named id {%d} while creating its watch track.", FoundProperty.Property.NameId);
			}
		}

		// Sort tracked properties if any change is detected
		if (PrevSize != ChildrenTracks.Num())
		{
			ChildrenTracks.KeySort([](const uint32 A, const uint32 B)
			{
				return A < B;
			});
		}
	}

	void FPropertiesTrack::RemoveTrackedProperty(uint32 InPropertyNameId)
	{
		if (InPropertyNameId)
		{
			FPropertyWatchManager * PropertyWatchManager = FPropertyWatchManager::Instance();
			check(PropertyWatchManager)
			
			// Cache size before modification
			const int32 PrevSize = ChildrenTracks.Num();

			if (const TSharedPtr<FPropertyTrack> * ToRemoveProperty = ChildrenTracks.Find(InPropertyNameId))
			{
				// Remove children of property
				TArray<TConstArrayView<TSharedPtr<FPropertyTrack>>> Stack;
				Stack.Push((*ToRemoveProperty)->GetChildren());

				// Remove property from tracking map and parent if any.
				{
					if (const uint32 ParentNameId = (*ToRemoveProperty)->GetObjectProperty()->Property.ParentNameId)
					{
						const TSharedPtr<FPropertyTrack> * ParentProperty = ChildrenTracks.Find(ParentNameId);

						if (ParentProperty)
						{
							(*ParentProperty)->RemoveChild(*ToRemoveProperty);
						}
						else
						{
							UE_LOGF(LogRewindDebugger, Warning, "Unable to find watch track for parent property with id {%d}", ParentNameId);
						}
					}

					ChildrenTracks.Remove((*ToRemoveProperty)->GetObjectProperty()->Property.NameId);

					// Ensure we are also not watching the property.
					PropertyWatchManager->UnwatchProperty(ObjectId, InPropertyNameId);
				}
			
				// Remove property tree from tracked properties
				while (!Stack.IsEmpty())
				{
					// Get parent property
					TConstArrayView<TSharedPtr<FPropertyTrack>> Parent = Stack.Top();
					Stack.Pop(EAllowShrinking::No);
				
					// Ensure child properties are deleted
					for (const TSharedPtr<FPropertyTrack> & Child : Parent)
					{
						// Remove child property
						ChildrenTracks.Remove(Child->GetObjectProperty()->Property.NameId);

						// Ensure we are also not watching the property.
						PropertyWatchManager->UnwatchProperty(ObjectId, Child->GetObjectProperty()->Property.NameId);

						// Add children for deletion
						Stack.Push(Child->GetChildren());
					}
				}
			
				bAreWatchedPropertiesDirty = true;
			}

			// Sort tracked properties if any change is detected
			if (PrevSize != ChildrenTracks.Num())
			{
				ChildrenTracks.KeySort([](const uint32 A, const uint32 B)
				{
					return A < B;
				});
			}
		}
	}

	bool FPropertiesTrack::UpdateInternal()
	{
		bool bChanged = bAreWatchedPropertiesDirty;
		bAreWatchedPropertiesDirty = false;
			
		// Update root properties
		for (const TTuple<uint32, TSharedPtr<FPropertyTrack>> & PropertyWatch : ChildrenTracks)
		{
			// Base node.
			if (PropertyWatch.Value->GetObjectProperty()->Property.ParentId == INDEX_NONE)
			{
				bChanged = PropertyWatch.Value->Update() || bChanged;
			}
		}
		
		return bChanged;
	}

	TSharedPtr<SWidget> FPropertiesTrack::GetDetailsViewInternal()
	{
		// Properties View
		const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
		TSharedRef<SObjectPropertiesView> Widget = SNew(SObjectPropertiesView, ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession())
		.CurrentTime_Raw(RewindDebugger, &IRewindDebugger::CurrentTraceTime);
		
		return Widget;
	}

	FSlateIcon FPropertiesTrack::GetIconInternal()
	{
		return Icon;
	}

	FName FPropertiesTrack::GetNameInternal() const
	{
		static FName Name = "PropertyWatch";
		return Name;
	}

	FText FPropertiesTrack::GetDisplayNameInternal() const
	{
		static FText DisplayName = LOCTEXT("PropertyTrackName", "Properties");
		return DisplayName;
	}

	uint64 FPropertiesTrack::GetObjectIdInternal() const
	{
		return ObjectId;
	}

	TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> FPropertiesTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
	{
		// Iterate through all the parent tracks
		for (const TTuple<uint32, TSharedPtr<FPropertyTrack>>& PropertyWatch : ChildrenTracks)
		{
			if (!PropertyWatch.Value->GetObjectProperty()->Property.ParentNameId)
			{
				OutTracks.Push(PropertyWatch.Value);
			}
		}

		return {};
	}

	void FPropertiesTrack::OnPropertyWatched(uint64 InObjectId, uint32 InPropertyNameId)
	{
		if (InObjectId == ObjectId)
		{
			AddTrackedProperty(InPropertyNameId);
		}
	}

	void FPropertiesTrack::OnPropertyUnwatched(uint64 InObjectId, uint32 InPropertyNameId)
	{
		if (InObjectId == ObjectId)
		{
			RemoveTrackedProperty(InPropertyNameId);
		}
	}

	///////////////////////////////////////////////////////////////////////////////
	// Properties Track Creator
	///////////////////////////////////////////////////////////////////////////////
#if WITH_ENGINE
	FName FPropertiesTrackCreator::GetTargetTypeNameInternal() const
	{
		return UObject::StaticClass()->GetFName();
	}

	static const FName PropertiesTrackCreatorName("PropertyWatch");
	FName FPropertiesTrackCreator::GetNameInternal() const
	{
		return PropertiesTrackCreatorName;
	}

	void FPropertiesTrackCreator::GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const 
    {
    	Types.Add({PropertiesTrackCreatorName, LOCTEXT("Properties", "Properties")});
    }


	TSharedPtr<FRewindDebuggerTrack> FPropertiesTrackCreator::CreateTrackInternal(const FObjectId& InObjectId) const
	{
		TSharedRef<FPropertiesTrack> NewTrack = MakeShared<FPropertiesTrack>(InObjectId.GetMainId());

		NewTrack->Initialize();
		
		return NewTrack;
	}

	bool FPropertiesTrackCreator::HasDebugInfoInternal(const FObjectId& InObjectId) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPropertiesTrack::HasDebugInfoInternal);
		bool bHasData = false;

		if (const TraceServices::IAnalysisSession* AnalysisSession = IRewindDebugger::Instance()->GetAnalysisSession())
		{
			if (const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName))
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
				GameplayProvider->ReadObjectPropertiesTimeline(InObjectId.GetMainId(), [&bHasData](const IGameplayProvider::ObjectPropertiesTimeline & ObjPropsTimeline)
				{
					bHasData = true;
				});
			}
		}

		return bHasData;
	}
#endif // WITH_ENGINE
		
}

#undef LOCTEXT_NAMESPACE
