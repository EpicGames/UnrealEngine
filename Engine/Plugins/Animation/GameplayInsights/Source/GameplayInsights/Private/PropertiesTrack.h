// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRewindDebuggerTrackCreator.h"
#include "RewindDebuggerTrack.h"
#include "Containers/Map.h"
#include "IGameplayProvider.h"

namespace RewindDebugger
{
	struct FObjectPropertyItem;
	class FPropertyTrack;
	
	/**
	 * Root track used to store an object's traced properties information and property tracks
	 */
	class FPropertiesTrack : public FRewindDebuggerTrack, public TSharedFromThis<FPropertiesTrack>
	{
	public:
		
		/** Constructor */
		FPropertiesTrack(uint64 InObjectId);

		/** Initialize properties track */
		void Initialize();
		
		/** @return Total property count for traced object at the start of recording */
		int64 GetTotalPropertyCountAtStart() const;
		
		/** Add a property from the Rewind Debugger timeline*/
		void AddTrackedProperty(uint32 InPropertyNameId);
		
		/** Remove a property from the Rewind Debugger timeline*/
		void RemoveTrackedProperty(uint32 InPropertyNameId);
	
	private:
		/** Begin IRewindDebuggerTrack interface */
		virtual bool UpdateInternal() override;
		virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
		virtual FSlateIcon GetIconInternal() override;
		virtual FName GetNameInternal() const override;
		virtual FText GetDisplayNameInternal() const override;
		virtual uint64 GetObjectIdInternal() const override;
		virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;
		/** End IRewindDebuggerTrack interface */

		/** Event handler used to create tracks for watched properties */
		void OnPropertyWatched(uint64 InObjectId, uint32 InPropertyNameId);

		/** Event handler used to delete tracks for an unwatched properties */
		void OnPropertyUnwatched(uint64 InObjectId, uint32 InPropertyNameId);
		
		/** Used to determine track icon */
		FSlateIcon Icon;
		
		/** Id for traced/target object */
		uint64 ObjectId;

		/** List of watch property tracks to display on Rewind Debugger timeline */
		TMap<uint32, TSharedPtr<FPropertyTrack>> ChildrenTracks;

		/** Used to determine if watched tracks have been externally modified */
		bool bAreWatchedPropertiesDirty;

		/** Count of recorded properties for traced object at the start of recording */
		int64 TotalPropertyCountAtStart;
	};

#if WITH_ENGINE
	/**
	 * Used to automatically create an FPropertiesTrack in Rewind Debugger
	 */
	class FPropertiesTrackCreator : public IRewindDebuggerTrackCreator
	{
		/** Begin IRewindDebuggerTrackCreator interface */
		virtual FName GetTargetTypeNameInternal() const override;
		virtual FName GetNameInternal() const override;
		virtual void GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const override;
		virtual TSharedPtr<FRewindDebuggerTrack> CreateTrackInternal(const FObjectId& InObjectId) const override;
		virtual bool HasDebugInfoInternal(const FObjectId& InObjectId) const override;
		/** End IRewindDebuggerTrackCreator interface */
	};
#endif // WITH_ENGINE
}