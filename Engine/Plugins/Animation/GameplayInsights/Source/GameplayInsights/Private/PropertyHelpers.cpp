// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyHelpers.h"
#include "Common/ProviderLock.h"
#include "PropertyTrack.h"
#include "GameplayProvider.h"
#include "VariantTreeNode.h"

#if WITH_EDITOR
#include "Settings/EditorStyleSettings.h"
#endif

FLinearColor FObjectPropertyHelpers::GetPropertyColor(const FObjectPropertyValue & InProperty)
{
	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	check(RewindDebugger)

	if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
	{
		if (const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			// Cached type strings
			static const FString FloatString = TEXT("float");
			static const FString DoubleString = TEXT("double");
			static const FString UInt8String = TEXT("uint8");
			static const FString Int32String = TEXT("int32");
			static const FString Int64String = TEXT("int64");

			// Property info strings
			const FString TypeString = GameplayProvider->GetPropertyName(InProperty.TypeStringId);
			const FString ValueString = InProperty.Value;
			
			// Note: Currently only supporting color for types that are being rendered as tracks in Rewind Debugger.
			// Additionally we are using hardcoded color values at the moment due standalone UnrealInsights being unable to depend on editor systems.
			if (TypeString.Equals(FloatString))
			{
				return FLinearColor(0.357667f, 1.0f, 0.060000f, 1.0f);
			}
			else if (TypeString.Equals(DoubleString))
			{
				return FLinearColor(0.039216f, 0.666667f, 0.0f, 1.0f);
			}
			else if (IsBoolProperty(ValueString, TypeString))
			{
				return FLinearColor(0.300000f, 0.0f, 0.0f, 1.0f);
			}
			else if (TypeString.Equals(UInt8String))
			{
				return FLinearColor(0.0f, 0.160000f, 0.131270f, 1.0f);
			}
			else if (TypeString.Equals(Int32String))
			{
				return FLinearColor(0.013575f, 0.770000f, 0.429609f, 1.0f);
			}
			else if (TypeString.Equals(Int64String))
			{
				return FLinearColor(0.413575f, 0.770000f, 0.429609f, 1.0f);
			}
			else if (ValueString.IsNumeric())
			{
				return FLinearColor(0.607717f, 0.224984f, 1.0f, 1.0f);
			}
		}
	}
	
	return FLinearColor(0.750000f, 0.6f, 0.4f, 1.0f);
}

TSharedRef<FVariantTreeNode> FObjectPropertyHelpers::GetVariantNodeFromProperty(uint32 InPropertyIndex, const IGameplayProvider& InGameplayProvider, const TConstArrayView<FObjectPropertyValue>& InStorage)
{
	static const FString Int32String = TEXT("int32");
	static const FString FloatString = TEXT("float");
	static const FString FVectorString = TEXT("FVector");
	static const FString FVector2DString = TEXT("FVector2D");
	static const FString FQuatString = TEXT("FQuat");
	
	const FObjectPropertyValue & InValue = InStorage[InPropertyIndex];

	const FText DisplayName = FText::FromName(GetPropertyDisplayName(InValue, InGameplayProvider));
	const FString PropertyTypeString = InGameplayProvider.GetPropertyName(InValue.TypeStringId);

	TSharedPtr<FVariantTreeNode> OutputNode;

	if (IsBoolProperty(InValue.Value, PropertyTypeString))
	{
		OutputNode = FVariantTreeNode::MakeBool(DisplayName, FString(InValue.Value).ToBool());
	}
	else if (PropertyTypeString.Equals(Int32String))
	{
		OutputNode = FVariantTreeNode::MakeInt32(DisplayName, InValue.ValueAsFloat);
	}
	else if (PropertyTypeString.Equals(FloatString))
	{
		OutputNode = FVariantTreeNode::MakeFloat(DisplayName, InValue.ValueAsFloat);
	}
	else if (PropertyTypeString.Equals(FVectorString))
	{
		const FVector& Vec3 = FVector(InStorage[InPropertyIndex + 1].ValueAsFloat,InStorage[InPropertyIndex + 2].ValueAsFloat,InStorage[InPropertyIndex + 3].ValueAsFloat);
		OutputNode = FVariantTreeNode::MakeVector(DisplayName, Vec3);
	}
	else if (PropertyTypeString.Equals(FVector2DString))
	{
		const FVector2D Vec2D = { InStorage[InPropertyIndex + 1].ValueAsFloat, InStorage[InPropertyIndex + 2].ValueAsFloat };
		OutputNode = FVariantTreeNode::MakeVector2D(DisplayName, Vec2D);
	}
	else if (PropertyTypeString.Equals(FQuatString))
	{
		const FQuat Quat = { InStorage[InPropertyIndex + 1].ValueAsFloat, InStorage[InPropertyIndex + 2].ValueAsFloat, InStorage[InPropertyIndex + 3].ValueAsFloat, InStorage[InPropertyIndex + 4].ValueAsFloat };
		OutputNode = FVariantTreeNode::MakeVector(DisplayName, Quat.Euler());
	}
	else
	{
		OutputNode = FVariantTreeNode::MakeString(DisplayName, InValue.Value);
	}

	OutputNode->SetPropertyNameId(InValue.NameId);

	return OutputNode.ToSharedRef();
}

bool FObjectPropertyHelpers::IsBoolProperty(const FString & InPropertyValueString, const FString & InPropertyTypeString)
{
	static const FString BoolString = TEXT("bool");
	static const FString FalseString = TEXT("false");
	static const FString TrueString = TEXT("true");

	// We have to check the type string for a "bool" if the FBoolProperty was native bool. However, we also have to handle the case were is not a native bool but instead a bitfield.
	return InPropertyTypeString.Equals(BoolString) || InPropertyValueString.Equals(FalseString, ESearchCase::IgnoreCase) || InPropertyValueString.Equals(TrueString, ESearchCase::IgnoreCase);
}

FName FObjectPropertyHelpers::GetPropertyDisplayName(const FObjectPropertyValue & InProperty, const IGameplayProvider& InGameplayProvider)
{
	// Query property full name.
	const TCHAR * Name = InGameplayProvider.GetPropertyName(InProperty.NameId);
	FString StringName { Name };

	// Get property short name.
	{
		int32 SplitPos;
		if (StringName.FindLastChar('.', SplitPos))
		{
			StringName = StringName.RightChop(SplitPos + 1);
		}
	}

	
	// Ensure human readable names.
	const bool bIsBool = IsBoolProperty(InProperty.Value, InGameplayProvider.GetPropertyName(InProperty.TypeStringId));

#if WITH_EDITOR
	const UEditorStyleSettings* Settings = GetDefault<UEditorStyleSettings>();
	const FName OutputName = Settings->bShowFriendlyNames ? *FName::NameToDisplayString(StringName, bIsBool) : *StringName;
#else
	const FName OutputName = *FName::NameToDisplayString(StringName, bIsBool);
#endif
	
	return OutputName;
}

FSlateIcon FObjectPropertyHelpers::GetPropertyIcon(const FObjectPropertyValue & InProperty)
{
	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	check(RewindDebugger)

	if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
	{
		if (const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			const FString PropertyTypeString = GameplayProvider->GetPropertyName(InProperty.TypeStringId);
		
			if (PropertyTypeString.StartsWith(TEXT("TArray")))
			{
				return FSlateIcon("EditorStyle", "Kismet.VariableList.ArrayTypeIcon");
			}
			else if (PropertyTypeString.StartsWith(TEXT("TMap")))
			{
				return FSlateIcon("EditorStyle", "Kismet.VariableList.MapKeyTypeIcon", NAME_None, "Kismet.VariableList.MapValueTypeIcon");
			}
			else if (PropertyTypeString.StartsWith(TEXT("TSet")))
			{
				return FSlateIcon("EditorStyle", "Kismet.VariableList.SetTypeIcon");
			}
		}
	}
	
	return FSlateIcon("EditorStyle", "Kismet.VariableList.TypeIcon");
}

TPair<const FObjectPropertyValue *, uint32> FObjectPropertyHelpers::ReadObjectPropertyValueCached(RewindDebugger::FObjectPropertyInfo& InProperty, uint64 InObjectId, const IGameplayProvider& InGameplayProvider, const FObjectPropertiesMessage& InMessage)
{
	TPair<const FObjectPropertyValue *, uint32> Output = { nullptr, 0 };
	
	InGameplayProvider.ReadObjectPropertiesStorage(InObjectId, InMessage, [&InProperty, &InMessage, &Output](const TConstArrayView<FObjectPropertyValue> & InStorage)
	{
		// Attempt to use cached index
		if (InProperty.CachedId != INDEX_NONE && InStorage.IsValidIndex(InProperty.CachedId))
		{
			const FObjectPropertyValue & FoundPropertyValue = InStorage[InProperty.CachedId];

			// Found property used previous known index.
			if (InProperty.Property.NameId == FoundPropertyValue.NameId)
			{
				Output.Key = &FoundPropertyValue;
				Output.Value = InProperty.CachedId;
				return;
			}
		}

		// No constant time access, now using linear search.
		{
			const int64 TotalPropertyCount = InMessage.PropertyValueEndIndex - InMessage.PropertyValueStartIndex;

			for (int64 i = 0; i < TotalPropertyCount; ++i)
			{
				if (InStorage.IsValidIndex(i))
				{
					const FObjectPropertyValue & FoundPropertyValue = InStorage[i];
					if (InProperty.Property.NameId == FoundPropertyValue.NameId)
					{
						InProperty.CachedId = i;
											
						Output.Key = &FoundPropertyValue;
						Output.Value = InProperty.CachedId;
						return;
					}
				}
			}
		}
	});

	return Output;
}

bool FObjectPropertyHelpers::FindPropertyValueFromNameId(uint32 InPropertyNameId, uint64 InObjectId, const TraceServices::IAnalysisSession & InSession, double InStartTime, double InEndTime, RewindDebugger::FObjectPropertyInfo & InOutProperty)
{
	bool bFoundProperty = false;

	if (const FGameplayProvider* GameplayProvider = InSession.ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName))
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
		GameplayProvider->ReadObjectPropertiesTimeline(InObjectId, [InStartTime, InEndTime, GameplayProvider, InObjectId, InPropertyNameId, &bFoundProperty, &InOutProperty](const IGameplayProvider::ObjectPropertiesTimeline & InObjPropsTimeline)
		{
			InObjPropsTimeline.EnumerateEvents(InStartTime, InEndTime, [GameplayProvider, InObjectId, InPropertyNameId, &bFoundProperty, &InOutProperty](double InStartTimeProcedure, double InEndTimeProcedure, uint32 InDepth, const FObjectPropertiesMessage& InMessage) -> TraceServices::EEventEnumerate
			{
				GameplayProvider->ReadObjectPropertiesStorage(InObjectId, InMessage, [GameplayProvider, InPropertyNameId, &bFoundProperty, &InOutProperty](const TConstArrayView<FObjectPropertyValue> & InStorage)
				{
					for (int32 i = 0; i < InStorage.Num(); ++i)
					{
						if (!bFoundProperty && InStorage[i].NameId == InPropertyNameId)
						{
							InOutProperty.Property = InStorage[i];
							InOutProperty.Name = GetPropertyDisplayName(InStorage[i], *GameplayProvider);
							InOutProperty.CachedId = INDEX_NONE;
							
							bFoundProperty = true;
							return;
						}
					}
				});

				return bFoundProperty ? TraceServices::EEventEnumerate::Stop : TraceServices::EEventEnumerate::Continue;
			});
		});
	}
	
	return bFoundProperty;
};