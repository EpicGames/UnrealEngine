// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once
#include "IGameplayProvider.h"
#include "Textures/SlateIcon.h"

struct FVariantTreeNode;

namespace RewindDebugger
{
	struct FObjectPropertyInfo;
}

/**
 * Helper class when working with traced object properties
 */
class FObjectPropertyHelpers
{
public:

	/** @return True if traced property was a FBoolProperty. */
	static bool IsBoolProperty(const FString & InPropertyValueString, const FString & InPropertyTypeString);
	
	/** @return Get formatted property display name */
	static FName GetPropertyDisplayName(const FObjectPropertyValue & InProperty, const IGameplayProvider& InGameplayProvider);
	
	/** @return Icon associated with the given property's type */
	static FSlateIcon GetPropertyIcon(const FObjectPropertyValue & InProperty);

	/** @return Color associated with the given property's type */
	static FLinearColor GetPropertyColor(const FObjectPropertyValue & InProperty);

	/** @return Variant display node from traced property
	 *  @note Assumes an FAnalysisSessionReadScope has been created before accessing data in InStorage variable.
	 */
	static TSharedRef<FVariantTreeNode> GetVariantNodeFromProperty(uint32 InPropertyIndex, const IGameplayProvider & InGameplayProvider, const TConstArrayView<FObjectPropertyValue> & InStorage);
	
	/** @return Read track's property traced value */
	static TPair<const FObjectPropertyValue *, uint32> ReadObjectPropertyValueCached(RewindDebugger::FObjectPropertyInfo & InProperty, uint64 InObjectId, const IGameplayProvider & InGameplayProvider, const FObjectPropertiesMessage& InMessage);
		
	/**
	 * Find object property value given a named id.
	 * @return True if property was found, false otherwise.
	 */
	static bool FindPropertyValueFromNameId(uint32 InPropertyNameId, uint64 InObjectId, const TraceServices::IAnalysisSession & InSession, double InStartTime, double InEndTime, RewindDebugger::FObjectPropertyInfo & InOutProperty);
};
