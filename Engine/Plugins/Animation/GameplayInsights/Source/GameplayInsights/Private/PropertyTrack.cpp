// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyTrack.h"

#include "Common/ProviderLock.h"
#include "PropertyHelpers.h"
#include "AnimationProvider.h"
#include "GameplayProvider.h"
#include "IRewindDebugger.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Frames.h"
#include "SPropertyTrackView.h"
#include "VariantTreeNode.h"
#include "BoolPropertyTrack.h"
#include "NumericPropertyTrack.h"
#include "PropertyWatchManager.h"

#if WITH_EDITOR
#include "ToolMenuSection.h"
#endif

#define LOCTEXT_NAMESPACE "FPropertyTrack"

namespace RewindDebugger
{
	///////////////////////////////////////////////////////////////////////////////
	// Property Track
	///////////////////////////////////////////////////////////////////////////////

	FPropertyTrack::FPropertyTrack(uint64 InObjectId, const TSharedPtr<FObjectPropertyInfo> & InObjectProperty, const TSharedPtr<FPropertyTrack> & InParentTrack) : ObjectId(InObjectId), ObjectProperty(InObjectProperty), Parent(InParentTrack)
	{
		Icon = FObjectPropertyHelpers::GetPropertyIcon(InObjectProperty->Property);
	}

	void FPropertyTrack::BuildContextMenu(FToolMenuSection& InMenuSection)
	{
#if WITH_EDITOR
		const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
		check(RewindDebugger)

		InMenuSection.AddMenuEntry(NAME_None,
			LOCTEXT("RemovePropertyTrack", "Remove Property Track"),
			LOCTEXT("RemovePropertyTrackTooltip", "Remove traced property track from timeline"),
			{},
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				FPropertyWatchManager * PropertyWatchManager = FPropertyWatchManager::Instance();
				check(PropertyWatchManager)
				
				if (!PropertyWatchManager->UnwatchProperty(ObjectId, ObjectProperty->Property.NameId))
				{
					UE_LOGF(LogRewindDebugger, Warning, "Failed to unwatch property with id {%d}", ObjectProperty->Property.NameId);
				}
			})));
#endif
	}

	const TSharedPtr<FObjectPropertyInfo>& FPropertyTrack::GetObjectProperty() const
	{
		return ObjectProperty;
	}

	const FName& FPropertyTrack::GetPropertyName() const
	{
		return ObjectProperty->Name;
	}

	TConstArrayView<TSharedPtr<FPropertyTrack>> FPropertyTrack::GetChildren() const
	{
		return Children;
	}

	const TSharedPtr<FPropertyTrack>& FPropertyTrack::GetParent() const
	{
		return Parent;
	}

	bool FPropertyTrack::AddUniqueChild(const TSharedPtr<FPropertyTrack>& InTrack)
	{
		bool bSuccess = false;
			
		if (InTrack.IsValid())
		{
			const int32 PrevSize = Children.Num();
			Children.AddUnique(InTrack);
			bSuccess = PrevSize != Children.Num();

			if (bSuccess)
			{
				// Sort by distance from parent or start.
				Children.StableSort([](const TSharedPtr<FPropertyTrack>& A, const TSharedPtr<FPropertyTrack>& B)
				{
					const FObjectPropertyValue& PropA = A.Get()->GetObjectProperty()->Property;
					const FObjectPropertyValue& PropB = B.Get()->GetObjectProperty()->Property;
					
					return PropA.NameId < PropB.NameId;
				});
			}
		}
		
		return bSuccess;
	}

	bool FPropertyTrack::RemoveChild(const TSharedPtr<FPropertyTrack>& InTrack)
	{
		bool bSuccess = false;
			
		if (InTrack.IsValid())
		{
			Children.Remove(InTrack);
			bSuccess = true;
		}

		return bSuccess;
	}

	void FPropertyTrack::SetParent(const TSharedPtr<FPropertyTrack>& InParent)
	{
		Parent = InParent;
	}

	bool FPropertyTrack::ReadObjectPropertyValueAtFrame(const TraceServices::FFrame& InFrame, const IGameplayProvider& InGameplayProvider, PropertyAtFrameCallback InCallback) const
	{
		bool bFoundProperty = false;
		
		InGameplayProvider.ReadObjectPropertiesTimeline(ObjectId, [this, &InGameplayProvider, &InFrame, &InCallback, &bFoundProperty](const IGameplayProvider::ObjectPropertiesTimeline & InObjPropsTimeline)
		{
			InObjPropsTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, [this, &InGameplayProvider, &InCallback, &bFoundProperty](double InStartTime, double InEndTime, uint32 InDepth, const FObjectPropertiesMessage& InObjPropsMessage) -> TraceServices::EEventEnumerate
			{
				const auto & [ReadProperty, ReadPropertyIndex] = FObjectPropertyHelpers::ReadObjectPropertyValueCached(*ObjectProperty, ObjectId, InGameplayProvider, InObjPropsMessage);
				
				if (ReadProperty)
				{
					InCallback(*ReadProperty, ReadPropertyIndex, InObjPropsMessage);
					bFoundProperty = true;
				}

				return TraceServices::EEventEnumerate::Stop;
			});
		});

		return bFoundProperty;
	}
	
	void FPropertyTrack::ReadObjectPropertyValueOverTime(double InStartTime, double InEndTime, PropertyOverTimeCallback InCallback) const
	{
		const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
		check(RewindDebugger)
		
		if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
		{
			if (const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName))
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
				// Query properties timeline
				GameplayProvider->ReadObjectPropertiesTimeline(ObjectId, [this, &InCallback, InStartTime, InEndTime, GameplayProvider](const IGameplayProvider::ObjectPropertiesTimeline & InObjPropsTimeline)
				{
					// Query properties over time
					InObjPropsTimeline.EnumerateEvents(InStartTime, InEndTime, [this, &InCallback, &InObjPropsTimeline, InStartTime, InEndTime, GameplayProvider](double InStartTimeInner, double InEndTimeInner, uint32 InDepth, const FObjectPropertiesMessage& InObjPropsMessage) -> TraceServices::EEventEnumerate
					{
						const auto & [ReadProperty, ReadPropertyIndex] =FObjectPropertyHelpers::ReadObjectPropertyValueCached(*ObjectProperty, ObjectId, *GameplayProvider, InObjPropsMessage);
						if (ReadProperty)
						{
							InCallback(*ReadProperty, ReadPropertyIndex, InObjPropsMessage, InObjPropsTimeline, InStartTime, InEndTime);
						}
						
						return TraceServices::EEventEnumerate::Continue;
					});
				});
			}
		}
	}

	TSharedPtr<FPropertyTrack> FPropertyTrack::Create(uint64 InObjectId, const TSharedPtr<FObjectPropertyInfo> & InObjectProperty, const TSharedPtr<FPropertyTrack> & InParentTrack)
	{
		// Handle properties with numeric values
		if (FNumericPropertyTrack::CanBeCreated(*InObjectProperty))
		{
			return MakeShared<FNumericPropertyTrack>(InObjectId, InObjectProperty, InParentTrack);
		}

		// Handle boolean properties
		if (FBoolPropertyTrack::CanBeCreated(*InObjectProperty))
		{
			return MakeShared<FBoolPropertyTrack>(InObjectId, InObjectProperty, InParentTrack);
		}
		
		return MakeShared<FPropertyTrack>(InObjectId, InObjectProperty, InParentTrack);
	}

	bool FPropertyTrack::CanBeCreated(const FObjectPropertyInfo& InObjectProperty)
	{
		return true;
	}

	bool FPropertyTrack::UpdateInternal()
	{
		// Update children
		bool bChanged = false;

		for (const TSharedPtr<FPropertyTrack> & Child : Children)
		{
			bChanged = Child->Update() || bChanged;
		}

		return bChanged;
	}

	TSharedPtr<SWidget> FPropertyTrack::GetDetailsViewInternal()
	{
		const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
		check(RewindDebugger)

		const TSharedRef<SPropertyTrackView> View = SNew(SPropertyTrackView, ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession())
		.CurrentTime_Raw(RewindDebugger, &IRewindDebugger::CurrentTraceTime);
		
		View->SetTrack(AsWeak());
		View->SetForegroundColor(FObjectPropertyHelpers::GetPropertyColor(ObjectProperty->Property));
		
		return View;
	}

	TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> FPropertyTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
	{
		return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(Children.GetData()), Children.Num());
	}

	FSlateIcon FPropertyTrack::GetIconInternal()
	{
		return Icon;
	}

	FName FPropertyTrack::GetNameInternal() const
	{
		static const FName Name = "VariantProperty";
		return Name;
	}

	FText FPropertyTrack::GetDisplayNameInternal() const
	{
		return FText::FromName(ObjectProperty->Name);
	}

	uint64 FPropertyTrack::GetObjectIdInternal() const
	{
		return ObjectId;
	}
}

#undef LOCTEXT_NAMESPACE