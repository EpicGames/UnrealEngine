// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RewindDebuggerTrack.h"
#include "IGameplayProvider.h"

namespace TraceServices
{
struct FFrame;
}

namespace RewindDebugger
{
	class FPropertiesTrack;
	
	/**
	 * Container item that stores information for a watched traced object's variable
	 */
	struct FObjectPropertyInfo
	{
		/** Default Constructor */
		FObjectPropertyInfo()
			: Name(NAME_None),
			  Property(),
			  CachedId(INDEX_NONE)
		{
		}
		
		/** Constructor */
		FObjectPropertyInfo(const FName & Name, const FObjectPropertyValue& TracedProperty, int64 CachedId = INDEX_NONE)
			: Name(Name),
			  Property(TracedProperty),
			  CachedId(CachedId)
		{
		}

		/** Traced variable name */
		FName Name = NAME_None;

		/** Traced variable data. Used to store identifiers. */
		FObjectPropertyValue Property;

		/** Previously found storage index */
		int64 CachedId = INDEX_NONE;
	};

	#define BASE_PROPERTY_TRACK() \
	static bool CanBeCreated(const FObjectPropertyInfo & InObjectProperty);
	
	/**
	 * Track that displays a traced object's property or traced value over time
	 */
	class FPropertyTrack : public FRewindDebuggerTrack, public TSharedFromThis<FPropertyTrack>
	{
	public:

		BASE_PROPERTY_TRACK()
		
		/** Constructor */
		FPropertyTrack(uint64 InObjectId, const TSharedPtr<FObjectPropertyInfo> & InObjectProperty, const TSharedPtr<FPropertyTrack> & InParentTrack);

		/** Begin IRewindDebuggerTrack interface */
		virtual void BuildContextMenu(FToolMenuSection& InMenuSection) override;
		/** End IRewindDebuggerTrack interface */
		
		/** @return Internal data item for traced variable */
		const TSharedPtr<FObjectPropertyInfo> & GetObjectProperty() const;

		/** @return Property name identifier */
		const FName & GetPropertyName() const;

		/** @return Children property tracks */
		TConstArrayView<TSharedPtr<FPropertyTrack>> GetChildren() const;

		/** @return Parent track */
		const TSharedPtr<FPropertyTrack> & GetParent() const;
		
		/**
		 * Add a property track as a child
		 * @return True if property was successfully added as a child.
		 */
		bool AddUniqueChild(const TSharedPtr<FPropertyTrack> & InTrack);
		
		/**
		 * Remove a child property track
		 * @return True if property was successfully removed from child container.
		 */
		bool RemoveChild(const TSharedPtr<FPropertyTrack> & InTrack);

		/**
		 * Update the track's parent.
		 * @note This procedure does not validate if the parent - child relationship is valid.
		 */
		void SetParent(const TSharedPtr<FPropertyTrack> & InParent);
		
		/** Callback signature for reading a property at a given frame */
		typedef TFunctionRef<void(const FObjectPropertyValue & /*InValue*/, uint32 /*InValueIndex*/, const FObjectPropertiesMessage & /*InObjPropsMessage*/)> PropertyAtFrameCallback;

		/**
		 * Read value for internal property at a given frame.
		 * @note Assumes function is called within a TraceServices::IAnalysisSession read access scope.
		 * @return True if property value at given frame was found.
		 */
		bool ReadObjectPropertyValueAtFrame(const TraceServices::FFrame & InFrame, const IGameplayProvider& InGameplayProvider, PropertyAtFrameCallback InCallback) const;

		/** Callback signature for reading a property at over a time range */
		typedef TFunctionRef<void(const FObjectPropertyValue & /*InValue*/, uint32 /*InValueIndex*/, const FObjectPropertiesMessage& /*InMessage*/, const IGameplayProvider::ObjectPropertiesTimeline & /* InTimeline */, double /*InStartTime*/, double /*InEndTime*/)> PropertyOverTimeCallback;

		/** Read value for internal property over a range of time. */
		void ReadObjectPropertyValueOverTime(double InStartTime, double InEndTime, PropertyOverTimeCallback InCallback) const;
		
		/** @return Specialized Property Track, if any, for given object property. */
		static TSharedPtr<FPropertyTrack> Create(uint64 InObjectId, const TSharedPtr<FObjectPropertyInfo> & InObjectProperty, const TSharedPtr<FPropertyTrack> & InParentTrack);
		
	protected:
		/** Begin IRewindDebuggerTrack interface */
		virtual bool UpdateInternal() override;
		virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
		virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;
		virtual FSlateIcon GetIconInternal() override;
		virtual FName GetNameInternal() const override;
		virtual FText GetDisplayNameInternal() const override;
		virtual uint64 GetObjectIdInternal() const override;
		/** End IRewindDebuggerTrack interface */
		
		/** Used to determine track icon */
		FSlateIcon Icon;
		
		/** Id for traced/target object */
		uint64 ObjectId;

		/** Used to store traced variable information */
		TSharedPtr<FObjectPropertyInfo> ObjectProperty;

		/** Parent property track */
		TSharedPtr<FPropertyTrack> Parent;
		
		/** Child properties tracks. Used if property is a struct, vector, etc. */
		TArray<TSharedPtr<FPropertyTrack>> Children;
	};
}