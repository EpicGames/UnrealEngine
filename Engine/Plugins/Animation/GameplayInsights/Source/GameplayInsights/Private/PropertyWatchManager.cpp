// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyWatchManager.h"
#include "IRewindDebugger.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

FPropertyWatchManager* FPropertyWatchManager::InternalInstance = nullptr;

FPropertyWatchManager::FPropertyWatchManager()
{
}

FPropertyWatchManager::~FPropertyWatchManager()
{
}

bool FPropertyWatchManager::WatchProperty(uint64 InObjectId, uint32 InPropertyNameId)
{
	if (InPropertyNameId)
	{
		TArray<uint32> & WatchStorage = WatchedProperties.FindOrAdd(InObjectId);

		const int32 PrevSize = WatchStorage.Num();
	
		WatchStorage.AddUnique(InPropertyNameId);

		// Trigger broadcast only if property was successfully added.
		const bool bWasPropertyWatched = PrevSize != WatchStorage.Num();
		if (bWasPropertyWatched)
		{
			OnPropertyWatchedDelegate.Broadcast(InObjectId, InPropertyNameId);
		}

		return bWasPropertyWatched;
	}

	return false;
}

bool FPropertyWatchManager::UnwatchProperty(uint64 InObjectId, uint32 InPropertyNameId)
{
	if (InPropertyNameId)
	{
		if (TArray<uint32> * WatchStorage = WatchedProperties.Find(InObjectId))
		{
			const int32 PrevSize = WatchStorage->Num();
		
			WatchStorage->RemoveSwap(InPropertyNameId, EAllowShrinking::No);

			// Trigger broadcast only if property was successfully removed.
			const bool bWasPropertyUnwatched = PrevSize != WatchStorage->Num();
			if (bWasPropertyUnwatched)
			{
				OnPropertyUnwatchedDelegate.Broadcast(InObjectId, InPropertyNameId);
			}
			
			return bWasPropertyUnwatched;
		}
	}

	return false;
}

int32 FPropertyWatchManager::GetWatchedPropertiesNum(uint64 InObjectId) const
{
	if (const TArray<uint32> * WatchStorage = WatchedProperties.Find(InObjectId))
	{
		return WatchStorage->Num();
	}
	return 0;
}

TConstArrayView<uint32> FPropertyWatchManager::GetWatchedProperties(uint64 InObjectId) const
{
	if (const TArray<uint32> * WatchStorage = WatchedProperties.Find(InObjectId))
	{
		return MakeArrayView(WatchStorage->GetData(), WatchStorage->Num());
	}
	
	return {};
}

void FPropertyWatchManager::ClearWatchedProperties(uint64 InObjetId)
{
	WatchedProperties.Remove(InObjetId);
}

void FPropertyWatchManager::ClearAllWatchedProperties()
{
	WatchedProperties.Empty();
}

FPropertyWatchManager::FOnPropertyWatched& FPropertyWatchManager::OnPropertyWatched()
{
	return OnPropertyWatchedDelegate;
}

FPropertyWatchManager::FOnPropertyUnwatched& FPropertyWatchManager::OnPropertyUnwatched()
{
	return OnPropertyUnwatchedDelegate;
}		

void FPropertyWatchManager::Initialize()
{
	InternalInstance = new FPropertyWatchManager;

#if WITH_EDITOR
	FEditorDelegates::BeginPIE.AddLambda([](const bool bIsSimulating)
	{
		if (InternalInstance)
		{
			InternalInstance->ClearAllWatchedProperties();
		}
	});
#endif
}

void FPropertyWatchManager::Shutdown()
{
	delete InternalInstance;
	InternalInstance = nullptr;
}

FPropertyWatchManager* FPropertyWatchManager::Instance()
{
	check(InternalInstance != nullptr)
	return InternalInstance;
}