// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IGameplayProvider.h"

namespace RewindDebugger
{
	struct FObjectPropertyItem;
}

/**
 * Used to keep track of traced variables that are being watched.
 */
class FPropertyWatchManager
{
public:
	
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPropertyWatched, uint64 /*InObjectId*/, uint32 /*InPropertyNameId*/)
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPropertyUnwatched, uint64 /*InObjectId*/, uint32 /*InPropertyNameId*/)

	/** Constructor */
	FPropertyWatchManager();

	/** Destructor */
	~FPropertyWatchManager();

	/**
	 * Start watching a property
	 * @return Whether property was started to being watched. If property is already being watched this will return false.
	 */
	bool WatchProperty(uint64 InObjectId, uint32 InPropertyNameId);

	/**
	 * Stop watching a property
	 * @return Whether property was stopped being watched. If property is already not watched this will return false.
	 */
	bool UnwatchProperty(uint64 InObjectId, uint32 InPropertyNameId);

	/** @return The total number of watched properties for a specific object */
	int32 GetWatchedPropertiesNum(uint64 InObjectId) const;

	/** @return A view of all the watched properties for a specific object */
	TConstArrayView<uint32> GetWatchedProperties(uint64 InObjectId) const;

	/** Stop watching all the registered properties of a given object */
	void ClearWatchedProperties(uint64 InObjetId);

	/** Stop watching all registered properties */
	void ClearAllWatchedProperties();
	
	/** @return Delete that broadcast when any property has started to be watched */
	FOnPropertyWatched & OnPropertyWatched();

	/** @return Delete that broadcasts when any property has ceased to be watched */
	FOnPropertyUnwatched & OnPropertyUnwatched();
	
	/** Create singleton instance */
	static void Initialize();

	/** Destroy singleton instance */
	static void Shutdown();
	
	/** @return Current PropertyWatchManager singleton instance */
	static FPropertyWatchManager* Instance();
	
protected:
	
	/** ObjectId -> Object's watched properties */
	TMap<uint64, TArray<uint32>> WatchedProperties;

	/** Delete variable to broadcast when any property has started to be watched */
	FOnPropertyWatched OnPropertyWatchedDelegate;

	/** Delete variable to broadcast when any property has ceased to be watched */
	FOnPropertyUnwatched OnPropertyUnwatchedDelegate;

	/** Singleton instance */
	static FPropertyWatchManager* InternalInstance;
};
