// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimGraphSchematicView.h"
#include "AnimationProvider.h"
#include "Common/ProviderLock.h"
#include "Widgets/Input/SSearchBox.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "GameplayProvider.h"
#include "TraceServices/Model/Frames.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/CoreStyle.h"
#include "GameplayInsightsStyle.h"
#include "IRewindDebugger.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SImage.h"
#include "SVariantValueView.h"

#if WITH_EDITOR
#include "Animation/AnimInstance.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"
#include "IAnimationBlueprintEditor.h"
#include "ObjectTrace.h"
#endif

#define LOCTEXT_NAMESPACE "SAnimGraphSchematicView"

static const FName AnimGraphSchematicName("AnimGraphSchematic");

namespace AnimGraphSchematicPropertyColumns
{
	static const FName Name("Name");
	static const FName Value("Value");
};

namespace AnimGraphSchematicColumns
{
	static const FName Type("Type");
	static const FName Name("Name");
	static const FName Asset("Asset");
	static const FName Weight("Weight");
	static const FName RootMotionWeight("Root Motion Weight");
};

enum class EAnimGraphSchematicFilterState
{
	Hidden,
	Visible,
	Highlighted,
};

// A node in the tree of 'properties' for an animation node's debug info
class FAnimGraphSchematicPropertyNode
{
public:
	FAnimGraphSchematicPropertyNode(const FText& InName, const TSharedPtr<FAnimNodeValueMessage>& InValue, const TraceServices::IAnalysisSession& InAnalysisSession)
		: AnalysisSession(InAnalysisSession)
		, Name(InName)
		, Value(InValue)
	{}

	TSharedRef<SWidget> MakeValueWidget()
	{
		TSharedPtr<FAnimNodeValueMessage> PinnedValue = Value.Pin();
		if(PinnedValue.IsValid())
		{
			return SVariantValueView::MakeVariantValueWidget(AnalysisSession, PinnedValue.ToSharedRef()->Value, FText::GetEmpty());
		}
		else
		{
			return SNullWidget::NullWidget;
		}
	}

	const TraceServices::IAnalysisSession& AnalysisSession;

	FText Name;

	TWeakPtr<FAnimNodeValueMessage> Value;

	TWeakPtr<FAnimGraphSchematicPropertyNode> Parent;

	TArray<TSharedRef<FAnimGraphSchematicPropertyNode>> Children;
};

// Container for an entry in the property view
class SAnimGraphSchematicPropertyNode : public SMultiColumnTableRow<TSharedRef<FAnimGraphSchematicPropertyNode>>
{
public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FAnimGraphSchematicPropertyNode> InNode)
	{
		Node = InNode;

		SMultiColumnTableRow<TSharedRef<FAnimGraphSchematicPropertyNode>>::Construct(
			FSuperRowType::FArguments()
			.Padding(1.0f),
			InOwnerTable
		);
	}

	// SMultiColumnTableRow interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		const bool bIsRoot = !Node->Parent.IsValid();

		if (InColumnName == AnimGraphSchematicPropertyColumns::Name)
		{
			return 
				SNew(SBorder)
				.BorderImage(bIsRoot ? FGameplayInsightsStyle::Get().GetBrush("SchematicViewRootLeft") : FCoreStyle::Get().GetBrush("NoBorder"))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(6, 0, 0, 0)
					.VAlign(VAlign_Center)
					[
						SNew(SExpanderArrow, SharedThis(this))
						.IndentAmount(0)
					]
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(FCoreStyle::Get().GetFontStyle(bIsRoot ? "ExpandableArea.TitleFont" : "SmallFont"))
						.Text(Node->Name)
					]
				];
		}
		else if(InColumnName == AnimGraphSchematicPropertyColumns::Value)
		{
			return
				SNew(SBorder)
				.BorderImage(bIsRoot ? FGameplayInsightsStyle::Get().GetBrush("SchematicViewRootMid") : FCoreStyle::Get().GetBrush("NoBorder"))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						Node->MakeValueWidget()
					]
				];
		}

		return SNullWidget::NullWidget;
	}

	TSharedPtr<FAnimGraphSchematicPropertyNode> Node;
};

// Node representing debug data for an anim node
class FAnimGraphSchematicNode : public TSharedFromThis<FAnimGraphSchematicNode>
{
public:
	FAnimGraphSchematicNode(int32 InNodeId, uint64 InAnimInstanceId, const FText& InType, const TraceServices::IAnalysisSession& InAnalysisSession)
		: AnalysisSession(InAnalysisSession)
		, NodeId(InNodeId)
		, AnimInstanceId(InAnimInstanceId)
		, Type(InType)
		, FilterState(EAnimGraphSchematicFilterState::Hidden)
		, bLinearized(false)
	{
	}

	void MakePropertyNodes(TArray<TSharedRef<FAnimGraphSchematicPropertyNode>>& OutNodes)
	{
		// Add 'root' representing the node itself
		TSharedRef<FAnimGraphSchematicPropertyNode> Root = MakeShared<FAnimGraphSchematicPropertyNode>(Type, nullptr, AnalysisSession);
		OutNodes.Add(Root);

		for(TSharedRef<FAnimNodeValueMessage> Value : Values)
		{
			TSharedRef<FAnimGraphSchematicPropertyNode> NewNode = MakeShared<FAnimGraphSchematicPropertyNode>(FText::FromString(Value->Key), Value, AnalysisSession);
			Root->Children.Add(NewNode);
			NewNode->Parent = Root;
		}
	}

	const TraceServices::IAnalysisSession& AnalysisSession;

	int32 NodeId = INDEX_NONE;
	uint64 AnimInstanceId = INDEX_NONE;

	FText Type;

	TMap<FName, TSharedRef<FAnimNodeValueMessage>> KeysAndValues;

	TArray<TSharedRef<FAnimNodeValueMessage>> Values;

	TWeakPtr<FAnimGraphSchematicNode> Parent;

	TArray<TSharedRef<FAnimGraphSchematicNode>> Children;

	TArray<TSharedRef<FAnimGraphSchematicNode>> FlattenedLinearChildren;

	EAnimGraphSchematicFilterState FilterState;

	bool bLinearized;
};

class SAnimGraphSchematicNode : public SMultiColumnTableRow<TSharedRef<FAnimGraphSchematicNode>>
{
	SLATE_BEGIN_ARGS(SAnimGraphSchematicNode) {}

	SLATE_ATTRIBUTE(FText, FilterText)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FAnimGraphSchematicNode> InNode)
	{
		Node = InNode;
		FilterText = InArgs._FilterText;

		SMultiColumnTableRow<TSharedRef<FAnimGraphSchematicNode>>::Construct(
			FSuperRowType::FArguments()
			.Padding(1.0f),
			InOwnerTable
		);
	}

	// SMultiColumnTableRow interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		const bool bIsRoot = !Node->Parent.IsValid();

		if (InColumnName == AnimGraphSchematicColumns::Type)
		{
			IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();

			bool bHasValidInfo = false;
			FString AnimInstanceClassPathName;
			const int32 AnimNodeId = Node->NodeId;
			const uint64 AnimInstanceId = Node->AnimInstanceId;

			if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
			{
				if (const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName))
				{
					TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

					const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(Node->AnimInstanceId);
					const FClassInfo* AnimInstanceClassInfo = ObjectInfo ? GameplayProvider->FindClassInfo(ObjectInfo->ClassId) : nullptr;

					if (ObjectInfo && AnimInstanceClassInfo)
					{
						bHasValidInfo = true;
						AnimInstanceClassPathName = AnimInstanceClassInfo->PathName;
					}
				}
			}

			if (bHasValidInfo)
			{
#if WITH_EDITOR
				return
					SNew(SBorder)
					.BorderImage(bIsRoot ? FGameplayInsightsStyle::Get().GetBrush("SchematicViewRootLeft") : FCoreStyle::Get().GetBrush("NoBorder"))
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(6, 0, 0, 0)
						[
							SNew(SExpanderArrow, SharedThis(this))
							.IndentAmount(12)
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SHyperlink)
							.Text(Node->Type)
							.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
							.ToolTipText(FText::Format(LOCTEXT("AssetHyperlinkTooltipFormat", "Open node '{0}'"), Node->Type))
							.HighlightText(FilterText)
							.OnNavigate_Lambda([AnimNodeId, AnimInstanceClassPathName, AnimInstanceId]()
							{
								TSoftObjectPtr<UAnimBlueprintGeneratedClass> InstanceClass;
								InstanceClass = FSoftObjectPath(AnimInstanceClassPathName);

								if (InstanceClass.LoadSynchronous())
								{
									if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(InstanceClass.Get()->ClassGeneratedBy))
									{
										GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AnimBlueprint);

										if (UObject* SelectedInstance = FObjectTrace::GetObjectFromId(AnimInstanceId))
										{
											AnimBlueprint->SetObjectBeingDebugged(SelectedInstance);
										}

										if (IAnimationBlueprintEditor* AnimBlueprintEditor = static_cast<IAnimationBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AnimBlueprint, true)))
										{
											int32 AnimNodeIndex = InstanceClass.Get()->GetAnimNodeProperties().Num() - AnimNodeId - 1;
											TWeakObjectPtr<const UEdGraphNode>* GraphNode = InstanceClass.Get()->AnimBlueprintDebugData.NodePropertyIndexToNodeMap.Find(AnimNodeIndex);
											if (GraphNode != nullptr && GraphNode->Get())
											{
												AnimBlueprintEditor->JumpToHyperlink(GraphNode->Get());
											}
										}
									}
								}
							})
						]
					];
#else
				return
					SNew(SBorder)
					.BorderImage(bIsRoot ? FGameplayInsightsStyle::Get().GetBrush("SchematicViewRootLeft") : FCoreStyle::Get().GetBrush("NoBorder"))
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(6, 0, 0, 0)
						[
							SNew(SExpanderArrow, SharedThis(this))
							.IndentAmount(12)
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SHyperlink)
							.Text(Node->Type)
							.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
							.HighlightText(FilterText)
						]
					];
#endif
			}
			else
			{
				return
					SNew(SBorder)
					.BorderImage(bIsRoot ? FGameplayInsightsStyle::Get().GetBrush("SchematicViewRootLeft") : FCoreStyle::Get().GetBrush("NoBorder"))
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(6, 0, 0, 0)
						[
							SNew(SExpanderArrow, SharedThis(this))
							.IndentAmount(12)
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SHyperlink)
							.Text(Node->Type)
							.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
							.HighlightText(FilterText)
						]
					];
			}
		}
		else
		{
			TSharedPtr<SWidget> ValueWidget = SNullWidget::NullWidget;
			TSharedRef<FAnimNodeValueMessage>* ValuePtr = Node->KeysAndValues.Find(InColumnName);
			if(ValuePtr)
			{
				ValueWidget = SVariantValueView::MakeVariantValueWidget(Node->AnalysisSession, (*ValuePtr)->Value, FText::GetEmpty());
			}

			return
				SNew(SBorder)
				.BorderImage(bIsRoot ? FGameplayInsightsStyle::Get().GetBrush("SchematicViewRootMid") : FCoreStyle::Get().GetBrush("NoBorder"))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						ValueWidget.ToSharedRef()
					]
				];
		}
	}

	TSharedPtr<FAnimGraphSchematicNode> Node;

	TAttribute<FText> FilterText;
};

void SAnimGraphSchematicView::Construct(const FArguments& InArgs, uint64 InAnimInstanceId, double InTimeMarker, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	AnimInstanceId = InAnimInstanceId;
	TimeMarker = InTimeMarker;
	AnalysisSession = &InAnalysisSession;

	// Create header row and add default columns
	HeaderRow = SNew(SHeaderRow);

	// Make default columns
	Columns.Add(AnimGraphSchematicColumns::Type, { 0, true });
	Columns.Add(AnimGraphSchematicColumns::Name, { 1, true });
	Columns.Add(AnimGraphSchematicColumns::Asset, { 2, true });
	Columns.Add(AnimGraphSchematicColumns::Weight, { 99999, true });
	Columns.Add(AnimGraphSchematicColumns::RootMotionWeight, { 100000, false });

	TreeView = SNew(STreeView<TSharedRef<FAnimGraphSchematicNode>>)
			.TreeItemsSource(&FilteredNodes)
			.OnGenerateRow(this, &SAnimGraphSchematicView::HandleGenerateRow)
			.OnGetChildren(this, &SAnimGraphSchematicView::HandleGetChildren)
			.HeaderRow(HeaderRow.ToSharedRef())
			.OnSelectionChanged(this, &SAnimGraphSchematicView::HandleSelectionChanged);

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(SearchBox, SSearchBox)
			.OnTextChanged_Lambda([this](const FText& InText){ FilterText = InText; RefreshFilter(); })
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(Splitter, SSplitter)
			.Orientation(Orient_Vertical)
			+SSplitter::Slot()
			.Value(0.7f)
			[
				SNew(SScrollBorder, TreeView.ToSharedRef())
				[
					TreeView.ToSharedRef()
				]
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		[
			SAssignNew(ViewButton, SComboButton)
			.ContentPadding(0)
			.ForegroundColor(this, &SAnimGraphSchematicView::GetViewButtonForegroundColor)
			.ButtonStyle(FGameplayInsightsStyle::Get(), "SchematicViewViewButton")
			.OnGetMenuContent(this, &SAnimGraphSchematicView::HandleGetViewMenuContent)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FGameplayInsightsStyle::Get().GetBrush("SchematicViewViewButtonIcon"))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2, 0, 0, 0)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ViewButton", "View Options"))
				]
			]
		]
	];

	RefreshColumns();
	RefreshNodes();
}

TSharedRef<ITableRow> SAnimGraphSchematicView::HandleGenerateRow(TSharedRef<FAnimGraphSchematicNode> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return 
		SNew(SAnimGraphSchematicNode, OwnerTable, Item)
		.FilterText_Lambda([this](){ return FilterText; });
}

void SAnimGraphSchematicView::HandleGetChildren(TSharedRef<FAnimGraphSchematicNode> InItem, TArray<TSharedRef<FAnimGraphSchematicNode>>& OutChildren)
{
	for(const TSharedRef<FAnimGraphSchematicNode>& Child : InItem->FlattenedLinearChildren)
	{
		if(Child->FilterState != EAnimGraphSchematicFilterState::Hidden)
		{
			OutChildren.Add(Child);
		}
	}
}

TSharedRef<ITableRow> SAnimGraphSchematicView::HandleGeneratePropertyRow(TSharedRef<FAnimGraphSchematicPropertyNode> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return 
		SNew(SAnimGraphSchematicPropertyNode, OwnerTable, Item);
}

void SAnimGraphSchematicView::HandleGetPropertyChildren(TSharedRef<FAnimGraphSchematicPropertyNode> InItem, TArray<TSharedRef<FAnimGraphSchematicPropertyNode>>& OutChildren)
{
	OutChildren.Append(InItem->Children);
}

// Add children recursively if only one child is present, linearizing runs of single children
static void AddChildren_Helper(TSharedRef<FAnimGraphSchematicNode> InItem, TArray<TSharedRef<FAnimGraphSchematicNode>>& OutChildren)
{
	if(InItem->FlattenedLinearChildren.Num() == 0)
	{
		if(InItem->Children.Num() == 1)
		{
			if(!InItem->Children[0]->bLinearized)
			{
				InItem->Children[0]->bLinearized = true;
				OutChildren.Add(InItem->Children[0]);
				AddChildren_Helper(InItem->Children[0], OutChildren);
			}
		}
		else if(InItem->Children.Num() > 1)
		{
			for(const TSharedRef<FAnimGraphSchematicNode>& Child : InItem->Children)
			{
				if(!Child->bLinearized)
				{
					Child->bLinearized = true;
					OutChildren.Add(Child);
				}
			}
		}
	}
}

void SAnimGraphSchematicView::RefreshNodes()
{
	UnfilteredNodes.Reset();
	FilteredNodes.Reset();
	LinearNodes.Reset();
	
	TMap<int32, TSharedRef<FAnimGraphSchematicNode>> NodeMap;

	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
	const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);

	if(AnimationProvider && GameplayProvider)
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);

		AnimationProvider->ReadAnimGraphTimeline(AnimInstanceId, [this, &NodeMap, AnimationProvider, GameplayProvider](const FAnimationProvider::AnimGraphTimeline& InGraphTimeline)
		{
			bool bFrameFound;
			TraceServices::FFrame Frame;
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
				const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
				bFrameFound = FrameProvider.GetFrameFromTime(TraceFrameType_Game, TimeMarker, Frame);
			}

			if (bFrameFound)
			{
				InGraphTimeline.EnumerateEvents(Frame.StartTime, Frame.EndTime, [this, &NodeMap, AnimationProvider, GameplayProvider](double InGraphStartTime, double InGraphEndTime, uint32 InDepth, const FAnimGraphMessage& InMessage)
				{
					// Check for an update phase (which contains weights)
					if(InMessage.Phase == EAnimGraphPhase::Update)
					{
						AnimationProvider->ReadAnimNodesTimeline(AnimInstanceId, [this, &NodeMap, InGraphStartTime, InGraphEndTime](const FAnimationProvider::AnimNodesTimeline& InNodesTimeline)
						{
							InNodesTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [this, &NodeMap](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNodeMessage& InMessage)
							{
								if(InMessage.NodeId != INDEX_NONE && InMessage.Phase == EAnimGraphPhase::Update)
								{
									TSharedPtr<FAnimGraphSchematicNode> Node;
									TSharedRef<FAnimGraphSchematicNode>* ExistingNodePtr = NodeMap.Find(InMessage.NodeId);
									if(ExistingNodePtr == nullptr)
									{
										Node = NodeMap.Add(InMessage.NodeId, MakeShared<FAnimGraphSchematicNode>(InMessage.NodeId, AnimInstanceId, FText::FromString(InMessage.NodeName), *AnalysisSession));

										// Add dummy values for weight and root motion weight
										TSharedRef<FAnimNodeValueMessage> WeightMessage = MakeShared<FAnimNodeValueMessage>();
										WeightMessage->NodeId = InMessage.NodeId;
										WeightMessage->FrameCounter = InMessage.FrameCounter;
										WeightMessage->Value.Type = EAnimNodeValueType::Float;
										WeightMessage->Key = TEXT("Weight");
										WeightMessage->Value.Float.Value = InMessage.Weight;

										Node->KeysAndValues.Add(AnimGraphSchematicColumns::Weight, WeightMessage);
										Node->Values.Add(WeightMessage);

										TSharedRef<FAnimNodeValueMessage> RootMotionWeightMessage = MakeShared<FAnimNodeValueMessage>();
										RootMotionWeightMessage->NodeId = InMessage.NodeId;
										RootMotionWeightMessage->FrameCounter = InMessage.FrameCounter;
										RootMotionWeightMessage->Value.Type = EAnimNodeValueType::Float;
										RootMotionWeightMessage->Key = TEXT("Root Motion Weight");
										RootMotionWeightMessage->Value.Float.Value = InMessage.RootMotionWeight;

										Node->KeysAndValues.Add(AnimGraphSchematicColumns::RootMotionWeight, RootMotionWeightMessage);
										Node->Values.Add(RootMotionWeightMessage);

										LinearNodes.Add(Node.ToSharedRef());
									}
									else
									{
										Node = *ExistingNodePtr;
									}

									if(InMessage.PreviousNodeId != INDEX_NONE)
									{
										check(InMessage.NodeId != InMessage.PreviousNodeId);
										TSharedRef<FAnimGraphSchematicNode>* ExistingPreviousNodePtr = NodeMap.Find(InMessage.PreviousNodeId);
										if(ExistingPreviousNodePtr != nullptr)
										{
											TSharedRef<FAnimGraphSchematicNode> PreviousNode = *ExistingPreviousNodePtr;
											PreviousNode->Children.Add(Node.ToSharedRef());
											Node->Parent = PreviousNode;
										}
									}
								}
								return TraceServices::EEventEnumerate::Continue;
							});
						});

						// TODO: Add visualizations of the reset of the data below
// 						AnimationProvider->ReadStateMachinesTimeline(AnimInstanceId, [InGraphStartTime, InGraphEndTime](const FAnimationProvider::StateMachinesTimeline& InStateMachinesTimeline)
// 						{
// 							InStateMachinesTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [](double InStartTime, double InEndTime, uint32 InDepth, const FAnimStateMachineMessage& InMessage)
// 							{
// 								//DebugData.RecordStateData(InMessage.StateMachineIndex, InMessage.StateIndex, InMessage.StateWeight, InMessage.ElapsedTime);
// 							});
// 						});
//						
// 						AnimationProvider->ReadAnimSequencePlayersTimeline(AnimInstanceId, [InGraphStartTime, InGraphEndTime, GameplayProvider](const FAnimationProvider::AnimSequencePlayersTimeline& InSequencePlayersTimeline)
// 						{
// 							InSequencePlayersTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [](double InStartTime, double InEndTime, uint32 InDepth, const FAnimSequencePlayerMessage& InMessage)
// 							{
// 								//DebugData.RecordSequencePlayer(InMessage.NodeId, InMessage.Position, InMessage.Length, InMessage.FrameCount);
// 							});
// 						});
// 
// 						AnimationProvider->ReadAnimBlendSpacePlayersTimeline(AnimInstanceId, [InGraphStartTime, InGraphEndTime, GameplayProvider](const FAnimationProvider::BlendSpacePlayersTimeline& InBlendSpacePlayersTimeline)
// 						{
// 							InBlendSpacePlayersTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [GameplayProvider](double InStartTime, double InEndTime, uint32 InDepth, const FBlendSpacePlayerMessage& InMessage)
// 							{
// // 								UBlendSpace* BlendSpace = nullptr;
// // 								const FObjectInfo* BlendSpaceInfo = GameplayProvider->FindObjectInfo(InMessage.BlendSpaceId);
// // 								if(BlendSpaceInfo)
// // 								{
// // 									BlendSpace = TSoftObjectPtr<UBlendSpace>(FSoftObjectPath(BlendSpaceInfo->PathName)).LoadSynchronous();
// // 								}
// // 
// // 								DebugData.RecordBlendSpacePlayer(InMessage.NodeId, BlendSpace, InMessage.PositionX, InMessage.PositionY, InMessage.PositionZ);
// 							});
// 						});

					}

					AnimationProvider->ReadAnimNodeValuesTimeline(AnimInstanceId, [this, InGraphStartTime, InGraphEndTime, AnimationProvider, &NodeMap](const FAnimationProvider::AnimNodeValuesTimeline& InNodeValuesTimeline)
					{
						InNodeValuesTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [this, AnimationProvider, &NodeMap](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNodeValueMessage& InMessage)
						{
							TSharedRef<FAnimGraphSchematicNode>* ExistingNodePtr = NodeMap.Find(InMessage.NodeId);
							if(ExistingNodePtr != nullptr)
							{
								TSharedRef<FAnimGraphSchematicNode> ExistingNode = *ExistingNodePtr;

								FName Key = InMessage.Key;

								// Add to columns (+2 to leave space for the fixed type and name columns in the index range)
								Columns.FindOrAdd(Key, { Columns.Num() + 2, false });
								TSharedRef<FAnimNodeValueMessage> SharedMessage = MakeShared<FAnimNodeValueMessage>();
								SharedMessage.Get() = InMessage;
								ExistingNode->KeysAndValues.Add(Key, SharedMessage);
								ExistingNode->Values.Add(SharedMessage);
							}
							return TraceServices::EEventEnumerate::Continue;
						});
					});
					return TraceServices::EEventEnumerate::Continue;
				});
			};
		});
	}

	// Re-sort columns
	Columns.ValueSort([](const FColumnState& Column0, const FColumnState& Column1)
	{
		return Column0.SortIndex < Column1.SortIndex;
	});

	// Add root nodes to unfiltered list
	for(const TSharedRef<FAnimGraphSchematicNode>& Node : LinearNodes)
	{
		if(!Node->Parent.IsValid())
		{
			UnfilteredNodes.Add(Node);
		}
	}

	// Build flattened linear children from runs of single-child nodes
	for(const TSharedRef<FAnimGraphSchematicNode>& Node : LinearNodes)
	{
		AddChildren_Helper(Node, Node->FlattenedLinearChildren);
	}

	RefreshFilter();
}

EAnimGraphSchematicFilterState SAnimGraphSchematicView::RefreshFilter_Helper(const TSharedRef<FAnimGraphSchematicNode>& InNode)
{
	InNode->FilterState = EAnimGraphSchematicFilterState::Hidden;

	for(const TSharedRef<FAnimGraphSchematicNode>& ChildNode : InNode->FlattenedLinearChildren)
	{
		EAnimGraphSchematicFilterState ChildFilterState = RefreshFilter_Helper(ChildNode);
		InNode->FilterState = FMath::Max(ChildFilterState, InNode->FilterState);
	}
	
	if(InNode->FilterState == EAnimGraphSchematicFilterState::Hidden)
	{
		if(FilterText.IsEmpty())
		{
			InNode->FilterState = EAnimGraphSchematicFilterState::Visible;
		}
		else if (InNode->Type.ToString().Contains(FilterText.ToString()))
		{
			InNode->FilterState = EAnimGraphSchematicFilterState::Highlighted;
		}
		else
		{
			InNode->FilterState = EAnimGraphSchematicFilterState::Hidden;
		}
	}

	bool bAutoExpand = (FilterText.IsEmpty() && InNode->FilterState != EAnimGraphSchematicFilterState::Hidden) ||
					  (!FilterText.IsEmpty() && InNode->FilterState == EAnimGraphSchematicFilterState::Highlighted);
	TreeView->SetItemExpansion(InNode, bAutoExpand);

	return InNode->FilterState;
}

void SAnimGraphSchematicView::RefreshFilter()
{
	FilteredNodes.Reset();

	for(const TSharedRef<FAnimGraphSchematicNode>& RootNode : UnfilteredNodes)
	{
		EAnimGraphSchematicFilterState FilterState = RefreshFilter_Helper(RootNode);
		if(FilterState != EAnimGraphSchematicFilterState::Hidden)
		{
			FilteredNodes.Add(RootNode);
		}
	}

	// re-select any previously selected nodes
	TArray<TSharedRef<FAnimGraphSchematicNode>> SelectedItems;
	for(const TSharedRef<FAnimGraphSchematicNode>& Node : LinearNodes)
	{
		if(SelectedNodeIds.Contains(Node->NodeId))
		{
			SelectedItems.Add(Node);
		}
	}

	if(SelectedItems.Num() > 0)
	{
		TreeView->SetItemSelection(SelectedItems, true);
		TreeView->RequestScrollIntoView(SelectedItems.Last());
	}

	TreeView->RequestTreeRefresh();
}

void SAnimGraphSchematicView::SetTimeMarker(double InTimeMarker)
{
	if (TimeMarker != InTimeMarker)
	{
		TimeMarker = InTimeMarker;
		RefreshNodes();
	}
}


FName SAnimGraphSchematicView::GetName() const
{
	return AnimGraphSchematicName;
}

TSharedRef<SWidget> SAnimGraphSchematicView::HandleGetViewMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection("Columns", LOCTEXT("ColumnsMenuHeader", "Columns"));
	{
		if(Columns.Num() > 0)
		{
			for(const TPair<FName, FColumnState>& ColumnPair : Columns)
			{
				const FName ColumnId = ColumnPair.Key;

				MenuBuilder.AddMenuEntry(
					FText::FromName(ColumnId), 
					LOCTEXT("ColumnTooltip", "Enable/disable this column"),
					FSlateIcon(), 
					FUIAction(
						FExecuteAction::CreateLambda([this, ColumnId]()
						{
							if(FColumnState* StatePtr = Columns.Find(ColumnId))
							{
								StatePtr->bEnabled = !StatePtr->bEnabled;
							}

							RefreshColumns();
						}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([this, ColumnId]()
						{
							return Columns.FindRef(ColumnId).bEnabled;
						})
					),
					NAME_None,
					EUserInterfaceActionType::Check
				);
			}
		}
		else
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("NoColumns", "No Optional Columns Found"), 
				LOCTEXT("NoColumnsTooltip", "No Optional Columns Found in the Current Session"), 
				FSlateIcon(), 
				FUIAction()
			);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SAnimGraphSchematicView::RefreshColumns()
{
	HeaderRow->ClearColumns();

	Columns.ValueSort([](const FColumnState& Column0, const FColumnState& Column1)
	{
		return Column0.SortIndex < Column1.SortIndex;
	});

	for(const TPair<FName, FColumnState>& ColumnPair : Columns)
	{
		if(ColumnPair.Value.bEnabled)
		{
			HeaderRow->AddColumn(
				SHeaderRow::FColumn::FArguments()
				.ColumnId(ColumnPair.Key)
				.DefaultLabel(FText::FromName(ColumnPair.Key))
			);
		}
	}
}

FSlateColor SAnimGraphSchematicView::GetViewButtonForegroundColor() const
{
	static const FName InvertedForegroundName("InvertedForeground");
	static const FName DefaultForegroundName("DefaultForeground");

	return ViewButton->IsHovered() ? FCoreStyle::Get().GetSlateColor(InvertedForegroundName) : FCoreStyle::Get().GetSlateColor(DefaultForegroundName);
}

void SAnimGraphSchematicView::HandleSelectionChanged(TSharedPtr<FAnimGraphSchematicNode> InNode, ESelectInfo::Type InSelectInfo)
{
	TArray<TSharedRef<FAnimGraphSchematicNode>> SelectedNodes;
	TreeView->GetSelectedItems(SelectedNodes);

	// Preserve selection of node IDs so scrubbing will re-select them
	if(InSelectInfo != ESelectInfo::Direct)
	{
		SelectedNodeIds.Reset();
		for(const TSharedRef<FAnimGraphSchematicNode>& Item : SelectedNodes)
		{
			SelectedNodeIds.Add(Item->NodeId);
		}
	}

	RefreshDetails(SelectedNodes);
}

void SAnimGraphSchematicView::RefreshDetails(const TArray<TSharedRef<FAnimGraphSchematicNode>>& InNodes)
{
	if(InNodes.Num() > 0)
	{
		// Nodes to show, so create/re-use content box in splitter
		TSharedPtr<SVerticalBox> VerticalBox;

		if(DetailsContentBox.IsValid())
		{
			VerticalBox = DetailsContentBox.Pin();
			VerticalBox->ClearChildren();
		}
		else
		{
			VerticalBox = SNew(SVerticalBox);
			DetailsContentBox = VerticalBox;

			Splitter->AddSlot()
			.Value(0.3f)
			[
				VerticalBox.ToSharedRef()
			];
		}

		// Add content for each node
		PropertyNodes.Reset();

		for(const TSharedRef<FAnimGraphSchematicNode>& Node : InNodes)
		{
			Node->MakePropertyNodes(PropertyNodes);
		}

		PropertyTreeView = SNew(STreeView<TSharedRef<FAnimGraphSchematicPropertyNode>>)
			.SelectionMode(ESelectionMode::None)
			.OnGenerateRow(this, &SAnimGraphSchematicView::HandleGeneratePropertyRow)
			.OnGetChildren(this, &SAnimGraphSchematicView::HandleGetPropertyChildren)
			.TreeItemsSource(&PropertyNodes)
			.HeaderRow(
				SNew(SHeaderRow)
				.Visibility(EVisibility::Collapsed)
				+SHeaderRow::Column(AnimGraphSchematicPropertyColumns::Name)
				.DefaultLabel(LOCTEXT("PropertiesNameColumn", "Name"))
				+SHeaderRow::Column(AnimGraphSchematicPropertyColumns::Value)
				.DefaultLabel(LOCTEXT("PropertiesValueColumn", "Value"))
			);

		for(const TSharedRef<FAnimGraphSchematicPropertyNode>& RootNode : PropertyNodes)
		{
			PropertyTreeView->SetItemExpansion(RootNode, true);
		}

		VerticalBox->AddSlot()
			.FillHeight(1.0f)
			[
				SNew(SScrollBorder, PropertyTreeView.ToSharedRef())
				[
					PropertyTreeView.ToSharedRef()
				]
			];
	}
	else
	{
		// no nodes to show, so remove splitter (and therefore content) if valid
		if(DetailsContentBox.IsValid())
		{
			DetailsContentBox.Pin()->ClearChildren();
			Splitter->RemoveAt(1);
		}
	}
}

#if WITH_ENGINE
FName FAnimGraphSchematicTrackCreator::GetTargetTypeNameInternal() const
{
	return UAnimInstance::StaticClass()->GetFName();
}

FName FAnimGraphSchematicTrackCreator::GetNameInternal() const
{
	return AnimGraphSchematicName;
}

void FAnimGraphSchematicTrackCreator::GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const
{
	Types.Add({AnimGraphSchematicName, LOCTEXT("Anim Graph", "Anim Graph")});
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FAnimGraphSchematicTrackCreator::CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	return MakeShared<FAnimGraphSchematicTrack>(InObjectId.GetMainId());
}
#endif // WITH_ENGINE

FText FAnimGraphSchematicTrack::GetDisplayNameInternal() const
{
	return LOCTEXT("Anim Graph Update", "Anim Graph");
}

FName FAnimGraphSchematicTrack::GetNameInternal() const
{
	return AnimGraphSchematicName;
}

FSlateIcon FAnimGraphSchematicTrack::GetIconInternal()
{
#if WITH_EDITOR
	return FSlateIconFinder::FindIconForClass(UAnimInstance::StaticClass());
#else
	return FSlateIcon();
#endif
}

TSharedPtr<SWidget> FAnimGraphSchematicTrack::GetDetailsViewInternal()
{
	TSharedPtr<SAnimGraphSchematicView> NewView;
	
	if (IRewindDebugger* RewindDebugger = IRewindDebugger::Instance())
	{
		if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
		{
			if (!View.IsValid())
			{
				NewView = SNew(SAnimGraphSchematicView, AnimInstanceId, RewindDebugger->CurrentTraceTime(), *AnalysisSession);
				View = NewView.ToWeakPtr();
			}
		}
	}

	return NewView;
}

bool FAnimGraphSchematicTrack::UpdateInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	TSharedPtr<IRewindDebuggerView> PinnedView = View.Pin();
	if (PinnedView.IsValid())
	{
		PinnedView->SetTimeMarker(RewindDebugger->CurrentTraceTime());
	}

	return false;
}

#if WITH_ENGINE
bool FAnimGraphSchematicTrackCreator::HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	const TraceServices::IAnalysisSession* AnalysisSession = IRewindDebugger::Instance()->GetAnalysisSession();
	
	bool bHasData = false;
	if (const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName))
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadAnimGraphTimeline(InObjectId.GetMainId(), [&bHasData](const FAnimationProvider::AnimGraphTimeline& InGraphTimeline)
		{
			bHasData = true;
		});
	}
	
	return bHasData;
}
#endif // WITH_ENGINE

#undef LOCTEXT_NAMESPACE
