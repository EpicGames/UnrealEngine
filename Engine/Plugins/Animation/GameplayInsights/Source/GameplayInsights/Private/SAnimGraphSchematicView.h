// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IRewindDebuggerTrackCreator.h"
#include "IRewindDebuggerView.h"
#include "RewindDebuggerTrack.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

namespace TraceServices { class IAnalysisSession; }
namespace UE::Insights::Timing { class ITimingViewSession; }
namespace UE::Insights::Timing { enum class ETimeChangedFlags; }
class FAnimationSharedData;
class IInsightsManager;
class FAnimGraphSchematicNode;
class FAnimGraphSchematicPropertyNode;
class SSearchBox;
class SHeaderRow;
class SComboButton;
class SSplitter;
enum class EAnimGraphSchematicFilterState;

class SAnimGraphSchematicView : public IRewindDebuggerView
{
	SLATE_BEGIN_ARGS(SAnimGraphSchematicView) {}
	SLATE_END_ARGS()
public:
	void Construct(const FArguments& InArgs, uint64 InAnimInstanceId, double InTimeMarker, const TraceServices::IAnalysisSession& InAnalysisSession);

	uint64 GetAnimInstanceId() const { return AnimInstanceId; }

	virtual void SetTimeMarker(double InTimeMarker) override;
	virtual FName GetName() const override;
	virtual uint64 GetObjectId() const override { return AnimInstanceId; }

private:
	// Generate a row widget for an item
	TSharedRef<ITableRow> HandleGenerateRow(TSharedRef<FAnimGraphSchematicNode> Item, const TSharedRef<STableViewBase>& OwnerTable);

	// Get the children of an item
	void HandleGetChildren(TSharedRef<FAnimGraphSchematicNode> InItem, TArray<TSharedRef<FAnimGraphSchematicNode>>& OutChildren);

	// Generate a row widget for a property item
	TSharedRef<ITableRow> HandleGeneratePropertyRow(TSharedRef<FAnimGraphSchematicPropertyNode> Item, const TSharedRef<STableViewBase>& OwnerTable);

	// Get the children of a property item
	void HandleGetPropertyChildren(TSharedRef<FAnimGraphSchematicPropertyNode> InItem, TArray<TSharedRef<FAnimGraphSchematicPropertyNode>>& OutChildren);

	// Handle the time marker being scrubbed
	void HandleTimeMarkerChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InTimeMarker);

	// Handle tree selection changing
	void HandleSelectionChanged(TSharedPtr<FAnimGraphSchematicNode> InNode, ESelectInfo::Type InSelectInfo);

	// Refresh the unfiltered and filtered nodes
	void RefreshNodes();

	// Recursive helper to filter tree
	EAnimGraphSchematicFilterState RefreshFilter_Helper(const TSharedRef<FAnimGraphSchematicNode>& InNode);

	// Refresh the displayed columns
	void RefreshColumns();

	// Refresh the filtered nodes
	void RefreshFilter();

	// Refresh the details panel
	void RefreshDetails(const TArray<TSharedRef<FAnimGraphSchematicNode>>& InNodes);

	// Handle getting content for the view menu
	TSharedRef<SWidget> HandleGetViewMenuContent();

	// Handle inverting the color of the button text when hovered
	FSlateColor GetViewButtonForegroundColor() const;

private:
	const TraceServices::IAnalysisSession* AnalysisSession;

	TSharedPtr<SSearchBox> SearchBox;

	TSharedPtr<SHeaderRow> HeaderRow;

	TSharedPtr<SSplitter> Splitter;

	TSharedPtr<STreeView<TSharedRef<FAnimGraphSchematicNode>>> TreeView;

	TSharedPtr<STreeView<TSharedRef<FAnimGraphSchematicPropertyNode>>> PropertyTreeView;

	TSharedPtr<SComboButton> ViewButton;

	TWeakPtr<SVerticalBox> DetailsContentBox;

	TArray<TSharedRef<FAnimGraphSchematicNode>> UnfilteredNodes;

	TArray<TSharedRef<FAnimGraphSchematicNode>> FilteredNodes;

	TArray<TSharedRef<FAnimGraphSchematicNode>> LinearNodes;

	TArray<TSharedRef<FAnimGraphSchematicPropertyNode>> PropertyNodes;

	TSet<int32> SelectedNodeIds;

	struct FColumnState
	{
		int32 SortIndex = 0;
		bool bEnabled = false;
	};

	TMap<FName, FColumnState> Columns;

	FText FilterText;

	double TimeMarker;

	uint64 AnimInstanceId;
};

class FAnimGraphSchematicTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:

	FAnimGraphSchematicTrack(uint64 InObjectId)
		: AnimInstanceId(InObjectId)
	{
	}

private:
	virtual FSlateIcon GetIconInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual FName GetNameInternal() const override;
	virtual FText GetDisplayNameInternal() const override;
	virtual uint64 GetObjectIdInternal() const override { return AnimInstanceId; }
	virtual bool UpdateInternal() override;

	TWeakPtr<SAnimGraphSchematicView> View;
	FSlateIcon Icon;
	uint64 AnimInstanceId;
};

#if WITH_ENGINE
class FAnimGraphSchematicTrackCreator : public RewindDebugger::IRewindDebuggerTrackCreator
{
public:
	virtual FName GetTargetTypeNameInternal() const override;
	virtual FName GetNameInternal() const override;
	virtual void GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const override;
};
#endif // WITH_ENGINE
