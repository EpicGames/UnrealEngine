// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimationCurvesView.h"
#include "AnimationProvider.h"
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Frames.h"
#include "VariantTreeNode.h"

#define LOCTEXT_NAMESPACE "SAnimationCurvesView"

void SAnimationCurvesView::GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const
{
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if (AnimationProvider)
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);

		TSharedRef<FVariantTreeNode> Header = OutVariants.Add_GetRef(FVariantTreeNode::MakeHeader(LOCTEXT("Animation Curves", "Animation Curves"), 0));

		AnimationProvider->ReadSkeletalMeshPoseTimeline(ObjectId, [this, &Header, &AnimationProvider, &InFrame](const FAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bInHasCurves)
		{
			InTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, [this ,&Header, &AnimationProvider, &InFrame](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InMessage)
			{
				if(InStartTime >= InFrame.StartTime && InStartTime <= InFrame.EndTime)
				{
					AnimationProvider->EnumerateSkeletalMeshCurves(InMessage, [this, &Header, &AnimationProvider](const FSkeletalMeshNamedCurve& InCurve)
					{
						if (!bCurveFilterSet || InCurve.Id == CurveFilter)
						{
							const TCHAR* CurveName = AnimationProvider->GetName(InCurve.Id);
							Header->AddChild(FVariantTreeNode::MakeFloat(FText::FromString(CurveName), InCurve.Value));
						}
					});
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}

static const FName AnimationCurvesName("AnimationCurves");
FName SAnimationCurvesView::GetName() const
{
	return AnimationCurvesName;
}

#undef LOCTEXT_NAMESPACE
