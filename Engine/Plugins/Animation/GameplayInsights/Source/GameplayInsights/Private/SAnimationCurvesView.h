// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SPropertiesDebugViewBase.h"

namespace TraceServices { class IAnalysisSession; }

class SAnimationCurvesView : public SPropertiesDebugViewBase
{
public:
	virtual void GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const override;
	virtual FName GetName() const override;
	
	void SetCurveFilter(uint32 InCurveFilter)
	{
		CurveFilter = InCurveFilter;
		bCurveFilterSet = true;
	}

	void ClearCurveFilter()
	{
		bCurveFilterSet = false;
	}

private:
	uint32 CurveFilter = 0;
	bool bCurveFilterSet = false;
};