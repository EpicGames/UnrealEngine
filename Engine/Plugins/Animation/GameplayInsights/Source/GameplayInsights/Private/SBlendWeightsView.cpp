// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBlendWeightsView.h"
#include "AnimationProvider.h"
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "GameplayProvider.h"
#include "TraceServices/Model/Frames.h"
#include "VariantTreeNode.h"

#define LOCTEXT_NAMESPACE "SBlendWeightsView"

void SBlendWeightsView::GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const
{
	const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if(GameplayProvider && AnimationProvider)
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);

		AnimationProvider->ReadTickRecordTimeline(ObjectId, [this, &AnimationProvider, &GameplayProvider, &OutVariants, &InFrame](const FAnimationProvider::TickRecordTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, [this, &AnimationProvider, &GameplayProvider, &OutVariants, &InFrame](double InStartTime, double InEndTime, uint32 InDepth, const FTickRecordMessage& InMessage)
			{
				if(InStartTime >= InFrame.StartTime && InEndTime <= InFrame.EndTime)
				{
					if (bAssetFilterSet)
					{
						if (InMessage.NodeId != NodeIdFilter || InMessage.AssetId != AssetIdFilter)
						{
							return TraceServices::EEventEnumerate::Continue;
						}
					}
					
					const FClassInfo& ClassInfo = GameplayProvider->GetClassInfoFromObject(InMessage.AssetId);
					TSharedRef<FVariantTreeNode> Header = OutVariants.Add_GetRef(FVariantTreeNode::MakeObject(FText::FromString(ClassInfo.Name), InMessage.AssetId, InMessage.AssetId,
																								InMessage.PlaybackTime, InMessage.BlendSpaceFilteredPositionX, InMessage.BlendSpaceFilteredPositionY));
					
					Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("BlendWeight", "Blend Weight"), InMessage.BlendWeight));
					Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("PlaybackTime", "Playback Time"), InMessage.PlaybackTime));
					Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("RootMotionWeight", "Root Motion Weight"), InMessage.RootMotionWeight));
					Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("PlayRate", "Play Rate"), InMessage.PlayRate));
					
					if(InMessage.bIsBlendSpace)
					{
						Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("BlendSpacePositionX", "Blend Space Position X"), InMessage.BlendSpacePositionX));
						Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("BlendSpacePositionY", "Blend Space Position Y"), InMessage.BlendSpacePositionY));
						Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("BlendSpaceFilteredPositionX", "Blend Space Filtered Position X"), InMessage.BlendSpaceFilteredPositionX));
						Header->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("BlendSpaceFilteredPositionY", "Blend Space Filtered Position Y"), InMessage.BlendSpaceFilteredPositionY));
					}

					static FString NoGroupName(LOCTEXT("NoSyncGroupName", "None").ToString());
					const TCHAR* CurveName = InMessage.bSyncGroupNameId == 0 ? *NoGroupName : AnimationProvider->GetName(InMessage.bSyncGroupNameId);
					TSharedRef<FVariantTreeNode> SyncGroup = FVariantTreeNode::MakeString(LOCTEXT("SyncGroupName", "Sync Group Name"), CurveName);
					Header->AddChild(SyncGroup);
					
					// @todo: I'd like to query the name of the sync markers given their index tho I'd prefer to load and cache the related asset in the respective blend weight track to avoid loading it every time here. Additionally, I'd like to add sync groups sub-tracks / curve tracks in a follow up CL.
					if (InMessage.bSyncGroupNameId != 0)
					{
						SyncGroup->AddChild(FVariantTreeNode::MakeBool(LOCTEXT("IsSyncGroupLeader", "Is Sync Group Leader"), InMessage.bIsSyncGroupLeader));
						SyncGroup->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("SyncLeaderScore", "Sync Leader Score"), InMessage.SyncLeaderScore));
						SyncGroup->AddChild(FVariantTreeNode::MakeInt32(LOCTEXT("PreviousMarker", "Previous Marker"), InMessage.PrevMarkerIndex));
						SyncGroup->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("PrevMarkerTime", "Time To Previous Marker"), InMessage.TimeToPrevMarker));
						SyncGroup->AddChild(FVariantTreeNode::MakeInt32(LOCTEXT("NextMarker", "Next Marker"), InMessage.NextMarkerIndex));
						SyncGroup->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("NextMarkerTime", "Time To Next Marker"), InMessage.TimeToNextMarker));
					}
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}

static const FName BlendWeightsName("BlendWeights");

FName SBlendWeightsView::GetName() const
{
	return BlendWeightsName;
}

#undef LOCTEXT_NAMESPACE
