// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SPropertiesDebugViewBase.h"

namespace TraceServices { class IAnalysisSession; }

class SBlendWeightsView : public SPropertiesDebugViewBase
{
public:
	virtual void GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const override;
	virtual FName GetName() const override;

	void SetAssetFilter(uint64 InAssetIdFilter, uint32 InNodeIdFilter)
	{
		AssetIdFilter = InAssetIdFilter;
		NodeIdFilter = InNodeIdFilter;
		bAssetFilterSet = true;
	}

	void ClearAssetFilter()
	{
		bAssetFilterSet = false;
	}

private:
	uint64 AssetIdFilter = 0;
	uint32 NodeIdFilter = 0;
	bool bAssetFilterSet = false;
};