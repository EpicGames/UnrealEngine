// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCurveTimelineView.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/DrawElements.h"
#include "SSimpleTimeSlider.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "SCurveTimelineView"


void SCurveTimelineView::Construct( const SCurveTimelineView::FArguments& InArgs )
{
    ViewRange = InArgs._ViewRange;
	CurveColor = InArgs._CurveColor;
	FillColor = InArgs._FillColor;
	RenderFill = InArgs._RenderFill;
	DesiredSize = InArgs._DesiredSize;
	CurveData = InArgs._CurveData;
	TrackName = InArgs._TrackName;

	// set clipping on by default, since the OnPaint function is drawing outside the bounds
	Clipping = EWidgetClipping::ClipToBounds;
	
	FillBrush = FAppStyle::GetBrush("Sequencer.SectionArea.Background");
}

int32 SCurveTimelineView::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	int32 NewLayer = PaintCurve( AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled );

	return FMath::Max( NewLayer, SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, NewLayer, InWidgetStyle, ShouldBeEnabled( bParentEnabled ) ) );
}

FReply SCurveTimelineView::OnMouseMove(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	UpdateCurveToolTip(InMyGeometry, InMouseEvent);
	return FReply::Unhandled();
}

void SCurveTimelineView::UpdateCurveToolTip(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMyGeometry.IsUnderLocation(InMouseEvent.GetScreenSpacePosition()))
	{
		if (const TSharedPtr<FTimelineCurveData> Curve = CurveData.Get())
		{
			// Mouse position in widget space
			const FVector2D HitPosition = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());

			// Range helper struct
			const TRange<double> DebugTimeRange = ViewRange.Get();
			const SSimpleTimeSlider::FScrubRangeToScreen RangeToScreen( DebugTimeRange, InMyGeometry.GetLocalSize() );
		
			// Mouse position from widget space to curve input space
			const double TargetTime = RangeToScreen.LocalXToInput(HitPosition.X);

			// Get curve value at given time
			TArray<FTimelineCurveData::CurvePoint> & CurvePoints = Curve->Points;
			const int NumPoints = CurvePoints.Num();
		
			if (NumPoints > 0)
			{
				for (int i = 1; i < NumPoints; ++i)
				{
					const FTimelineCurveData::CurvePoint& Point1 = CurvePoints[i-1];
					const FTimelineCurveData::CurvePoint& Point2 = CurvePoints[i];

					// Find points that contain mouse hit-point time
					if ( Point1.Time >= TargetTime && TargetTime <= Point2.Time)
					{
						// Choose point with the smallest delta
						const float Delta1 = abs(TargetTime - Point1.Time);
						const float Delta2 = abs(TargetTime - Point2.Time);

						// Get closest point
						const FTimelineCurveData::CurvePoint & TargetPoint = Delta1 < Delta2 ? Point1 : Point2;

						// Tooltip text formatting
						FNumberFormattingOptions FormattingOptions;
						FormattingOptions.MaximumFractionalDigits = 3;
						CurveToolTipOutputText = FText::Format(LOCTEXT("CurveToolTipValueFormat", "Value: {0}"), FText::AsNumber(TargetPoint.Value, &FormattingOptions));
						CurveToolTipInputText = FText::Format(LOCTEXT("CurveToolTipTimeFormat", "Time: {0}"), FText::AsNumber(TargetPoint.Time, &FormattingOptions));
				
						// Update tooltip info
						if (CurveToolTip.IsValid() == false)
						{
							SetToolTip(
								SAssignNew(CurveToolTip, SToolTip)
								.BorderImage( FCoreStyle::Get().GetBrush( "ToolTip.BrightBackground" ) )
								[
									SNew(SVerticalBox)
									+ SVerticalBox::Slot()
									[
										SNew(STextBlock)
										.Text(this, &SCurveTimelineView::GetTrackName)
										.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
										.ColorAndOpacity(FLinearColor::Black)
									]
									+ SVerticalBox::Slot()
									[
										SNew(STextBlock)
										.Text(this, &SCurveTimelineView::GetCurveToolTipInputText)
										.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
										.ColorAndOpacity(FLinearColor::Black)
									]
									+ SVerticalBox::Slot()
									[
										SNew(STextBlock)
										.Text(this, &SCurveTimelineView::GetCurveToolTipOutputText)
										.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
										.ColorAndOpacity(FLinearColor::Black)
									]
								]);
						}
				
						break;
					}
				}
			}
		}
	}
}

FVector2D SCurveTimelineView::ComputeDesiredSize( float ) const
{
	return DesiredSize.Get();
}

int32 SCurveTimelineView::PaintCurve(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	static const float LargeFrameTime = 0.125;  // ideally, we could check the recorded frame data for actual frame length
	 
	// convert time range to from rewind debugger times to profiler times
	TRange<double> DebugTimeRange = ViewRange.Get();
	FLinearColor LineColor = CurveColor.Get();
	
	SSimpleTimeSlider::FScrubRangeToScreen RangeToScreen( DebugTimeRange, AllottedGeometry.GetLocalSize() );
	FVector2D Size = AllottedGeometry.GetLocalSize();

	if (TSharedPtr<FTimelineCurveData> Curve = CurveData.Get())
	{
		// Min/Max for normalization
		// maybe expose these as attributes too, so they are recomputed less frequently
		float MinValue = 1.0e10;
		float MaxValue = -1.0e10;

		if (bUseFixedRange)
		{
			MinValue = FixedRangeMin;
			MaxValue = FixedRangeMax;
		}
		else
		{
			for(auto& Point : Curve->Points)
			{
				MinValue = FMath::Min(MinValue, Point.Value);
				MaxValue = FMath::Max(MaxValue, Point.Value);
			}
		}
		
		const float Range = FMath::Max(UE_KINDA_SMALL_NUMBER, MaxValue - MinValue);
		
		auto &CurvePoints = Curve->Points;
		const int NumPoints = CurvePoints.Num();
		
		const ESlateDrawEffect LineDrawEffects = ESlateDrawEffect::NoPixelSnapping;	

		if (NumPoints > 0)
		{
			if (RenderFill.Get())
			{
			
				const FSlateRenderTransform& Transform = AllottedGeometry.GetAccumulatedRenderTransform();
				FColor Color = FillColor.Get().ToFColor(true);

				float Y1 = Size.Y-1;
				for(int i=1; i<NumPoints; i++)
				{
					const FTimelineCurveData::CurvePoint& Point2 = CurvePoints[i];

					// find time of previous sample - step backwards until a sample that has a different time
					double CurrentTime = CurvePoints[i].Time;
					double PreviousTime = CurrentTime;
					for (int j = i - 1; j >= 0; j--)
					{
						if (CurvePoints[j].Time != CurrentTime)
						{
							PreviousTime = CurvePoints[j].Time;
							break;
						}
					}
					
					const float NormalizedValue = (Point2.Value - MinValue) / Range;
					float Y2 = Size.Y - (Size.Y - 2) * NormalizedValue;
				
					if (CurrentTime - PreviousTime < LargeFrameTime)
					{
						float X1 = RangeToScreen.InputToLocalX(PreviousTime);
						float X2 = RangeToScreen.InputToLocalX(CurrentTime);
						FPaintGeometry PaintGeo = AllottedGeometry.ToPaintGeometry(FVector2f(X2-X1, Y1-Y2), FSlateLayoutTransform(FVector2f(X1, Y2)));
						FSlateDrawElement::MakeBox(OutDrawElements, LayerId, PaintGeo, FillBrush, ESlateDrawEffect::None, FillColor.Get());
					}
				}
			}
			else
			{
				// Lines rendering
				TArray<FVector2D> Points;
				Points.Reserve(NumPoints);
		 
				auto MakeLines = [&OutDrawElements, &LayerId, &AllottedGeometry, &Points, &LineDrawEffects, &LineColor]()
					{
						if (Points.IsEmpty())
						{
							return;
						}

						if (Points.Num() == 1)
						{
							// adding an additional point AdditionalPixels apart to be able to draw a line
							static const float AdditionalPixels = 2.f;
							const FVector2D AdditionalPoint(Points[0].X + AdditionalPixels, Points[0].Y);
							Points.Add(AdditionalPoint);
						}

						FSlateDrawElement::MakeLines(
							OutDrawElements,
							LayerId++,
							AllottedGeometry.ToPaintGeometry(),
							Points,
							LineDrawEffects,
							LineColor,
							false
						);
					};

				float PrevTime = CurvePoints[0].Time;
				for(int i=0; i<NumPoints; i++)
				{
					const FTimelineCurveData::CurvePoint& Point = CurvePoints[i];
					if (Point.Time - PrevTime > LargeFrameTime)
					{
						// break the line list - data has stopped and started again
						MakeLines();
						
						Points.SetNum(0,EAllowShrinking::No);
					}
			
					float X = RangeToScreen.InputToLocalX(Point.Time);
					PrevTime = Point.Time;
					const float NormalizedValue = (Point.Value - MinValue) / Range;
					float Y = Size.Y - (Size.Y - 2) * NormalizedValue;
					Points.Add(FVector2D(X,Y));
				}
			
				MakeLines();
				
			}
		
		}
	}

	return LayerId;
}

#undef LOCTEXT_NAMESPACE
