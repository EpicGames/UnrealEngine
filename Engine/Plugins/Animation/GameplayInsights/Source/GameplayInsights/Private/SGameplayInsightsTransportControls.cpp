// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayInsightsTransportControls.h"

#if WITH_EDITOR

#include "EditorWidgetsModule.h"
#include "Modules/ModuleManager.h"
#include "Insights/ITimingViewSession.h"
#include "TraceServices/Model/Frames.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "GameplaySharedData.h"
#include "GameplayInsightsStyle.h"
#include "Widgets/Input/SSpinBox.h"
#include "Math/UnitConversion.h"
#include "Widgets/Input/NumericUnitTypeInterface.inl"

#include <limits>

#define LOCTEXT_NAMESPACE "SGameplayInsightsTransportControls"

void SGameplayInsightsTransportControls::Construct(const FArguments& InArgs, FGameplaySharedData& InSharedData)
{
	SharedData = &InSharedData;

	PlayRate = 1.0;
	bPlaying = false;
	bReverse = false;
	bSettingMarker = false;
	SelectionStartTime = 0.0;
	SelectionEndTime = 0.0;
	bSelectionRangeValid = false;
	bLooping = true;

	SharedData->GetTimingViewSession().OnTimeMarkerChanged().AddSP(this, &SGameplayInsightsTransportControls::HandleTimeMarkerChanged);
	SharedData->GetTimingViewSession().OnSelectionChanged().AddSP(this, &SGameplayInsightsTransportControls::HandleSelectionRangeChanged);

	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");

	FTransportControlArgs TransportControlArgs;
	TransportControlArgs.OnForwardPlay = FOnClicked::CreateSP(this, &SGameplayInsightsTransportControls::OnClick_Forward);
	TransportControlArgs.OnBackwardPlay = FOnClicked::CreateSP(this, &SGameplayInsightsTransportControls::OnClick_Backward);
	TransportControlArgs.OnForwardStep = FOnClicked::CreateSP(this, &SGameplayInsightsTransportControls::OnClick_Forward_Step);
	TransportControlArgs.OnBackwardStep = FOnClicked::CreateSP(this, &SGameplayInsightsTransportControls::OnClick_Backward_Step);
	TransportControlArgs.OnForwardEnd = FOnClicked::CreateSP(this, &SGameplayInsightsTransportControls::OnClick_Forward_End);
	TransportControlArgs.OnBackwardEnd = FOnClicked::CreateSP(this, &SGameplayInsightsTransportControls::OnClick_Backward_End);
	TransportControlArgs.OnGetPlaybackMode = FOnGetPlaybackMode::CreateSP(this, &SGameplayInsightsTransportControls::GetPlaybackMode);
	TransportControlArgs.OnToggleLooping = FOnClicked::CreateSP(this, &SGameplayInsightsTransportControls::OnClick_ToggleLooping);
	TransportControlArgs.OnGetLooping = FOnGetLooping::CreateSP(this, &SGameplayInsightsTransportControls::GetLooping);

	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			EditorWidgetsModule.CreateTransportControl(TransportControlArgs)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.0f, 0.0f)
		[
			SNew(SSpinBox<double>)
			.Value_Lambda([this](){ return PlayRate; })
			.ToolTipText(LOCTEXT("PlayRate", "Playback speed"))
			.OnValueCommitted_Lambda([this](double InValue, ETextCommit::Type InCommitType){ PlayRate = InValue; })
			.MinValue(0.001f)
			.MaxValue(100.0f)
			.Style(&FGameplayInsightsStyle::Get().GetWidgetStyle<FSpinBoxStyle>("TransportControls.HyperlinkSpinBox"))
			.ClearKeyboardFocusOnCommit(true)
			.Delta(0.01f)
			.LinearDeltaSensitivity(25)
			.TypeInterface(MakeShared<TNumericUnitTypeInterface<double>>(EUnit::Multiplier))
		]
	];

	RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime)
	{
		if(bPlaying)
		{
			double CurrentTime = SharedData->GetTimingViewSession().GetTimeMarker();
			double Delta = (double)(bReverse ? -InDeltaTime : InDeltaTime);
			if(bSelectionRangeValid)
			{
				// Loop within the range if the selection is valid
				double NewTime = CurrentTime + (Delta * PlayRate);
				if(bReverse)
				{
					if(NewTime < SelectionStartTime)
					{
						if(bLooping)
						{
							NewTime = SelectionEndTime;
						}
						else
						{
							NewTime = SelectionStartTime;
							bPlaying = false;
						}
					}
					else if(NewTime > SelectionEndTime)
					{
						NewTime = SelectionEndTime;
					}
				}
				else
				{
					if(NewTime < SelectionStartTime)
					{
						NewTime = SelectionStartTime;
					}
					else if(NewTime > SelectionEndTime)
					{
						if(bLooping)
						{
							NewTime = SelectionStartTime;
						}
						else
						{
							NewTime = SelectionEndTime;
							bPlaying = false;
						}
					}
				}

				SetTimeMarker(NewTime, false);
			}
			else
			{
				SetTimeMarker(CurrentTime + (Delta * PlayRate), false);
			}
		}

		return EActiveTimerReturnType::Continue;
	}));
}

FReply SGameplayInsightsTransportControls::OnClick_Forward_Step()
{
	TraceServices::FAnalysisSessionReadScope SessionReadScope(SharedData->GetAnalysisSession());
	const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(SharedData->GetAnalysisSession());

	double CurrentTime = SharedData->GetTimingViewSession().GetTimeMarker();
	if(CurrentTime == std::numeric_limits<double>::infinity())
	{
		const TraceServices::FFrame* FirstFrame = FramesProvider.GetFrame(ETraceFrameType::TraceFrameType_Game, 0);
		if(FirstFrame)
		{
			CurrentTime = FirstFrame->StartTime + (double)SMALL_NUMBER;
		}
	}
	
	TraceServices::FFrame Frame;
	if(FramesProvider.GetFrameFromTime(ETraceFrameType::TraceFrameType_Game, CurrentTime, Frame))
	{
		if(Frame.Index < FramesProvider.GetFrameCount(ETraceFrameType::TraceFrameType_Game) - 1)
		{
			const TraceServices::FFrame* NextFrame = FramesProvider.GetFrame(ETraceFrameType::TraceFrameType_Game, Frame.Index + 1);
			if(NextFrame)
			{
				SetTimeMarker(NextFrame->StartTime + (double)SMALL_NUMBER, false);
			}
		}
	}

	bPlaying = false;
	bReverse = false;

	return FReply::Handled();
}

FReply SGameplayInsightsTransportControls::OnClick_Forward_End()
{
	TraceServices::FAnalysisSessionReadScope SessionReadScope(SharedData->GetAnalysisSession());
	const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(SharedData->GetAnalysisSession());
	
	if(FramesProvider.GetFrameCount(ETraceFrameType::TraceFrameType_Game) > 0)
	{
		const TraceServices::FFrame* LastFrame = FramesProvider.GetFrame(ETraceFrameType::TraceFrameType_Game, FramesProvider.GetFrameCount(ETraceFrameType::TraceFrameType_Game) - 1);
		if(LastFrame)
		{
			SetTimeMarker(LastFrame->StartTime + (double)SMALL_NUMBER, true);
		}
	}

	bPlaying = false;
	bReverse = false;

	return FReply::Handled();
}

FReply SGameplayInsightsTransportControls::OnClick_Backward_Step()
{
	TraceServices::FAnalysisSessionReadScope SessionReadScope(SharedData->GetAnalysisSession());
	const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(SharedData->GetAnalysisSession());

	double CurrentTime = SharedData->GetTimingViewSession().GetTimeMarker();
	if(CurrentTime == std::numeric_limits<double>::infinity())
	{
		const TraceServices::FFrame* LastFrame = FramesProvider.GetFrame(ETraceFrameType::TraceFrameType_Game, FramesProvider.GetFrameCount(TraceFrameType_Game) - 1);
		if(LastFrame)
		{
			CurrentTime = LastFrame->StartTime + (double)SMALL_NUMBER;
		}
	}

	TraceServices::FFrame Frame;
	if(FramesProvider.GetFrameFromTime(ETraceFrameType::TraceFrameType_Game, CurrentTime, Frame))
	{
		if(Frame.Index > 0)
		{
			const TraceServices::FFrame* PrevFrame = FramesProvider.GetFrame(ETraceFrameType::TraceFrameType_Game, Frame.Index - 1);
			if(PrevFrame)
			{
				SetTimeMarker(PrevFrame->StartTime + (double)SMALL_NUMBER, false);
			}
		}
	}

	bPlaying = false;
	bReverse = false;

	return FReply::Handled();
}

FReply SGameplayInsightsTransportControls::OnClick_Backward_End()
{
	TraceServices::FAnalysisSessionReadScope SessionReadScope(SharedData->GetAnalysisSession());
	const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(SharedData->GetAnalysisSession());
	
	const TraceServices::FFrame* FirstFrame = FramesProvider.GetFrame(ETraceFrameType::TraceFrameType_Game, 0);
	if(FirstFrame)
	{
		SetTimeMarker(FirstFrame->StartTime + (double)SMALL_NUMBER, true);
	}

	bPlaying = false;
	bReverse = false;

	return FReply::Handled();
}

FReply SGameplayInsightsTransportControls::OnClick_Forward()
{
	double CurrentTime = SharedData->GetTimingViewSession().GetTimeMarker();
	if(CurrentTime == std::numeric_limits<double>::infinity())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(SharedData->GetAnalysisSession());
		const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(SharedData->GetAnalysisSession());
	
		const TraceServices::FFrame* FirstFrame = FramesProvider.GetFrame(ETraceFrameType::TraceFrameType_Game, 0);
		if(FirstFrame)
		{
			SetTimeMarker(FirstFrame->StartTime + (double)SMALL_NUMBER, false);
		}
	}

	bPlaying = bReverse ? true : !bPlaying;
	bReverse = false;
	return FReply::Handled();
}

FReply SGameplayInsightsTransportControls::OnClick_Backward()
{
	double CurrentTime = SharedData->GetTimingViewSession().GetTimeMarker();
	if(CurrentTime == std::numeric_limits<double>::infinity())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(SharedData->GetAnalysisSession());
		const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(SharedData->GetAnalysisSession());
	
		const TraceServices::FFrame* LastFrame = FramesProvider.GetFrame(ETraceFrameType::TraceFrameType_Game, FramesProvider.GetFrameCount(TraceFrameType_Game) - 1);
		if(LastFrame)
		{
			SetTimeMarker(LastFrame->StartTime + (double)SMALL_NUMBER, false);
		}
	}

	bPlaying = !bReverse ? true : !bPlaying;
	bReverse = true;
	return FReply::Handled();
}

FReply SGameplayInsightsTransportControls::OnClick_ToggleLooping()
{
	bLooping = !bLooping;
	return FReply::Handled();
}

bool SGameplayInsightsTransportControls::GetLooping() const
{
	return bLooping;
}

EPlaybackMode::Type SGameplayInsightsTransportControls::GetPlaybackMode() const
{
	if(bPlaying)
	{
		return bReverse ? EPlaybackMode::PlayingReverse : EPlaybackMode::PlayingForward;
	}

	return EPlaybackMode::Stopped;
}

void SGameplayInsightsTransportControls::SetTimeMarker(double InTime, bool bInScroll)
{
	bSettingMarker = true;
	if(bInScroll)
	{
		SharedData->GetTimingViewSession().SetAndCenterOnTimeMarker(InTime);
	}
	else
	{
		SharedData->GetTimingViewSession().SetTimeMarker(InTime);
	}
	bSettingMarker = false;
}

void SGameplayInsightsTransportControls::HandleTimeMarkerChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InTimeMarker)
{
	if(!bSettingMarker)
	{
		// turn off playback if someone else scrubbed the timeline
		bPlaying = false;
	}
}

void SGameplayInsightsTransportControls::HandleSelectionRangeChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InStartTime, double InEndTime)
{
	if(InFlags != UE::Insights::Timing::ETimeChangedFlags::Interactive)
	{
		bSelectionRangeValid = InStartTime != std::numeric_limits<double>::infinity() && InEndTime != std::numeric_limits<double>::infinity() && InEndTime > InStartTime;

		SelectionStartTime = InStartTime;
		SelectionEndTime = InEndTime;
	}
}

#undef LOCTEXT_NAMESPACE

#endif
