// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"

#if WITH_EDITOR

#include "ITransportControl.h"

class FGameplaySharedData;
namespace UE::Insights::Timing { enum class ETimeChangedFlags; }

class SGameplayInsightsTransportControls : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGameplayInsightsTransportControls) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FGameplaySharedData& InSharedData);

private:
	FReply OnClick_Forward_Step();

	FReply OnClick_Forward_End();

	FReply OnClick_Backward_Step();

	FReply OnClick_Backward_End();

	FReply OnClick_Forward();

	FReply OnClick_Backward();

	FReply OnClick_ToggleLooping();

	bool GetLooping() const;

	EPlaybackMode::Type GetPlaybackMode() const;

	void SetTimeMarker(double InTime, bool bInScroll);

	void HandleTimeMarkerChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InTimeMarker);

	void HandleSelectionRangeChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InStartTime, double InEndTime);

private:
	FGameplaySharedData* SharedData;

	double PlayRate;

	bool bPlaying;

	bool bReverse;

	bool bLooping;

	bool bSettingMarker;

	double SelectionStartTime;
	double SelectionEndTime;
	bool bSelectionRangeValid;
};

#endif