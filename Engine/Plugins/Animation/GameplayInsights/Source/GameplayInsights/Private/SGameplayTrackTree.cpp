// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameplayTrackTree.h"

#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SHeaderRow.h"
#include "ObjectEventsTrack.h"
#include "GameplaySharedData.h"

#define LOCTEXT_NAMESPACE "SGameplayTrackTree"

enum class EGameplayTrackFilterState
{
	Hidden,
	Visible,
	Highlighted,
};

// Simple wrapper around a gameplay track for filtering
struct FGameplayTrackTreeEntry : public TSharedFromThis<FGameplayTrackTreeEntry>
{
	FGameplayTrackTreeEntry(TSharedRef<FBaseTimingTrack> InTimingTrack)
		: WeakTimingTrack(InTimingTrack)
		, FilterState(EGameplayTrackFilterState::Hidden)
		, bIsAnyChildVisible(false)
		, bAreAllChildrenVisible(false)
	{}

	FText GetName() const
	{
		FText Name;

		TSharedPtr<FBaseTimingTrack> TimingTrack = WeakTimingTrack.Pin();
		if(TimingTrack.IsValid())
		{
			Name = FText::FromString(TimingTrack->GetName());
		}

		return Name;
	}

	ECheckBoxState GetCheckBoxState() const
	{
		if(IsVisible())
		{
			return ECheckBoxState::Checked;
		}
		else if(bIsAnyChildVisible)
		{
			return ECheckBoxState::Undetermined;
		}
	
		return ECheckBoxState::Unchecked;
	}

	bool IsVisible() const
	{
		TSharedPtr<FBaseTimingTrack> TimingTrack = WeakTimingTrack.Pin();
		if(TimingTrack.IsValid())
		{
			return TimingTrack->IsVisible();
		}

		return false;
	}

	void SetVisibilityFlag(bool bInIsVisible)
	{
		TSharedPtr<FBaseTimingTrack> TimingTrack = WeakTimingTrack.Pin();
		if(TimingTrack.IsValid())
		{
			TimingTrack->SetVisibilityFlag(bInIsVisible);
		}

		for(const TSharedRef<FGameplayTrackTreeEntry>& Child : Children)
		{
			Child->SetVisibilityFlag(bInIsVisible);
		}
	}

	void CacheVisibility()
	{
		CacheVisibility_Helper(bIsAnyChildVisible, bAreAllChildrenVisible);
	}

private:

	void CacheVisibility_Helper(bool &bOutAnyChildVisible, bool& bOutAllChildrenVisible)
	{
		bIsAnyChildVisible = false;
		bAreAllChildrenVisible = Children.Num() > 0;

		for(const TSharedRef<FGameplayTrackTreeEntry>& Child : Children)
		{
			bIsAnyChildVisible |= Child->IsVisible();
			bAreAllChildrenVisible &= Child->IsVisible();

			Child->CacheVisibility_Helper(Child->bIsAnyChildVisible, Child->bAreAllChildrenVisible);

			bOutAnyChildVisible |= Child->bIsAnyChildVisible;
			bOutAllChildrenVisible &=  Child->bAreAllChildrenVisible;
		}
	}

public:

	TWeakPtr<FBaseTimingTrack> WeakTimingTrack;
	TArray<TSharedRef<FGameplayTrackTreeEntry>> Children;
	EGameplayTrackFilterState FilterState;
	bool bIsAnyChildVisible;
	bool bAreAllChildrenVisible;
};

// A list entry widget for a gameplay track
class SGameplayTrackTreeEntry : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGameplayTrackTreeEntry) {}

	SLATE_ARGUMENT(TSharedPtr<FGameplayTrackTreeEntry>, TreeEntry)

	SLATE_ATTRIBUTE(FText, SearchText)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SGameplayTrackTree>& InTree)
	{
		WeakTreeEntry = InArgs._TreeEntry;
		SearchText = InArgs._SearchText;
		Tree = InTree;

		ChildSlot
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([this]()
			{ 
				TSharedPtr<FGameplayTrackTreeEntry> TreeEntry = WeakTreeEntry.Pin();
				if(TreeEntry.IsValid())
				{
					return TreeEntry->GetCheckBoxState(); 
				}
				return ECheckBoxState::Unchecked; 
			})
			.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
			{
				TSharedPtr<FGameplayTrackTreeEntry> TreeEntry = WeakTreeEntry.Pin();
				if(TreeEntry.IsValid())
				{
					TreeEntry->SetVisibilityFlag(InCheckBoxState == ECheckBoxState::Checked);

					Tree.Pin()->CacheVisibility();
				}
			})
			.Content()
			[
				SNew(STextBlock)
				.Text(InArgs._TreeEntry->GetName())
				.HighlightText(SearchText)
			]
		];
	}

	// The tree entry we represent
	TWeakPtr<FGameplayTrackTreeEntry> WeakTreeEntry;

	// The tree we are contained in
	TWeakPtr<SGameplayTrackTree> Tree;

	// The search text to highlight
	TAttribute<FText> SearchText;
};

void SGameplayTrackTree::Construct(const FArguments& InArgs, FGameplaySharedData& InSharedData)
{
	SharedData = &InSharedData;

	InSharedData.OnTracksChanged().AddSP(this, &SGameplayTrackTree::HandleTracksChanged);

	TreeView = SNew(STreeView<TSharedRef<FGameplayTrackTreeEntry>>)
		.SelectionMode(ESelectionMode::None)
		.TreeItemsSource(&FilteredTracks)
		.OnGenerateRow(this, &SGameplayTrackTree::OnGenerateRow)
		.OnGetChildren(this, &SGameplayTrackTree::OnGetChildren);

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f)
			[
				// Checkbox for bulk operations
				SNew(SCheckBox)
				.IsChecked_Lambda([this]()
				{
					int32 NumVisible = 0;
					for (TSharedRef<FGameplayTrackTreeEntry> Track : FilteredTracks)
					{
						if (Track->IsVisible())
						{
							NumVisible++;
						}
					}

					if (NumVisible == 0)
					{
						return ECheckBoxState::Unchecked;
					}
					else if (NumVisible == FilteredTracks.Num())
					{
						return ECheckBoxState::Checked;
					}

					return ECheckBoxState::Undetermined;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
				{
					const bool bVisible = InCheckBoxState != ECheckBoxState::Unchecked;
					for (TSharedRef<FGameplayTrackTreeEntry> Track : FilteredTracks)
					{
						Track->SetVisibilityFlag(bVisible);
					}

					CacheVisibility();
				})
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				// Search box allows for filtering
				SAssignNew(SearchBox, SSearchBox)
				.OnTextChanged_Lambda([this](const FText& InText){ SearchText = InText; RefreshFilter(); })
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SScrollBorder, TreeView.ToSharedRef())
			[
				TreeView.ToSharedRef()
			]
		]
	];

	// Set focus to the search box on creation
	FSlateApplication::Get().SetKeyboardFocus(SearchBox);
	FSlateApplication::Get().SetUserFocus(0, SearchBox);

	RefreshFilter();
}

void SGameplayTrackTree::HandleTracksChanged()
{
	RefreshFilter();
}

TSharedRef<ITableRow> SGameplayTrackTree::OnGenerateRow(TSharedRef<FGameplayTrackTreeEntry> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return
		SNew(STableRow<TSharedRef<FGameplayTrackTreeEntry>>, InOwnerTable)
		.ToolTipText(InItem->GetName())
		[
			SNew(SGameplayTrackTreeEntry, SharedThis(this))
			.TreeEntry(InItem)
			.SearchText_Lambda([this](){ return SearchText; })
		];
}

void SGameplayTrackTree::OnGetChildren(TSharedRef<FGameplayTrackTreeEntry> InItem, TArray<TSharedRef<FGameplayTrackTreeEntry>>& OutChildren)
{
	for(const TSharedRef<FGameplayTrackTreeEntry>& ChildItem : InItem->Children)
	{
		if(ChildItem->FilterState != EGameplayTrackFilterState::Hidden)
		{
			OutChildren.Add(ChildItem);
		}
	}
}

EGameplayTrackFilterState SGameplayTrackTree::RefreshFilter_Helper(const TSharedRef<FGameplayTrackTreeEntry>& InTreeEntry)
{
	TSharedPtr<FObjectEventsTrack> ObjectEventsTrack;
	TSharedPtr<FBaseTimingTrack> TimingTrack = InTreeEntry->WeakTimingTrack.Pin();
	if(TimingTrack.IsValid())
	{
		if(TimingTrack->Is<FObjectEventsTrack>())
		{
			ObjectEventsTrack = StaticCastSharedPtr<FObjectEventsTrack>(TimingTrack);
		}
	}

	if(ObjectEventsTrack.IsValid() && ObjectEventsTrack->GetGameplayTrack().GetChildTracks().Num() > 0)
	{
		InTreeEntry->FilterState = EGameplayTrackFilterState::Hidden;

		for(FGameplayTrack* ChildTrack : ObjectEventsTrack->GetGameplayTrack().GetChildTracks())
		{
			TSharedRef<FGameplayTrackTreeEntry> NewTreeEntry = MakeShared<FGameplayTrackTreeEntry>(ChildTrack->GetTimingTrack());
			InTreeEntry->Children.Add(NewTreeEntry);

			TreeView->SetItemExpansion(NewTreeEntry, true);

			EGameplayTrackFilterState ChildFilterState = RefreshFilter_Helper(NewTreeEntry);
			InTreeEntry->FilterState = FMath::Max(ChildFilterState, InTreeEntry->FilterState);
		}
	}
	
	if(InTreeEntry->FilterState == EGameplayTrackFilterState::Hidden)
	{
		if(SearchText.IsEmpty())
		{
			InTreeEntry->FilterState = EGameplayTrackFilterState::Visible;
		}
		else if (InTreeEntry->GetName().ToString().Contains(SearchText.ToString()))
		{
			InTreeEntry->FilterState = EGameplayTrackFilterState::Highlighted;
		}
		else
		{
			InTreeEntry->FilterState = EGameplayTrackFilterState::Hidden;
		}
	}

	if(InTreeEntry->FilterState == EGameplayTrackFilterState::Highlighted)
	{
		TreeView->SetItemExpansion(InTreeEntry, true);
	}

	return InTreeEntry->FilterState;
}

void SGameplayTrackTree::RefreshFilter()
{
	FilteredTracks.Reset();
	RootEntries.Reset();

	for(const TSharedRef<FBaseTimingTrack>& RootTrack : SharedData->GetRootTracks())
	{
		TSharedRef<FGameplayTrackTreeEntry> NewTreeEntry = MakeShared<FGameplayTrackTreeEntry>(RootTrack);
		RootEntries.Add(NewTreeEntry);
		EGameplayTrackFilterState FilterState = RefreshFilter_Helper(NewTreeEntry);
		if(FilterState != EGameplayTrackFilterState::Hidden)
		{
			FilteredTracks.Add(NewTreeEntry);
			TreeView->SetItemExpansion(NewTreeEntry, true);
		}
	}

	CacheVisibility();

	TreeView->RequestTreeRefresh();
}

void SGameplayTrackTree::CacheVisibility()
{
	for(const TSharedRef<FGameplayTrackTreeEntry>& RootEntry : RootEntries)
	{
		RootEntry->CacheVisibility();
	}
}

#undef LOCTEXT_NAMESPACE
