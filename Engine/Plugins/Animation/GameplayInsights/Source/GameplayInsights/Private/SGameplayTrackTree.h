// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class SSearchBox;
class FBaseTimingTrack;
class FGameplaySharedData;
struct FGameplayTrackTreeEntry;
enum class EGameplayTrackFilterState;

// A widget representing all the gameplay tracks
class SGameplayTrackTree : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGameplayTrackTree) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FGameplaySharedData& InSharedData);

	void CacheVisibility();

private:
	// Generate a row for the tree
	TSharedRef<ITableRow> OnGenerateRow(TSharedRef<FGameplayTrackTreeEntry> Item, const TSharedRef<STableViewBase>& OwnerTable);

	// Get the children of an item
	void OnGetChildren(TSharedRef<FGameplayTrackTreeEntry> InItem, TArray<TSharedRef<FGameplayTrackTreeEntry>>& OutChildren);

	// Refresh the tree as the filter has changed
	void RefreshFilter();

	// Recursive helper to filter tree
	EGameplayTrackFilterState RefreshFilter_Helper(const TSharedRef<FGameplayTrackTreeEntry>& InTreeEntry);

	// Handle the tracks changing under us
	void HandleTracksChanged();

private:
	// Shared data source
	FGameplaySharedData* SharedData;

	// The search widget
	TSharedPtr<SSearchBox> SearchBox;

	// The list view widget
	TSharedPtr<STreeView<TSharedRef<FGameplayTrackTreeEntry>>> TreeView;

	// Text we are searching for
	FText SearchText;

	// Filtered list of tracks
	TArray<TSharedRef<FGameplayTrackTreeEntry>> FilteredTracks;

	// Unfiltered list of tracks
	TArray<TSharedRef<FGameplayTrackTreeEntry>> RootEntries;
};
