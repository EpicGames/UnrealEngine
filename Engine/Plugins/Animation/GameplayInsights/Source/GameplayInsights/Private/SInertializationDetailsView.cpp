// Copyright Epic Games, Inc. All Rights Reserved.

#include "SInertializationDetailsView.h"
#include "AnimationProvider.h"
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "GameplayProvider.h"
#include "TraceServices/Model/Frames.h"
#include "VariantTreeNode.h"

#define LOCTEXT_NAMESPACE "SInertializationDetailsView"

struct FInertializationDetailsNodeItem
{
	TArray<const TCHAR*, TInlineAllocator<16>> PropertyNames;
	TArray<FVariantValue, TInlineAllocator<16>> PropertyValues;
};

void SInertializationDetailsView::GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const
{
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	TSortedMap<TTuple<int32, uint64>, FInertializationDetailsNodeItem, TInlineAllocator<8>> NodeMap;

	if (AnimationProvider)
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);

		// Find all inertialization nodes within window

		AnimationProvider->ReadAnimNodesTimeline(ObjectId, [this, &NodeMap, &AnimationProvider, &InFrame](const FAnimationProvider::AnimNodesTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, [this, &NodeMap, &AnimationProvider, &InFrame](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNodeMessage& InMessage)
			{
				if (InStartTime >= InFrame.StartTime && InEndTime <= InFrame.EndTime)
				{
					if (bFilterSet && InMessage.NodeId != NodeIdFilter)
					{
						return TraceServices::EEventEnumerate::Continue;
					}

					for (const TCHAR* InertializationNodeType : { TEXT("AnimNode_DeadBlending"), TEXT("AnimNode_Inertialization") })
					{
						if (FCString::Strcmp(InertializationNodeType, InMessage.NodeTypeName) == 0)
						{
							NodeMap.FindOrAdd({ InMessage.NodeId, InMessage.AnimInstanceId });
							break;
						}
					}
				}

				return TraceServices::EEventEnumerate::Continue;
			});
		});

		// Fill in Node Trace Details
		AnimationProvider->ReadAnimNodeValuesTimeline(ObjectId, [this, &NodeMap, &AnimationProvider, &InFrame](const FAnimationProvider::AnimNodeValuesTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, [this, &NodeMap, &AnimationProvider, &InFrame](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNodeValueMessage& InMessage)
			{
				if (InStartTime >= InFrame.StartTime && InEndTime <= InFrame.EndTime)
				{
					if (FInertializationDetailsNodeItem* NodeItem = NodeMap.Find({ InMessage.NodeId, InMessage.AnimInstanceId }))
					{
						NodeItem->PropertyNames.Add(InMessage.Key);
						NodeItem->PropertyValues.Add(InMessage.Value);
					}
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}

	// Make UI Variant Items

	for (TPair<TTuple<int32, uint64>, FInertializationDetailsNodeItem>& NodePair : NodeMap)
	{
		bool bInertializationActive = false;

		const int32 PropertyNum = NodePair.Value.PropertyNames.Num();

		// Check if inertialization node is Active

		for (int32 PropertyIdx = 0; PropertyIdx < PropertyNum; PropertyIdx++)
		{
			if (FCString::Strcmp(NodePair.Value.PropertyNames[PropertyIdx], TEXT("State")) == 0 &&
				FCString::Strcmp(NodePair.Value.PropertyValues[PropertyIdx].String.Value, TEXT("EInertializationState::Active")) == 0)
			{
				bInertializationActive = true;
			}
		}

		// If node is active then add to details view

		if (bInertializationActive)
		{
			TSharedRef<FVariantTreeNode> Header = OutVariants.Add_GetRef(
				FVariantTreeNode::MakeAnimNode(LOCTEXT("InertializationNode", "Inertialization Node"), NodePair.Key.Get<0>(), NodePair.Key.Get<1>()));

			for (int32 PropertyIdx = 0; PropertyIdx < PropertyNum; PropertyIdx++)
			{
				Header->AddChild(MakeShared<FVariantTreeNode>(
					FText::FromString(NodePair.Value.PropertyNames[PropertyIdx]), 
					NodePair.Value.PropertyValues[PropertyIdx], 0, INDEX_NONE));
			}
		}
	}
}

static const FName InertializationDetailsName("Inertialization");

FName SInertializationDetailsView::GetName() const
{
	return InertializationDetailsName;
}

#undef LOCTEXT_NAMESPACE
