// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SPropertiesDebugViewBase.h"

namespace TraceServices { class IAnalysisSession; }

class SInertializationDetailsView : public SPropertiesDebugViewBase
{
public:
	virtual void GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const override;
	virtual FName GetName() const override;

	void SetFilter(uint32 InNodeIdFilter)
	{
		NodeIdFilter = InNodeIdFilter;
		bFilterSet = true;
	}

	void ClearFilter()
	{
		bFilterSet = false;
	}

private:
	uint32 NodeIdFilter = 0;
	bool bFilterSet = false;
};