// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMontageView.h"
#include "AnimationProvider.h"
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "GameplayProvider.h"
#include "IRewindDebugger.h"
#include "Styling/SlateIconFinder.h"
#include "TraceServices/Model/Frames.h"
#include "VariantTreeNode.h"

#if WITH_EDITOR
#include "Animation/AnimInstance.h"
#endif

#define LOCTEXT_NAMESPACE "SMontageView"

void SMontageView::GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const
{
	const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if(AnimationProvider && GameplayProvider)
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
		
		AnimationProvider->ReadMontageTimeline(ObjectId, [this, &InFrame, &AnimationProvider, &GameplayProvider, &OutVariants](const FAnimationProvider::AnimMontageTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, [this, &AnimationProvider, &GameplayProvider, &OutVariants, &InFrame](double InStartTime, double InEndTime, uint32 InDepth, const FAnimMontageMessage& InMessage)
			{
				if(InStartTime >= InFrame.StartTime && InEndTime <= InFrame.EndTime)
				{
					if (!bAssetFilterSet || InMessage.MontageId == AssetIdFilter)
					{
						const FObjectInfo& MontageInfo = GameplayProvider->GetObjectInfo(InMessage.MontageId);
					
						TSharedRef<FVariantTreeNode> MontageHeader = OutVariants.Add_GetRef(FVariantTreeNode::MakeObject(FText::FromString(MontageInfo.Name), InMessage.MontageId, InMessage.MontageId, InMessage.Position));

						MontageHeader->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("EventWeight", "Weight"), InMessage.Weight));
						MontageHeader->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("EventDesiredWeight", "Desired Weight"), InMessage.DesiredWeight));
						MontageHeader->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("EventPosition", "Position"), InMessage.Position));

						const TCHAR* CurrentSectionName = AnimationProvider->GetName(InMessage.CurrentSectionNameId);
						MontageHeader->AddChild(FVariantTreeNode::MakeString(LOCTEXT("CurrentSectionName", "Current Section"), CurrentSectionName));
						const TCHAR* NextSectionName = AnimationProvider->GetName(InMessage.NextSectionNameId);
						MontageHeader->AddChild(FVariantTreeNode::MakeString(LOCTEXT("NextSectionName", "Next Section"), NextSectionName));
					}
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
	
}


static const FName MontageName("Montages");

FName SMontageView::GetName() const
{
	return MontageName;
}

#undef LOCTEXT_NAMESPACE
