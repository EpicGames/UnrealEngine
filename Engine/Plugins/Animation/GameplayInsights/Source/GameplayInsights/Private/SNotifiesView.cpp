// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNotifiesView.h"
#include "AnimationProvider.h"
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "GameplayProvider.h"
#include "IRewindDebugger.h"
#include "Styling/SlateIconFinder.h"
#include "TraceServices/Model/Frames.h"
#include "VariantTreeNode.h"

#if WITH_EDITOR
#include "Animation/AnimInstance.h"
#endif

#define LOCTEXT_NAMESPACE "SNotifiesView"

static const TCHAR* GetNotifyType(EAnimNotifyMessageType InType, bool bInForEvent)
{
	switch (InType)
	{
	case EAnimNotifyMessageType::Begin:
	case EAnimNotifyMessageType::End:
	case EAnimNotifyMessageType::Tick:
	{
		static FText StateType = LOCTEXT("AnimNotifyState", "Anim Notify State");
		static FText BeginType = LOCTEXT("AnimNotifyStateBegin", "Anim Notify State (Begin)");
		static FText EndType = LOCTEXT("AnimNotifyStateEnd", "Anim Notify State (End)");
		static FText TickType = LOCTEXT("AnimNotifyStateTick", "Anim Notify State (Tick)");
		if (bInForEvent)
		{
			switch (InType)
			{
			case EAnimNotifyMessageType::Begin:
				return *BeginType.ToString();
			case EAnimNotifyMessageType::End:
				return *EndType.ToString();
			case EAnimNotifyMessageType::Tick:
				return *TickType.ToString();
			}
		}
		else
		{
			return *StateType.ToString();
		}
	}
	case EAnimNotifyMessageType::Event:
	{
		static FText Type = LOCTEXT("AnimNotify", "Anim Notify");
		return *Type.ToString();
	}
	case EAnimNotifyMessageType::SyncMarker:
	{
		static FText Type = LOCTEXT("SyncMarker", "Sync Marker");
		return *Type.ToString();
	}
	}

	static FText UnknownType = LOCTEXT("UnknownType", "Unknown");
	return *UnknownType.ToString();
}

void SNotifiesView::GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const
{
	TSharedRef<FVariantTreeNode> Header = OutVariants.Add_GetRef(FVariantTreeNode::MakeHeader(LOCTEXT("AnimNotifiesHeader", "Notifies and Sync Markers"), 0));

	const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if(GameplayProvider && AnimationProvider)
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
		bool bForEvent = true;
		auto ProcessEvent = [this, &AnimationProvider, &GameplayProvider, &Header, &bForEvent](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNotifyMessage& InMessage)
		{
			if (bFilterIsSet && InMessage.NameId != FilterNotifyNameId)
			{
				return TraceServices::EEventEnumerate::Continue;
			}
			
			const TCHAR* Name = AnimationProvider->GetName(InMessage.NameId);
			TSharedRef<FVariantTreeNode> NotifyHeader = Header->AddChild(FVariantTreeNode::MakeHeader(FText::FromString(Name), InMessage.NameId));

			NotifyHeader->AddChild(FVariantTreeNode::MakeString(LOCTEXT("EventType", "Type"), GetNotifyType(InMessage.NotifyEventType, bForEvent)));
			NotifyHeader->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("EventDuration", "Duration"), InEndTime - InStartTime));
			NotifyHeader->AddChild(FVariantTreeNode::MakeFloat(LOCTEXT("EventTime", "Time"), InStartTime));

			if(InMessage.NotifyEventType != EAnimNotifyMessageType::SyncMarker)
			{
				NotifyHeader->AddChild(FVariantTreeNode::MakeObject(LOCTEXT("Asset", "Asset"), InMessage.AssetId, InMessage.AssetId, InMessage.Time));

				const FObjectInfo& NotifyInfo = GameplayProvider->GetObjectInfo(InMessage.NotifyId);
				bool bIsState = InMessage.NotifyEventType == EAnimNotifyMessageType::Begin || InMessage.NotifyEventType == EAnimNotifyMessageType::End || InMessage.NotifyEventType == EAnimNotifyMessageType::Tick;
				NotifyHeader->AddChild(FVariantTreeNode::MakeClass((bIsState ? LOCTEXT("Notify State Class", "Notify State Class") : LOCTEXT("NotifyClass", "Notify Class")), NotifyInfo.ClassId));
			}

			return TraceServices::EEventEnumerate::Continue;
		};

		AnimationProvider->ReadNotifyTimeline(ObjectId, [&InFrame, &ProcessEvent](const FAnimationProvider::AnimNotifyTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, ProcessEvent);
		});

		bForEvent = true;
		AnimationProvider->EnumerateNotifyStateTimelines(ObjectId, [&InFrame, &ProcessEvent](uint32 InNotifyNameId, const FAnimationProvider::AnimNotifyTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, ProcessEvent);
		});
	}
}


static const FName NotifiesName("Notifies");

FName SNotifiesView::GetName() const
{
	return NotifiesName;
}

#undef LOCTEXT_NAMESPACE
