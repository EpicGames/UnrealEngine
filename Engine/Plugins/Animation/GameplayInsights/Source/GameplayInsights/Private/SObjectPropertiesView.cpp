// Copyright Epic Games, Inc. All Rights Reserved.

#include "SObjectPropertiesView.h"
#include "AnimationProvider.h"
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "GameplayProvider.h"
#include "IRewindDebugger.h"
#include "PropertiesTrack.h"
#include "PropertyHelpers.h"
#include "PropertyWatchManager.h"
#include "TraceServices/Model/Frames.h"
#include "VariantTreeNode.h"
#include "Misc/DefaultValueHelper.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "SObjectPropertiesView"

void SObjectPropertiesView::GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const
{
	if (const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName))
	{
		TSharedRef<FVariantTreeNode> Header = OutVariants.Add_GetRef(FVariantTreeNode::MakeHeader(LOCTEXT("PropertiesHeader", "Properties"), INDEX_NONE));
		TArray<TSharedPtr<FVariantTreeNode>> PropertyVariants;

		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
		GameplayProvider->ReadObjectPropertiesTimeline(ObjectId, [this, &InFrame, GameplayProvider, &Header, &PropertyVariants](const FGameplayProvider::ObjectPropertiesTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, [this, GameplayProvider, &Header, &PropertyVariants](double InStartTime, double InEndTime, uint32 InDepth, const FObjectPropertiesMessage& InMessage)
			{
				GameplayProvider->ReadObjectPropertiesStorage(ObjectId, InMessage, [&PropertyVariants, &Header, GameplayProvider](const TConstArrayView<FObjectPropertyValue> & InStorage)
				{
					for (int32 i = 0; i < InStorage.Num(); ++i)
					{
						PropertyVariants.Add(FObjectPropertyHelpers::GetVariantNodeFromProperty(i, *GameplayProvider, InStorage));
						
						// note assumes that order is parent->child in the properties array
						if(InStorage[i].ParentId != INDEX_NONE)
						{
							PropertyVariants[InStorage[i].ParentId]->AddChild(PropertyVariants.Last().ToSharedRef());
						}
						else
						{
							Header->AddChild(PropertyVariants.Last().ToSharedRef());
						}
					}
				});

				return TraceServices::EEventEnumerate::Stop;
			});
		});
	}
}

static const FName ObjectPropertiesName("ObjectProperties");

FName SObjectPropertiesView::GetName() const
{
	return ObjectPropertiesName;
}

void SObjectPropertiesView::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	SPropertiesDebugViewBase::BuildContextMenu(MenuBuilder);

	MenuBuilder.BeginSection(GetName(), LOCTEXT("DetailsLabel", "Property"));
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddPropertyWatch", "Add Property Watch"),
		LOCTEXT("AddPropertyTooltip", "Adds a watch track for the given property"),
		{},
		FExecuteAction::CreateLambda([this]()
		{
			FPropertyWatchManager * PropertyWatchManager = FPropertyWatchManager::Instance();
			check(PropertyWatchManager)

			if (SelectedPropertyId)
			{
				if (!PropertyWatchManager->WatchProperty(ObjectId, SelectedPropertyId))
				{
					UE_LOGF(LogRewindDebugger, Warning, "Failed to watch property with id {%d}", SelectedPropertyId);
				}
			}
			else
			{
				UE_LOGF(LogRewindDebugger, Warning, "No property was selected or an invalid property was used while attempting to add a property watch track.")
			}
		}));
	MenuBuilder.EndSection();
}

#undef LOCTEXT_NAMESPACE
