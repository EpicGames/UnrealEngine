// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SPropertiesDebugViewBase.h"

namespace TraceServices { class IAnalysisSession; }

/**
 * Used to display the properties of a traced object
 */
class SObjectPropertiesView : public SPropertiesDebugViewBase
{
public:
	/** Begin SPropertiesDebugViewBase interface */
	virtual void GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const override;
	virtual FName GetName() const override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	/** End SPropertiesDebugViewBase interface */
};