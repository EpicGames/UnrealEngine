// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPoseWatchCurvesView.h"
#include "AnimationProvider.h"
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "GameplayProvider.h"
#include "TraceServices/Model/Frames.h"
#include "VariantTreeNode.h"

#define LOCTEXT_NAMESPACE "SPoseWatchCurvesView"

void SPoseWatchCurvesView::GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const
{
	const FAnimationProvider* AnimationProvider = AnalysisSession->ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if (AnimationProvider)
	{
		TSharedRef<FVariantTreeNode> Header = OutVariants.Add_GetRef(FVariantTreeNode::MakeHeader(LOCTEXT("Animation Curves", "Animation Curves"), 0));

		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadPoseWatchTimeline(ObjectId, [this, &Header, &AnimationProvider, &InFrame](const FAnimationProvider::PoseWatchTimeline& InPoseWatchTimeline)
		{
			InPoseWatchTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, [this, &Header, &AnimationProvider, &InFrame](double InStartTime, double InEndTime, uint32 InDepth, const FPoseWatchMessage& InMessage)
			{
				if (!bCurveFilterSet || InMessage.PoseWatchId == PoseWatchFilter)
				{
					if (InStartTime >= InFrame.StartTime && InStartTime <= InFrame.EndTime)
					{
						AnimationProvider->EnumeratePoseWatchCurves(InMessage, [this, &Header, &AnimationProvider](const FSkeletalMeshNamedCurve& InCurve)
						{
							if (!bCurveFilterSet || InCurve.Id == CurveFilter)
							{
								const TCHAR* CurveName = AnimationProvider->GetName(InCurve.Id);
								Header->AddChild(FVariantTreeNode::MakeFloat(FText::FromString(CurveName), InCurve.Value));
							}
						});
					}
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}

static const FName PoseWatchCurvesName("PoseWatchCurves");
FName SPoseWatchCurvesView::GetName() const
{
	return PoseWatchCurvesName;
}

#undef LOCTEXT_NAMESPACE
