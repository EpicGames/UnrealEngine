// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SPropertiesDebugViewBase.h"

namespace TraceServices { class IAnalysisSession; }

class SPoseWatchCurvesView : public SPropertiesDebugViewBase
{
public:
	virtual void GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const override;
	virtual FName GetName() const override;
	
	void SetPoseWatchCurveFilter(uint64 InPoseWatchFilter, uint32 InCurveFilter)
	{
		PoseWatchFilter = InPoseWatchFilter;
		CurveFilter = InCurveFilter;
		bCurveFilterSet = true;
	}

	void ClearPoseWatchCurveFilter()
	{
		bCurveFilterSet = false;
	}

private:
	uint64 PoseWatchFilter = 0;
	uint32 CurveFilter = 0;
	bool bCurveFilterSet = false;
};