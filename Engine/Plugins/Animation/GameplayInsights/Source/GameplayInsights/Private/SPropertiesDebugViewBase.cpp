// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPropertiesDebugViewBase.h"
#include "AnimationProvider.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "GameplayProvider.h"
#include "TraceServices/Model/Frames.h"
#include "VariantTreeNode.h"
#include "PropertiesTrack.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "SPropertiesDebugViewBase"

void SPropertiesDebugViewBase::Construct(const FArguments& InArgs, uint64 InObjectId, double InTimeMarker, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	ObjectId = InObjectId;
	AnalysisSession = &InAnalysisSession;
	CurrentTime = InArgs._CurrentTime;
	SelectedPropertyId = 0;
	
	View = SNew(SVariantValueView, InAnalysisSession)
	.OnGetVariantValues(this, &SPropertiesDebugViewBase::GetVariantsAtFrame)
	.OnContextMenuOpening(this, &SPropertiesDebugViewBase::CreateContextMenu)
	.OnMouseButtonDownOnVariantValue(this, &SPropertiesDebugViewBase::HandleOnMouseButtonDown);
	
	ChildSlot
	[
		View.ToSharedRef()
	];
}

void SPropertiesDebugViewBase::SetTimeMarker(double Time)
{
	if (TimeMarker != Time)
	{
		TimeMarker = Time;
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
		const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
		TraceServices::FFrame MarkerFrame;
		if(FramesProvider.GetFrameFromTime(ETraceFrameType::TraceFrameType_Game, TimeMarker, MarkerFrame))
		{
			View->RequestRefresh(MarkerFrame);
		}
	}
}

void SPropertiesDebugViewBase::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if (CurrentTime.IsBound())
	{
		SetTimeMarker(CurrentTime.Get());
	}

	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SPropertiesDebugViewBase::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
}

TSharedPtr<SWidget> SPropertiesDebugViewBase::CreateContextMenu() 
{
	FMenuBuilder MenuBuilder(true, nullptr);
	BuildContextMenu(MenuBuilder);
	return MenuBuilder.MakeWidget();
}

void SPropertiesDebugViewBase::HandleOnMouseButtonDown(const TSharedPtr<FVariantTreeNode>& InVariantValueNode, const TraceServices::FFrame& InFrame, const FPointerEvent& InKeyEvent)
{
	SelectedPropertyId = InVariantValueNode->GetPropertyNamedId();
}

#undef LOCTEXT_NAMESPACE
