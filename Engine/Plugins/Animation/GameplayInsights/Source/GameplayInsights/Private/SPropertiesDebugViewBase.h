// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRewindDebuggerView.h"
#include "IRewindDebuggerViewCreator.h"
#include "SVariantValueView.h"

namespace TraceServices { class IAnalysisSession; }

class FMenuBuilder;

class SPropertiesDebugViewBase : public IRewindDebuggerView
{
	SLATE_BEGIN_ARGS(SPropertiesDebugViewBase) {}
	SLATE_ATTRIBUTE(double, CurrentTime)
	SLATE_END_ARGS()
public:
	void Construct(const FArguments& InArgs, uint64 InObjectId, double InTimeMarker, const TraceServices::IAnalysisSession& InAnalysisSession);

	/** Begin IRewindDebuggerView interface */
	virtual void SetTimeMarker(double InTimeMarker) override;
	virtual uint64 GetObjectId() const override { return ObjectId; }
	/** End IRewindDebuggerView interface */

	/** Begin SWidget interface */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	/** End SWidget interface */

	/** Get all the variants / properties at the given frame */
	virtual void GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const = 0;

	/** Build the context menu for the view widget */
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder);

	/** @return Inner view widget for the object variant */
	const TSharedPtr<SVariantValueView> & GetView() const { return View; };
	
protected:

	// Event handlers

	/** Setup base context menu */
	TSharedPtr<SWidget> CreateContextMenu();
	
	/** Handle mouse button event from widget */
	void HandleOnMouseButtonDown(const TSharedPtr<FVariantTreeNode> & InVariantValueNode, const TraceServices::FFrame & InFrame, const FPointerEvent& InKeyEvent);
	
	TSharedPtr<SVariantValueView> View;

	uint64 ObjectId;
	double TimeMarker;
	uint32 SelectedPropertyId;
	
	TAttribute<double> CurrentTime;
	const TraceServices::IAnalysisSession* AnalysisSession;
};