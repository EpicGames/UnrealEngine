// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPropertyTrackView.h"

#include "Common/ProviderLock.h"
#include "GameplayProvider.h"
#include "PropertyHelpers.h"
#include "PropertyTrack.h"
#include "VariantTreeNode.h"
#include "PropertyWatchManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "SPropertyTrackView"

void SPropertyTrackView::GetVariantsAtFrame(const TraceServices::FFrame& InFrame,TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const
{
	FMemMark Mark(FMemStack::Get());
	
	if (const TSharedPtr<RewindDebugger::FPropertyTrack> PropertyTrack = Track.Pin())
	{
		TSharedPtr<FVariantTreeNode> Header = OutVariants.Add_GetRef(FVariantTreeNode::MakeHeader(FText::FromName(PropertyTrack->GetPropertyName()), INDEX_NONE));

		if (const FGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			TArray<TSharedPtr<FVariantTreeNode>, TMemStackAllocator<>> PropertyVariants;
			
			PropertyTrack->ReadObjectPropertyValueAtFrame(InFrame, *GameplayProvider, [GameplayProvider, &PropertyTrack, &Header, &PropertyVariants, &OutVariants](const FObjectPropertyValue& InValue, uint32 InValueIndex, const FObjectPropertiesMessage& InMessage)
			{
				GameplayProvider->ReadObjectPropertiesStorage(PropertyTrack->GetUObjectId(), InMessage, [GameplayProvider, InValueIndex, &PropertyTrack, &Header, &PropertyVariants, &OutVariants](const TConstArrayView<FObjectPropertyValue> & InStorage)
				{
					// Handle case where out property has children.
					for (int32 i = InValueIndex; i < InStorage.Num(); ++i)
					{
						PropertyVariants.Add(FObjectPropertyHelpers::GetVariantNodeFromProperty(i, *GameplayProvider, InStorage));
						
						// note assumes that order is parent->child in the properties array
						if (InStorage[i].ParentId != INDEX_NONE && uint32(InStorage[i].ParentId) >= InValueIndex && InStorage[i].ParentId < i)
						{
							if (InStorage[i].ParentId != INDEX_NONE && InStorage[i].ParentId != InValueIndex)
							{
								PropertyVariants[InStorage[i].ParentId - InValueIndex]->AddChild(PropertyVariants.Last().ToSharedRef());
							}
							else
							{
								Header->AddChild(PropertyVariants.Last().ToSharedRef());
							}
						}
						else if (i != InValueIndex)
						{
							break;
						}
					}

					// Single property case.
					if (Header->GetChildren().IsEmpty())
					{
						// No need for a header.
						OutVariants.Pop();
						
						OutVariants.Add(PropertyVariants[0].ToSharedRef());
					}
				});
			});
		}
		else
		{
			Header->AddChild(FVariantTreeNode::MakeString(FText::FromName(PropertyTrack->GetPropertyName()), PropertyTrack->GetObjectProperty()->Property.Value));
		}
	}
}

FName SPropertyTrackView::GetName() const
{
	return Track.IsValid() ? Track.Pin()->GetName() : FName(TEXT("SPropertyTrackView"));
}

void SPropertyTrackView::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	SPropertiesDebugViewBase::BuildContextMenu(MenuBuilder);

	MenuBuilder.BeginSection(GetName(), LOCTEXT("DetailsLabel", "Property"));
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddPropertyWatch", "Add Property Watch"),
		LOCTEXT("AddPropertyTooltip", "Adds a watch track for the given property"),
		{},
		FExecuteAction::CreateLambda([this]()
		{
			FPropertyWatchManager * PropertyWatchManager = FPropertyWatchManager::Instance();
			check(PropertyWatchManager)

			if (SelectedPropertyId)
			{
				if (!PropertyWatchManager->WatchProperty(ObjectId, SelectedPropertyId))
				{
					UE_LOGF(LogRewindDebugger, Warning, "Failed to watch property with id {%d}", SelectedPropertyId);
				}
			}
			else
			{
				UE_LOGF(LogRewindDebugger, Warning, "No property was selected or an invalid property was used while attempting to add a property watch track.")
			}
		}));
	MenuBuilder.EndSection();
}


void SPropertyTrackView::SetTrack(const TWeakPtr<RewindDebugger::FPropertyTrack>& InTrack)
{
	Track = InTrack;
}

#undef LOCTEXT_NAMESPACE
