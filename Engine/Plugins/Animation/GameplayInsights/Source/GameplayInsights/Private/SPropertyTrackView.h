// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertiesTrack.h"

#include "SPropertiesDebugViewBase.h"

namespace RewindDebugger
{
	class FPropertyTrack;
}

namespace TraceServices { class IAnalysisSession; }

/**
 * Used to display the properties of a single traced variable. 
 */
class SPropertyTrackView : public SPropertiesDebugViewBase
{
public:

	/** Begin SPropertiesDebugViewBase interface */
	virtual void GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const override;
	virtual FName GetName() const override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	/** End SPropertiesDebugViewBase interface */

	/** Update the track from which information will be queried and displayed */
	void SetTrack(const TWeakPtr<RewindDebugger::FPropertyTrack> & InTrack);

private:
	TWeakPtr<RewindDebugger::FPropertyTrack> Track = nullptr;
};