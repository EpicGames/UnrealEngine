// Copyright Epic Games, Inc. All Rights Reserved.

#include "STrackVariantValueView.h"

#include "SVariantValueView.h"
#include "GameplayTrack.h"
#include "GameplayGraphTrack.h"

// TraceInsights
#include "Insights/ViewModels/BaseTimingTrack.h"
#include "Insights/ITimingViewSession.h"

void STrackVariantValueView::Construct(const FArguments& InArgs, const TSharedRef<FBaseTimingTrack>& InTimingTrack, UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	TimingTrack = InTimingTrack;
	AnalysisSession = &InAnalysisSession;

	InTimingViewSession.OnTimeMarkerChanged().AddSP(this, &STrackVariantValueView::HandleTimeMarkerChanged);

	ChildSlot
	[
		SAssignNew(VariantValueView, SVariantValueView, InAnalysisSession)
		.OnGetVariantValues_Lambda([this](const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutValues)
		{
			TSharedPtr<FBaseTimingTrack> PinnedTrack = TimingTrack.Pin();
			if(PinnedTrack.IsValid())
			{
				if(PinnedTrack->Is<FGameplayTimingEventsTrack>())
				{
					StaticCastSharedPtr<FGameplayTimingEventsTrack>(PinnedTrack)->GetVariantsAtFrame(InFrame, OutValues);
				}
				else if(PinnedTrack->Is<FGameplayGraphTrack>())
				{
					StaticCastSharedPtr<FGameplayGraphTrack>(PinnedTrack)->GetVariantsAtFrame(InFrame, OutValues);
				}
			}
		})
	];

	TraceServices::FAnalysisSessionReadScope SessionReadScope(InAnalysisSession);

	const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(InAnalysisSession);
	TraceServices::FFrame MarkerFrame;
	if(FramesProvider.GetFrameFromTime(ETraceFrameType::TraceFrameType_Game, InTimingViewSession.GetTimeMarker(), MarkerFrame))
	{
		VariantValueView->RequestRefresh(MarkerFrame);
	}
}

void STrackVariantValueView::HandleTimeMarkerChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InTimeMarker)
{
	TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

	const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
	TraceServices::FFrame MarkerFrame;
	if(FramesProvider.GetFrameFromTime(ETraceFrameType::TraceFrameType_Game, InTimeMarker, MarkerFrame))
	{
		VariantValueView->RequestRefresh(MarkerFrame);
	}
}
