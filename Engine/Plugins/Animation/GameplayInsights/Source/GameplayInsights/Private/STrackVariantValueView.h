// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FBaseTimingTrack;
namespace TraceServices { class IAnalysisSession; }
namespace UE::Insights::Timing { class ITimingViewSession; }
namespace UE::Insights::Timing { enum class ETimeChangedFlags; }
class SVariantValueView;

// Wrapper for a variant value view for a track
class STrackVariantValueView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STrackVariantValueView) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FBaseTimingTrack>& InTimingTrack, UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession);

	TSharedPtr<FBaseTimingTrack> GetTimingTrack() const { return TimingTrack.Pin(); }

private:
	// Handle the time marker being scrubbed
	void HandleTimeMarkerChanged(UE::Insights::Timing::ETimeChangedFlags InFlags, double InTimeMarker);

private:
	TWeakPtr<FBaseTimingTrack> TimingTrack;

	TSharedPtr<SVariantValueView> VariantValueView;

	const TraceServices::IAnalysisSession* AnalysisSession;
};
