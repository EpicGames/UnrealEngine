// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVariantValueView.h"
#include "Common/ProviderLock.h"
#include "Widgets/Input/SSearchBox.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "GameplayProvider.h"
#include "AnimationProvider.h"
#include "TraceServices/Model/Frames.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/CoreStyle.h"
#include "GameplayInsightsStyle.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "VariantTreeNode.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/StringBuilder.h"

#if WITH_EDITOR
#include "AnimPreviewInstance.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Editor.h"
#include "IAnimationEditor.h"
#include "IAnimationBlueprintEditor.h"
#include "IPersonaToolkit.h"
#include "SourceCodeNavigation.h"
#include "Subsystems/AssetEditorSubsystem.h"
#endif

#define LOCTEXT_NAMESPACE "SVariantValueView"

namespace VariantColumns
{
	static const FName Name("Name");
	static const FName Value("Value");
};

TSharedRef<SWidget> SVariantValueView::MakeVariantValueWidget(const TraceServices::IAnalysisSession& InAnalysisSession, const FVariantValue& InValue, const TAttribute<FText>& InHighlightText)
{ 
	switch(InValue.Type)
	{
	case EAnimNodeValueType::Bool:
		return 
			SNew(SCheckBox)
			.IsEnabled(false)
			.IsChecked(InValue.Bool.bValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);

	case EAnimNodeValueType::Int32:
		return
			SNew(SBox)
			.WidthOverride(125.0f)
			[
				SNew(STextBlock)
				.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
				.Text(FText::AsNumber(InValue.Int32.Value))
				.HighlightText(InHighlightText)
			];

	case EAnimNodeValueType::Float:
		return 
			SNew(SBox)
			.WidthOverride(125.0f)
			[
				SNew(STextBlock)
				.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
				.Text(FText::AsNumber(InValue.Float.Value))
				.HighlightText(InHighlightText)
			];

	case EAnimNodeValueType::Vector2D:
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(125.0f)
				[
					SNew(STextBlock)
					.IsEnabled(false)
					.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
					.Text(FText::AsNumber(InValue.Vector2D.Value.X))
					.HighlightText(InHighlightText)
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(125.0f)
				[
					SNew(STextBlock)
					.IsEnabled(false)
					.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
					.Text(FText::AsNumber(InValue.Vector2D.Value.Y))
					.HighlightText(InHighlightText)
				]
			];

	case EAnimNodeValueType::Vector:
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(125.0f)
				[
					SNew(STextBlock)
					.IsEnabled(false)
					.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
					.Text(FText::AsNumber(InValue.Vector.Value.X))
					.HighlightText(InHighlightText)
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(125.0f)
				[
					SNew(STextBlock)
					.IsEnabled(false)
					.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
					.Text(FText::AsNumber(InValue.Vector.Value.Y))
					.HighlightText(InHighlightText)
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(125.0f)
				[
					SNew(STextBlock)
					.IsEnabled(false)
					.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
					.Text(FText::AsNumber(InValue.Vector.Value.Z))
					.HighlightText(InHighlightText)
				]
			];

	case EAnimNodeValueType::String:
		return 
			SNew(STextBlock)
			.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
			.Text(FText::FromString(InValue.String.Value))
			.HighlightText(InHighlightText);

	case EAnimNodeValueType::Object:
	{
		const FGameplayProvider* GameplayProvider = InAnalysisSession.ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
		if(GameplayProvider)
		{
			FString ObjectName;
			FString ObjectPathName;
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
				const FObjectInfo& ObjectInfo = GameplayProvider->GetObjectInfo(InValue.Object.Value);
				ObjectName = ObjectInfo.Name;
				ObjectPathName = ObjectInfo.PathName;
			}
#if WITH_EDITOR
			return
				SNew(SHyperlink)
				.Text(FText::FromString(ObjectName))
				.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
				.ToolTipText(FText::Format(LOCTEXT("AssetHyperlinkTooltipFormat", "Open asset '{0}'"), FText::FromString(ObjectPathName)))
				.OnNavigate_Lambda([ObjectPathName, InValue]()
				{
					UObject* Asset = nullptr;

					FString PackagePathString = FPackageName::ObjectPathToPackageName(ObjectPathName);

					UPackage* Package = LoadPackage(NULL, ToCStr(PackagePathString), LOAD_NoRedirects);
					if (Package)
					{
						Package->FullyLoad();

						FString AssetName = FPaths::GetBaseFilename(ObjectPathName);
						Asset = FindObject<UObject>(Package, *AssetName);
					}
					else
					{
						// fallback for unsaved assets
						Asset = FindObject<UObject>(nullptr, *ObjectPathName);
					}

					if (Asset != nullptr)
					{
						if (UAssetEditorSubsystem* AssetEditorSS = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
						{
							AssetEditorSS->OpenEditorForAsset(Asset);
							if (IAssetEditorInstance* Editor = AssetEditorSS->FindEditorForAsset(Asset, true))
							{
								if (Editor->GetEditorName()=="AnimationEditor")
								{
									IAnimationEditor* AnimationEditor = static_cast<IAnimationEditor*>(Editor);
									UDebugSkelMeshComponent* PreviewComponent = AnimationEditor->GetPersonaToolkit()->GetPreviewMeshComponent();
									PreviewComponent->PreviewInstance->SetPosition(InValue.Object.PlaybackTime);
									PreviewComponent->PreviewInstance->SetPlaying(false);
									PreviewComponent->PreviewInstance->SetBlendSpacePosition(FVector(InValue.Object.BlendX, InValue.Object.BlendY, 0.0f));
								}
							}
						}
					}
				});
#else
			return
				SNew(STextBlock)
				.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
				.Text(FText::FromString(ObjectName))
				.ToolTipText(FText::FromString(ObjectPathName))
				.HighlightText(InHighlightText);
#endif
		}
	}
	break;

	case EAnimNodeValueType::Class:
	{
		const FGameplayProvider* GameplayProvider = InAnalysisSession.ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
		if(GameplayProvider)
		{
			FString ClassName;
			FString ClassPathName;
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
				const FClassInfo& ClassInfo = GameplayProvider->GetClassInfo(InValue.Class.Value);
				ClassName = ClassInfo.Name;
				ClassPathName = ClassInfo.PathName;
			}
#if WITH_EDITOR
			return
				SNew(SHyperlink)
				.Text(FText::FromString(ClassName))
				.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
				.ToolTipText(FText::Format(LOCTEXT("ClassHyperlinkTooltipFormat", "Open class '{0}'"), FText::FromString(ClassPathName)))
				.OnNavigate_Lambda([ClassName, ClassPathName]()
				{
					if (UClass* FoundClass = UClass::TryFindTypeSlow<UClass>(ClassName))
					{
						if (FoundClass->IsNative())
						{
							// for native classes navigatte to source code
							FSourceCodeNavigation::NavigateToClass(FoundClass);
						}
					}

					// for non-native classes, try opening the asset
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ClassPathName);
				});
#else
			return
				SNew(STextBlock)
				.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
				.Text(FText::FromString(ClassName))
				.ToolTipText(FText::FromString(ClassPathName))
				.HighlightText(InHighlightText);
#endif
		}
	}

	case EAnimNodeValueType::AnimNode:
	{
		const FGameplayProvider* GameplayProvider = InAnalysisSession.ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
		const FAnimationProvider* AnimationProvider = InAnalysisSession.ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

		if (GameplayProvider && AnimationProvider)
		{
			FString AnimNodeName;
			int32 AnimNodeId = INDEX_NONE;
			FString AnimInstanceClassPathName;
			{
				TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
				TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);

				const FAnimNodeInfo* AnimNodeInfo = AnimationProvider->FindAnimNodeInfo(InValue.AnimNode.Value, InValue.AnimNode.AnimInstanceId);
				const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(InValue.AnimNode.AnimInstanceId);
				const FClassInfo* AnimInstanceClassInfo = ObjectInfo ? GameplayProvider->FindClassInfo(ObjectInfo->ClassId) : nullptr;

				if (AnimNodeInfo && ObjectInfo && AnimInstanceClassInfo)
				{
					AnimNodeName = AnimNodeInfo->Name;
					AnimNodeId = AnimNodeInfo->Id;
					AnimInstanceClassPathName = AnimInstanceClassInfo->PathName;
				}
			}

			if (AnimNodeId != INDEX_NONE)
			{
#if WITH_EDITOR
				return
					SNew(SHyperlink)
					.Text(FText::FromString(AnimNodeName))
					.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
					.ToolTipText(FText::Format(LOCTEXT("NodeHyperlinkTooltipFormat", "Open node '{0}'"), FText::FromString(AnimNodeName)))
					.OnNavigate_Lambda([AnimNodeId, AnimInstanceClassPathName, InValue]()
					{
						TSoftObjectPtr<UAnimBlueprintGeneratedClass> InstanceClass;
						InstanceClass = FSoftObjectPath(AnimInstanceClassPathName);

						if (InstanceClass.LoadSynchronous())
						{
							if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(InstanceClass.Get()->ClassGeneratedBy))
							{
								GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AnimBlueprint);

								if (IAnimationBlueprintEditor* AnimBlueprintEditor = static_cast<IAnimationBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AnimBlueprint, true)))
								{
									int32 AnimNodeIndex = InstanceClass.Get()->GetAnimNodeProperties().Num() - AnimNodeId - 1;
									TWeakObjectPtr<const UEdGraphNode>* GraphNode = InstanceClass.Get()->AnimBlueprintDebugData.NodePropertyIndexToNodeMap.Find(AnimNodeIndex);
									if (GraphNode != nullptr && GraphNode->Get())
									{
										AnimBlueprintEditor->JumpToHyperlink(GraphNode->Get());
									}
								}
							}
						}
					});
#else
				return
					SNew(STextBlock)
					.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
					.Text(FText::FromString(AnimNodeName))
					.ToolTipText(FText::FromString(AnimNodeName))
					.HighlightText(InHighlightText);
#endif
			}
		}
	}
	break;
	}

	return SNullWidget::NullWidget; 
}

// Container for an entry in the property view
class SVariantValueNode : public SMultiColumnTableRow<TSharedRef<FVariantTreeNode>>
{
public:
	SLATE_BEGIN_ARGS(SVariantValueNode) {}

	SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_EVENT(FOnMouseButtonDownOnVariantValueNode, OnMouseButtonDownOnVariantValueNode)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FVariantTreeNode> InNode, const TraceServices::IAnalysisSession& InAnalysisSession)
	{
		Node = InNode;
		AnalysisSession = &InAnalysisSession;
		HighlightText = InArgs._HighlightText;
		OnMouseButtonDownOnVariantValueNode = InArgs._OnMouseButtonDownOnVariantValueNode;
		
		SMultiColumnTableRow<TSharedRef<FVariantTreeNode>>::Construct(
			FSuperRowType::FArguments()
			.Padding(1.0f),
			InOwnerTable
		);
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		OnMouseButtonDownOnVariantValueNode.ExecuteIfBound(Node, MouseEvent);

		/** We return a FReply::Unhandled to allow for event to bubble up in order for context menu to be built. */
		return FReply::Unhandled();
	}
	
	// SMultiColumnTableRow interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		const bool bIsRoot = !Node->GetParent().IsValid();

		if (InColumnName == VariantColumns::Name)
		{
			return 
				SNew(SBorder)
				.BorderImage(bIsRoot ? FGameplayInsightsStyle::Get().GetBrush("SchematicViewRootLeft") : FCoreStyle::Get().GetBrush("NoBorder"))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(6, 0, 0, 0)
					.VAlign(VAlign_Center)
					[
						SNew(SExpanderArrow, SharedThis(this))
						.IndentAmount(4)
					]
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(FCoreStyle::Get().GetFontStyle(bIsRoot ? "ExpandableArea.TitleFont" : "SmallFont"))
						.Text(Node->GetName())
						.HighlightText(HighlightText)
					]
				];
		}
		else if(InColumnName == VariantColumns::Value)
		{
			return
				SNew(SBorder)
				.BorderImage(bIsRoot ? FGameplayInsightsStyle::Get().GetBrush("SchematicViewRootMid") : FCoreStyle::Get().GetBrush("NoBorder"))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SVariantValueView::MakeVariantValueWidget(*AnalysisSession, Node->GetValue(), HighlightText)
					]
				];
		}

		return SNullWidget::NullWidget;
	}

	const TraceServices::IAnalysisSession* AnalysisSession;
	TSharedPtr<FVariantTreeNode> Node;
	TAttribute<FText> HighlightText;
	FOnMouseButtonDownOnVariantValueNode OnMouseButtonDownOnVariantValueNode;
};

void SVariantValueView::Construct(const FArguments& InArgs, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	OnGetVariantValues = InArgs._OnGetVariantValues;
	OnMouseButtonDownOnVariantValue = InArgs._OnMouseButtonDownOnVariantValue;
	
	AnalysisSession = &InAnalysisSession;
	bNeedsRefresh = false;
	bRecordExpansion = true;

	TextFilter = MakeShared<FTextFilterExpressionEvaluator>(ETextFilterExpressionEvaluatorMode::BasicString);

	VariantTreeView = SNew(STreeView<TSharedRef<FVariantTreeNode>>)
		.SelectionMode(ESelectionMode::Multi)
		.OnGenerateRow(this, &SVariantValueView::HandleGeneratePropertyRow)
		.OnGetChildren(this, &SVariantValueView::HandleGetPropertyChildren)
		.OnExpansionChanged(this, &SVariantValueView::HandleExpansionChanged)
		.OnContextMenuOpening(InArgs._OnContextMenuOpening)
		.TreeItemsSource(&FilteredNodes)
		.HeaderRow(
			SNew(SHeaderRow)
			.Visibility(EVisibility::Collapsed)
			+SHeaderRow::Column(VariantColumns::Name)
			.DefaultLabel(LOCTEXT("ValueNameColumn", "Name"))
			+SHeaderRow::Column(VariantColumns::Value)
			.DefaultLabel(LOCTEXT("ValueValueColumn", "Value"))
		);
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSearchBox)
			.OnTextChanged_Lambda([this](const FText& InText){ FilterText = InText; RefreshFilter(); })
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SScrollBorder, VariantTreeView.ToSharedRef())
			[
				VariantTreeView.ToSharedRef()
			]
		]
	];

	RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime)
	{
		if(bNeedsRefresh)
		{
			RefreshNodes();
			bNeedsRefresh = false;
		}
		return EActiveTimerReturnType::Continue;
	}));

	BindCommands();
}

void SVariantValueView::BindCommands()
{
	CommandList = MakeShared<FUICommandList>();

	CommandList->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SVariantValueView::HandleCopy),
		FCanExecuteAction::CreateSP(this, &SVariantValueView::CanCopy));
}

FReply SVariantValueView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedRef<ITableRow> SVariantValueView::HandleGeneratePropertyRow(TSharedRef<FVariantTreeNode> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return 
		SNew(SVariantValueNode, OwnerTable, Item, *AnalysisSession)
		.HighlightText(MakeAttributeLambda([this](){ return FilterText; }))
		.OnMouseButtonDownOnVariantValueNode_Lambda([this](const TSharedPtr<FVariantTreeNode> & InVariantValueNode, const FPointerEvent & InPointerEvent)
		{
			OnMouseButtonDownOnVariantValue.ExecuteIfBound(InVariantValueNode, Frame, InPointerEvent);
		});
}

void SVariantValueView::HandleGetPropertyChildren(TSharedRef<FVariantTreeNode> InItem, TArray<TSharedRef<FVariantTreeNode>>& OutChildren)
{
	for(const TSharedRef<FVariantTreeNode>& Child : InItem->GetChildren())
	{
		if(Child->GetFilterState() != EVariantTreeNodeFilterState::Hidden)
		{
			OutChildren.Add(Child);
		}
	}
}

void SVariantValueView::HandleExpansionChanged(TSharedRef<FVariantTreeNode> InItem, bool bInExpanded)
{
	if(bRecordExpansion)
	{
		if(InItem->GetId() != INDEX_NONE)
		{
			if(bInExpanded)
			{
				ExpandedIds.Add(InItem->GetId());
			}
			else
			{
				ExpandedIds.Remove(InItem->GetId());
			}
		}
	}
}

void SVariantValueView::RefreshExpansionRecursive(const TSharedRef<FVariantTreeNode>& InVariantTreeNode)
{
	for(const TSharedRef<FVariantTreeNode>& ChildNode : InVariantTreeNode->GetChildren())
	{
		VariantTreeView->SetItemExpansion(ChildNode, ExpandedIds.Contains(ChildNode->GetId()));

		RefreshExpansionRecursive(ChildNode);
	}
}

void SVariantValueView::RefreshNodes()
{
	VariantTreeNodes.Reset();

	OnGetVariantValues.ExecuteIfBound(Frame, VariantTreeNodes);

	if(VariantTreeNodes.Num() > 0)
	{
		for(const TSharedRef<FVariantTreeNode>& VariantTreeNode : VariantTreeNodes)
		{
			RefreshExpansionRecursive(VariantTreeNode);
		}
	}

	RefreshFilter();
}

/** Filter utility class */
class FVariantValueViewFilterContext : public ITextFilterExpressionContext
{
public:
	explicit FVariantValueViewFilterContext(const FString& InString)
		: String(InString)
	{
	}

	virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		return TextFilterUtils::TestBasicStringExpression(String, InValue, InTextComparisonMode);
	}

	virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		return false;
	}

private:
	const FString& String;
};

EVariantTreeNodeFilterState SVariantValueView::RefreshFilter_Helper(const TSharedRef<FVariantTreeNode>& InNode)
{
	InNode->SetFilterState(EVariantTreeNodeFilterState::Hidden);

	for(const TSharedRef<FVariantTreeNode>& ChildNode : InNode->GetChildren())
	{
		EVariantTreeNodeFilterState ChildFilterState = RefreshFilter_Helper(ChildNode);
		InNode->SetFilterState(FMath::Max(ChildFilterState, InNode->GetFilterState()));
	}
	
	if(InNode->GetFilterState() == EVariantTreeNodeFilterState::Hidden)
	{
		if(FilterText.IsEmpty())
		{
			InNode->SetFilterState(EVariantTreeNodeFilterState::Visible);
		}
		else if(TextFilter->TestTextFilter(FVariantValueViewFilterContext(InNode->GetName().ToString())) ||
				TextFilter->TestTextFilter(FVariantValueViewFilterContext(InNode->GetValueAsString(*AnalysisSession))))
		{
			InNode->SetFilterState(EVariantTreeNodeFilterState::Highlighted);
		}
		else
		{
			InNode->SetFilterState(EVariantTreeNodeFilterState::Hidden);
		}
	}

	{
		TGuardValue<bool> GuardValue(bRecordExpansion, false);
		bool bAutoExpand = !FilterText.IsEmpty() && InNode->GetFilterState() == EVariantTreeNodeFilterState::Highlighted;
		VariantTreeView->SetItemExpansion(InNode, bAutoExpand);
	}

	return InNode->GetFilterState();
}

void SVariantValueView::RefreshFilter()
{
	FilteredNodes.Reset();
	
	TextFilter->SetFilterText(FilterText);

	for(const TSharedRef<FVariantTreeNode>& VariantTreeNode : VariantTreeNodes)
	{
		EVariantTreeNodeFilterState FilterState = RefreshFilter_Helper(VariantTreeNode);
		if(FilterState != EVariantTreeNodeFilterState::Hidden)
		{
			FilteredNodes.Add(VariantTreeNode);
		}

		// Always expand first level
		VariantTreeView->SetItemExpansion(VariantTreeNode, true);
	}

	if (FilterText.IsEmpty())
	{
		// Restore standard expansion
		for (const TSharedRef<FVariantTreeNode>& VariantTreeNode : VariantTreeNodes)
		{
			RefreshExpansionRecursive(VariantTreeNode);
		}
	}

	VariantTreeView->RequestTreeRefresh();
}

void SVariantValueView::CopyHelper(const TSharedRef<FVariantTreeNode>& InNode, TStringBuilder<512>& InOutStringBuilder)
{
	if (InNode->GetFilterState() != EVariantTreeNodeFilterState::Hidden && VariantTreeView->IsItemSelected(InNode))
	{
		const TCHAR* ExpansionString = TEXT(" ");
		if (InNode->GetChildren().Num() > 0)
		{
			ExpansionString = VariantTreeView->IsItemExpanded(InNode) ? TEXT("+") : TEXT("-");
		}
		InOutStringBuilder.Appendf(TEXT("%s %s\t\t%s\r\n"), ExpansionString, *InNode->GetName().ToString(), *InNode->GetValueAsString(*AnalysisSession));
	}

	for (const TSharedRef<FVariantTreeNode>& ChildNode : InNode->GetChildren())
	{
		CopyHelper(ChildNode, InOutStringBuilder);
	}
}

void SVariantValueView::HandleCopy()
{
	TStringBuilder<512> StringBuilder;
	for (const TSharedRef<FVariantTreeNode>& VariantTreeNode : FilteredNodes)
	{
		CopyHelper(VariantTreeNode, StringBuilder);
	}

	FPlatformApplicationMisc::ClipboardCopy(StringBuilder.GetData());
}

bool SVariantValueView::CanCopy() const
{
	return VariantTreeView->GetNumItemsSelected() > 0;
}

#undef LOCTEXT_NAMESPACE
