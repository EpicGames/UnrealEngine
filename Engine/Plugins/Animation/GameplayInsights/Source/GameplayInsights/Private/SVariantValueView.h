// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "TraceServices/Model/Frames.h"
#include "Containers/StringFwd.h"

namespace TraceServices { class IAnalysisSession; }
struct FVariantValue;
struct FVariantTreeNode;
class FTextFilterExpressionEvaluator;
enum class EVariantTreeNodeFilterState;
class FUICommandList;

// Delegate called to get variant values to display
DECLARE_DELEGATE_TwoParams(FOnGetVariantValues, const TraceServices::FFrame& /*InFrame*/, TArray<TSharedRef<FVariantTreeNode>>& /*OutValues*/);
DECLARE_DELEGATE_ThreeParams(FOnMouseButtonDownOnVariantValue, const TSharedPtr<FVariantTreeNode> & /*InVariantValueNode*/, const TraceServices::FFrame & /*InFrame*/, const FPointerEvent& /*InKeyEvent*/);
DECLARE_DELEGATE_TwoParams(FOnMouseButtonDownOnVariantValueNode, const TSharedPtr<FVariantTreeNode> & /*InVariantValueNode*/, const FPointerEvent& /*InKeyEvent*/);

class SVariantValueView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SVariantValueView) {}

	SLATE_EVENT(FOnGetVariantValues, OnGetVariantValues)
	SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
	SLATE_EVENT(FOnMouseButtonDownOnVariantValue, OnMouseButtonDownOnVariantValue)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TraceServices::IAnalysisSession& InAnalysisSession);

	/** Refresh the displayed variants. */
	void RequestRefresh(const TraceServices::FFrame& InFrame) { Frame = InFrame; bNeedsRefresh = true; }

	/** @return Trace frame that the view represents */
	const TraceServices::FFrame & GetFrame() const { return Frame; }

	/** @return Trace session that the view references */
	const TraceServices::IAnalysisSession&  GetAnalysisSession() const { return *AnalysisSession; }
	
	/** @return New Widget for the given Variant Value */
	static TSharedRef<SWidget> MakeVariantValueWidget(const TraceServices::IAnalysisSession& InAnalysisSession, const FVariantValue& InValue, const TAttribute<FText>& InHighlightText);

private:
	// SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	// Bind commands to actions
	void BindCommands();

	// Generate a row widget for a property item
	TSharedRef<ITableRow> HandleGeneratePropertyRow(TSharedRef<FVariantTreeNode> Item, const TSharedRef<STableViewBase>& OwnerTable);

	// Get the children of a property item
	void HandleGetPropertyChildren(TSharedRef<FVariantTreeNode> InItem, TArray<TSharedRef<FVariantTreeNode>>& OutChildren);

	// Persist expansion state when it changes
	void HandleExpansionChanged(TSharedRef<FVariantTreeNode> InItem, bool bInExpanded);

	// Refresh the nodes
	void RefreshNodes();
	
	// Helper function for RefreshFilter()
	EVariantTreeNodeFilterState RefreshFilter_Helper(const TSharedRef<FVariantTreeNode>& InNode);

	// Refresh the filtered nodes
	void RefreshFilter();

	// Refresh node expansion state
	void RefreshExpansionRecursive(const TSharedRef<FVariantTreeNode>& InVariantTreeNode);

	// Recursive helper function for copying
	void CopyHelper(const TSharedRef<FVariantTreeNode>& InNode, TStringBuilder<512>& InOutStringBuilder);

	// Handle Ctrl+C
	void HandleCopy();

	// Check if copy can occur
	bool CanCopy() const;

private:
	const TraceServices::IAnalysisSession* AnalysisSession;

	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;

	TSharedPtr<STreeView<TSharedRef<FVariantTreeNode>>> VariantTreeView;

	TArray<TSharedRef<FVariantTreeNode>> VariantTreeNodes;

	TArray<TSharedRef<FVariantTreeNode>> FilteredNodes;

	TSharedPtr<FUICommandList> CommandList;

	TraceServices::FFrame Frame;

	FOnGetVariantValues OnGetVariantValues;

	FOnMouseButtonDownOnVariantValue OnMouseButtonDownOnVariantValue;
	
	bool bNeedsRefresh;

	bool bRecordExpansion;

	TSet<uint64> ExpandedIds;

	FText FilterText;
};
