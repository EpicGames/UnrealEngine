// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshCurvesTrack.h"
#include "Common/ProviderLock.h"
#include "GameplayProvider.h"
#include "AnimationProvider.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "AnimationSharedData.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "Modules/ModuleManager.h"
#include "Insights/ViewModels/GraphTrackBuilder.h"
#include "VariantTreeNode.h"
#include "TraceServices/Model/Frames.h"

#define LOCTEXT_NAMESPACE "SkeletalMeshCurvesTrack"

INSIGHTS_IMPLEMENT_RTTI(FSkeletalMeshCurvesTrack)

FSkeletalMeshCurvesTrack::FSkeletalMeshCurvesTrack(const FAnimationSharedData& InSharedData, uint64 InObjectID, const TCHAR* InName)
	: FGameplayGraphTrack(InSharedData.GetGameplaySharedData(), InObjectID, FText::Format(LOCTEXT("TrackNameFormat", "Curves - {0}"), FText::FromString(FString(InName))))
	, SharedData(InSharedData)
{
}

void FSkeletalMeshCurvesTrack::AddAllSeries()
{
	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	bool bFirstSeries = AllSeries.Num() == 0;
	if(AnimationProvider)
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);

		AnimationProvider->EnumerateSkeletalMeshCurveIds(GetGameplayTrack().GetObjectId(), [this, &bFirstSeries, &AnimationProvider](uint32 InCurveId)
		{
			if(!AllSeries.ContainsByPredicate([InCurveId](const TSharedPtr<FGraphSeries>& InSeries){ return StaticCastSharedPtr<FSkeletalMeshCurveSeries>(InSeries)->CurveId == InCurveId; }))
			{
				auto MakeCurveSeriesColor = [](uint32 InSeed, bool bInLine)
				{
					FRandomStream Stream(InSeed);
					const uint8 Hue = (uint8)(Stream.FRand() * 255.0f);
					const uint8 SatVal = bInLine ? 196 : 128;
					return FLinearColor::MakeFromHSV8(Hue, SatVal, SatVal);
				};

				TSharedRef<FSkeletalMeshCurveSeries> Series = MakeShared<FSkeletalMeshCurveSeries>();

				const TCHAR* CurveName = AnimationProvider->GetName(InCurveId);

				Series->SetName(CurveName);
				Series->SetDescription(FText::Format(LOCTEXT("CurveNameFormat", "Values for curve '{0}'"), FText::FromString(CurveName)));

				const FLinearColor LineColor = MakeCurveSeriesColor(InCurveId, true);
				const FLinearColor FillColor = MakeCurveSeriesColor(InCurveId, false);
				Series->SetColor(LineColor, LineColor, FillColor);

				Series->CurveId = InCurveId;
				Series->SetVisibility(bFirstSeries);
				Series->SetBaselineY(25.0f);
				Series->SetScaleY(20.0f);
				Series->EnableAutoZoom();
				AllSeries.Add(Series);	

				bFirstSeries = false;
			}
		});
	}
}

bool FSkeletalMeshCurvesTrack::UpdateSeriesBounds(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport)
{
	bool bFoundEvents = false;

	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if(AnimationProvider)
	{
		FSkeletalMeshCurveSeries& CurveSeries = *static_cast<FSkeletalMeshCurveSeries*>(&InSeries);

		CurveSeries.CurrentMin = 0.0f;
		CurveSeries.CurrentMax = 0.0f;

		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadSkeletalMeshPoseTimeline(GetGameplayTrack().GetObjectId(), [&bFoundEvents, &InViewport, &CurveSeries, &AnimationProvider](const FAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bInHasCurves)
		{
			InTimeline.EnumerateEvents(InViewport.GetStartTime(), InViewport.GetEndTime(), [&bFoundEvents, &CurveSeries, &AnimationProvider](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InMessage)
			{
				AnimationProvider->EnumerateSkeletalMeshCurves(InMessage, [&bFoundEvents, &CurveSeries](const FSkeletalMeshNamedCurve& InCurve)
				{
					if(InCurve.Id == CurveSeries.CurveId)
					{
						CurveSeries.CurrentMin = FMath::Min(CurveSeries.CurrentMin, InCurve.Value);
						CurveSeries.CurrentMax = FMath::Max(CurveSeries.CurrentMax, InCurve.Value);
						bFoundEvents = true;
					}
				});
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}

	return bFoundEvents;
}

void FSkeletalMeshCurvesTrack::UpdateSeries(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport)
{
	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if(AnimationProvider)
	{
		FSkeletalMeshCurveSeries& CurveSeries = *static_cast<FSkeletalMeshCurveSeries*>(&InSeries);

		FGraphTrackBuilder Builder(*this, CurveSeries, InViewport);

		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
		AnimationProvider->ReadSkeletalMeshPoseTimeline(GetGameplayTrack().GetObjectId(), [this, &Builder, &InViewport, &CurveSeries, &AnimationProvider](const FAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bInHasCurves)
		{
			int32 FrameCounter = 0;
			int32 LastFrameWithCurve = 0;

			InTimeline.EnumerateEvents(InViewport.GetStartTime(), InViewport.GetEndTime(), [this, &FrameCounter, &LastFrameWithCurve, &Builder, &CurveSeries, &AnimationProvider](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InMessage)
			{
				FrameCounter++;

				AnimationProvider->EnumerateSkeletalMeshCurves(InMessage, [this, &FrameCounter, &LastFrameWithCurve, &Builder, &CurveSeries, &InStartTime, &InEndTime](const FSkeletalMeshNamedCurve& InCurve)
				{
					if(InCurve.Id == CurveSeries.CurveId)
					{
						Builder.AddEvent(InStartTime, InEndTime - InStartTime, InCurve.Value, LastFrameWithCurve == FrameCounter - 1);

						LastFrameWithCurve = FrameCounter;
					}
				});
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}

void FSkeletalMeshCurvesTrack::InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const
{
	const FGraphTrackEvent& GraphTrackEvent = *static_cast<const FGraphTrackEvent*>(&HoveredTimingEvent);

	FTimingEventSearchParameters SearchParameters(HoveredTimingEvent.GetStartTime(), HoveredTimingEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch);

	FindSkeletalMeshPoseMessage(SearchParameters, [this, &Tooltip, &GraphTrackEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FSkeletalMeshPoseMessage& InMessage)
	{
		Tooltip.ResetContent();

		Tooltip.AddTitle(GetName());

		const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
		if(AnimationProvider)
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);
			const TCHAR* CurveName = AnimationProvider->GetName(StaticCastSharedRef<const FSkeletalMeshCurveSeries>(GraphTrackEvent.GetSeries())->CurveId);

			Tooltip.AddNameValueTextLine(LOCTEXT("CurveName", "Curve").ToString(), CurveName);
		}

		Tooltip.AddNameValueTextLine(LOCTEXT("EventTime", "Time").ToString(), FText::AsNumber(InFoundStartTime).ToString());
		Tooltip.AddNameValueTextLine(LOCTEXT("EventValue", "Value").ToString(), FText::AsNumber(GraphTrackEvent.GetValue()).ToString());

		Tooltip.UpdateLayout();
	});
}

const TSharedPtr<const ITimingEvent> FSkeletalMeshCurvesTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	FindSkeletalMeshPoseMessage(InSearchParameters, [this, &FoundEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FSkeletalMeshPoseMessage& InFoundMessage)
	{
		FoundEvent = MakeShared<const FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth);
	});

	return FoundEvent;
}

void FSkeletalMeshCurvesTrack::FindSkeletalMeshPoseMessage(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FSkeletalMeshPoseMessage&)> InFoundPredicate) const
{
	TTimingEventSearch<FSkeletalMeshPoseMessage>::Search(
		InParameters,

		[this](TTimingEventSearch<FSkeletalMeshPoseMessage>::FContext& InContext)
		{
			const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

			if(AnimationProvider)
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);

				AnimationProvider->ReadSkeletalMeshPoseTimeline(GetGameplayTrack().GetObjectId(), [&InContext](const FAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bInHasCurves)
				{
					InTimeline.EnumerateEvents(InContext.GetParameters().StartTime, InContext.GetParameters().EndTime, [&InContext](double InEventStartTime, double InEventEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InMessage)
					{
						InContext.Check(InEventStartTime, InEventEndTime, 0, InMessage);
						return TraceServices::EEventEnumerate::Continue;
					});
				});
			}
		},

		[&InFoundPredicate](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FSkeletalMeshPoseMessage& InEvent)
		{
			InFoundPredicate(InFoundStartTime, InFoundEndTime, InFoundDepth, InEvent);
		});
}

void FSkeletalMeshCurvesTrack::GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const
{
	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
	if (AnimationProvider)
	{
		TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);

		TSharedRef<FVariantTreeNode> Header = OutVariants.Add_GetRef(FVariantTreeNode::MakeHeader(FText::FromString(GetName()), 0));

		AnimationProvider->ReadSkeletalMeshPoseTimeline(GetGameplayTrack().GetObjectId(), [&Header, &AnimationProvider, &InFrame](const FAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bInHasCurves)
		{
			InTimeline.EnumerateEvents(InFrame.StartTime, InFrame.EndTime, [&Header, &AnimationProvider, &InFrame](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InMessage)
			{
				if(InStartTime >= InFrame.StartTime && InStartTime <= InFrame.EndTime)
				{
					AnimationProvider->EnumerateSkeletalMeshCurves(InMessage, [&Header, &AnimationProvider](const FSkeletalMeshNamedCurve& InCurve)
					{
						const TCHAR* CurveName = AnimationProvider->GetName(InCurve.Id);
						Header->AddChild(FVariantTreeNode::MakeFloat(FText::FromString(CurveName), InCurve.Value));
					});
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}

#undef LOCTEXT_NAMESPACE
