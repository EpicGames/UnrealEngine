// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayGraphTrack.h"

class FAnimationSharedData;
class FTimingEventSearchParameters;
struct FSkeletalMeshPoseMessage;
class FSkeletalMeshCurvesTrack;

class FSkeletalMeshCurveSeries : public FGameplayGraphSeries
{
public:
	uint32 CurveId;
};

class FSkeletalMeshCurvesTrack : public FGameplayGraphTrack
{
	INSIGHTS_DECLARE_RTTI(FSkeletalMeshCurvesTrack, FGameplayGraphTrack)

public:
	FSkeletalMeshCurvesTrack(const FAnimationSharedData& InSharedData, uint64 InObjectID, const TCHAR* InName);

	virtual void InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const override;
	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;
	virtual void AddAllSeries() override;
	virtual bool UpdateSeriesBounds(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport) override;
	virtual void UpdateSeries(FGameplayGraphSeries& InSeries, const FTimingTrackViewport& InViewport) override;
	virtual void GetVariantsAtFrame(const TraceServices::FFrame& InFrame, TArray<TSharedRef<FVariantTreeNode>>& OutVariants) const override;

private:
	// Helper function used to find a skeletal mesh pose
	void FindSkeletalMeshPoseMessage(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FSkeletalMeshPoseMessage&)> InFoundPredicate) const;

private:
	/** The shared data */
	const FAnimationSharedData& SharedData;
};
