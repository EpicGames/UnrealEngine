// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshPoseTrack.h"
#include "AssetRegistry/AssetData.h"
#include "Common/ProviderLock.h"
#include "GameplayProvider.h"
#include "AnimationProvider.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "AnimationSharedData.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Modules/ModuleManager.h"

#if WITH_ENGINE
#include "Engine/SkeletalMesh.h"
#include "UObject/SoftObjectPtr.h"
#include "InsightsSkeletalMeshComponent.h"
#include "GameFramework/Actor.h"
#endif

#define LOCTEXT_NAMESPACE "SkeletalMeshPoseTrack"

INSIGHTS_IMPLEMENT_RTTI(FSkeletalMeshPoseTrack)

FSkeletalMeshPoseTrack::FSkeletalMeshPoseTrack(const FAnimationSharedData& InSharedData, uint64 InObjectID, const TCHAR* InName)
	: FGameplayTimingEventsTrack(InSharedData.GetGameplaySharedData(), InObjectID, FText::Format(LOCTEXT("TrackNameFormat", "Pose - {0}"), FText::FromString(FString(InName))))
	, SharedData(InSharedData)
	, Color(FLinearColor::MakeRandomColor())
	, bDrawPose(false)
	, bDrawSkeleton(false)
	, bPotentiallyDebugged(false)
{
#if WITH_ENGINE
	OnWorldCleanupHandle = FWorldDelegates::OnWorldCleanup.AddRaw(this, &FSkeletalMeshPoseTrack::OnWorldCleanup);
	OnWorldBeginTearDownHandle = FWorldDelegates::OnWorldBeginTearDown.AddRaw(this, &FSkeletalMeshPoseTrack::RemoveWorld);
	OnPreWorldFinishDestroyHandle = FWorldDelegates::OnPreWorldFinishDestroy.AddRaw(this, &FSkeletalMeshPoseTrack::RemoveWorld);
#endif
}

FSkeletalMeshPoseTrack::~FSkeletalMeshPoseTrack()
{
#if WITH_ENGINE
	FWorldDelegates::OnPreWorldFinishDestroy.Remove(OnPreWorldFinishDestroyHandle);
	FWorldDelegates::OnWorldBeginTearDown.Remove(OnWorldBeginTearDownHandle);
	FWorldDelegates::OnWorldCleanup.Remove(OnWorldCleanupHandle);

	for(auto& WorldCacheEntry : WorldCache)
	{
		if(WorldCacheEntry.Value.Component)
		{
			WorldCacheEntry.Value.Component->UnregisterComponent();
			WorldCacheEntry.Value.Component->MarkAsGarbage();
			WorldCacheEntry.Value.Component = nullptr;
		}

		if(WorldCacheEntry.Value.Actor)
		{
			WorldCacheEntry.Value.Actor->Destroy();
			WorldCacheEntry.Value.Actor = nullptr;
		}
	}

	WorldCache.Empty();
#endif
}

void FSkeletalMeshPoseTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

	if(AnimationProvider)
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);

		AnimationProvider->ReadSkeletalMeshPoseTimeline(GetGameplayTrack().GetObjectId(), [&Context, &Builder](const FAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bInHasCurves)
		{
			InTimeline.EnumerateEvents(Context.GetViewport().GetStartTime(), Context.GetViewport().GetEndTime(), [&Builder](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InMessage)
			{
				Builder.AddEvent(InStartTime, InEndTime, 0, InMessage.MeshName);
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}
}

void FSkeletalMeshPoseTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	DrawEvents(Context);
	GetGameplayTrack().DrawHeaderForTimingTrack(Context, *this, false);
}

void FSkeletalMeshPoseTrack::InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const
{
	FTimingEventSearchParameters SearchParameters(HoveredTimingEvent.GetStartTime(), HoveredTimingEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch);

	FindSkeletalMeshPoseMessage(SearchParameters, [this, &Tooltip](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FSkeletalMeshPoseMessage& InMessage)
	{
		Tooltip.ResetContent();

		Tooltip.AddTitle(LOCTEXT("SkeletalMeshPoseTooltipTitle", "Skeletal Mesh Pose").ToString());

		Tooltip.AddNameValueTextLine(LOCTEXT("EventTime", "Time").ToString(), FText::AsNumber(InFoundStartTime).ToString());

		FString PathName;
		{
			const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			const FObjectInfo* SkeletalMeshObjectInfo = GameplayProvider->FindObjectInfo(InMessage.MeshId);
			if (SkeletalMeshObjectInfo)
			{
				PathName = SkeletalMeshObjectInfo->PathName;
			}
		}

		if (!PathName.IsEmpty())
		{
			Tooltip.AddNameValueTextLine(LOCTEXT("Mesh", "Mesh").ToString(), PathName);

			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			if(!AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(PathName)).IsValid())
			{
				Tooltip.AddTextLine(LOCTEXT("MeshNotFound", "Mesh not found").ToString(), FLinearColor::Red);
			}
		}

		Tooltip.AddNameValueTextLine(LOCTEXT("BoneCount", "Bone Count").ToString(), FText::AsNumber(InMessage.NumTransforms).ToString());
		Tooltip.AddNameValueTextLine(LOCTEXT("CurveCount", "Curve Count").ToString(), FText::AsNumber(InMessage.NumCurves).ToString());
		Tooltip.AddNameValueTextLine(LOCTEXT("LODIndex", "LOD").ToString(), FText::AsNumber(InMessage.LodIndex).ToString());
		Tooltip.AddNameValueTextLine(LOCTEXT("EventWorld", "World").ToString(), GetGameplayTrack().GetWorldName(SharedData.GetAnalysisSession()).ToString());

		Tooltip.UpdateLayout();
	});
}

const TSharedPtr<const ITimingEvent> FSkeletalMeshPoseTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	FindSkeletalMeshPoseMessage(InSearchParameters, [this, &FoundEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FSkeletalMeshPoseMessage& InFoundMessage)
	{
		FoundEvent = MakeShared<const FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth);
	});

	return FoundEvent;
}

void FSkeletalMeshPoseTrack::FindSkeletalMeshPoseMessage(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FSkeletalMeshPoseMessage&)> InFoundPredicate) const
{
	TTimingEventSearch<FSkeletalMeshPoseMessage>::Search(
		InParameters,

		[this](TTimingEventSearch<FSkeletalMeshPoseMessage>::FContext& InContext)
		{
			const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);

			if(AnimationProvider)
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);

				AnimationProvider->ReadSkeletalMeshPoseTimeline(GetGameplayTrack().GetObjectId(), [&InContext](const FAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bInHasCurves)
				{
					InTimeline.EnumerateEvents(InContext.GetParameters().StartTime, InContext.GetParameters().EndTime, [&InContext](double InEventStartTime, double InEventEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InMessage)
					{
						InContext.Check(InEventStartTime, InEventEndTime, 0, InMessage);
						return TraceServices::EEventEnumerate::Continue;
					});
				});
			}
		},

		[&InFoundPredicate](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FSkeletalMeshPoseMessage& InEvent)
		{
			InFoundPredicate(InFoundStartTime, InFoundEndTime, InFoundDepth, InEvent);
		});
}

void FSkeletalMeshPoseTrack::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
#if WITH_ENGINE
	MenuBuilder.BeginSection(TEXT("DrawingSection"), LOCTEXT("Drawing", "Drawing (Component)"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleDrawPose", "Draw Pose"),
			LOCTEXT("ToggleDrawPose_Tooltip", "Draw the poses in this track"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					bDrawPose = !bDrawPose;
					UpdateComponentVisibility();
					SharedData.InvalidateViewports();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this](){ return bDrawPose; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleDrawSkeleton", "Draw Skeleton"),
			LOCTEXT("ToggleDrawSkeleton_Tooltip", "Draw the skeleton for poses in this track (when pose drawing is also enabled)"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{ 
					bDrawSkeleton = !bDrawSkeleton; 
					SharedData.InvalidateViewports();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this](){ return bDrawSkeleton; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);

	if(GameplayProvider)
	{
		FString ActorName;
		RewindDebugger::FObjectId ActorObjectId;
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

			const FObjectInfo* ComponentObjectInfo = GameplayProvider->FindObjectInfo(GetGameplayTrack().GetAssociatedObjectId());
			if(ComponentObjectInfo != nullptr)
			{
				// @FIXME: Outer does always equal owning actor, although does in nearly all cases with skeletal mesh components
				const FObjectInfo* ActorObjectInfo = GameplayProvider->FindObjectInfo(ComponentObjectInfo->GetOuterId());
				if(ActorObjectInfo != nullptr)
				{
					ActorName = ActorObjectInfo->Name;
					ActorObjectId = ActorObjectInfo->GetId();
				}
			}
		}

		if(ActorObjectId.IsSet())
		{
			{
				MenuBuilder.BeginSection(TEXT("DrawingSection"), FText::Format(LOCTEXT("DrawingActor", "Drawing ({0})"), FText::FromString(ActorName)));
				{
					MenuBuilder.AddMenuEntry(
						LOCTEXT("ToggleDrawPoseActor", "Draw Pose for Actor"),
						LOCTEXT("ToggleDrawPoseActor_Tooltip", "Draw the poses in this track and all other tracks for the current actor"),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([this, ActorObjectId, GameplayProvider]()
							{
								TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

								bool bSetDrawPose = true;
								SharedData.EnumerateSkeletalMeshPoseTracks([GameplayProvider, ActorObjectId, &bSetDrawPose](const TSharedRef<FSkeletalMeshPoseTrack>& InTrack)
								{
									const FObjectInfo* OtherComponentObjectInfo = GameplayProvider->FindObjectInfo(InTrack->GetGameplayTrack().GetAssociatedObjectId());
									if(OtherComponentObjectInfo && OtherComponentObjectInfo->GetOuterId() == ActorObjectId)
									{
										bSetDrawPose &= InTrack->bDrawPose;
									}
								});

								SharedData.EnumerateSkeletalMeshPoseTracks([GameplayProvider, ActorObjectId, &bSetDrawPose](const TSharedRef<FSkeletalMeshPoseTrack>& InTrack)
								{
									const FObjectInfo* OtherComponentObjectInfo = GameplayProvider->FindObjectInfo(InTrack->GetGameplayTrack().GetAssociatedObjectId());
									if(OtherComponentObjectInfo && OtherComponentObjectInfo->GetOuterId() == ActorObjectId)
									{
										InTrack->bDrawPose = !bSetDrawPose;
										InTrack->UpdateComponentVisibility();
									}
								});

								SharedData.InvalidateViewports();
							}),
							FCanExecuteAction(),
							FIsActionChecked::CreateLambda([this, ActorObjectId, GameplayProvider]()
							{
								TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

								bool bDrawPoseSet = true;
								SharedData.EnumerateSkeletalMeshPoseTracks([GameplayProvider, ActorObjectId, &bDrawPoseSet](const TSharedRef<FSkeletalMeshPoseTrack>& InTrack)
								{
									const FObjectInfo* OtherComponentObjectInfo = GameplayProvider->FindObjectInfo(InTrack->GetGameplayTrack().GetAssociatedObjectId());
									if(OtherComponentObjectInfo && OtherComponentObjectInfo->GetOuterId() == ActorObjectId)
									{
										bDrawPoseSet &= InTrack->bDrawPose;
									}
								});

								return bDrawPoseSet;
							})),
						NAME_None,
						EUserInterfaceActionType::ToggleButton
					);

					MenuBuilder.AddMenuEntry(
						LOCTEXT("ToggleDrawSkeletonActor", "Draw Skeleton for Actor"),
						LOCTEXT("ToggleDrawSkeletonActor_Tooltip", "Draw the skeleton for poses in this track and all other tracks for the current actor (when pose drawing is also enabled)"),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([this, ActorObjectId, GameplayProvider]()
							{
								TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

								bool bSetDrawSkeleton = true;
								SharedData.EnumerateSkeletalMeshPoseTracks([GameplayProvider, ActorObjectId, &bSetDrawSkeleton](const TSharedRef<FSkeletalMeshPoseTrack>& InTrack)
								{
									const FObjectInfo* OtherComponentObjectInfo = GameplayProvider->FindObjectInfo(InTrack->GetGameplayTrack().GetAssociatedObjectId());
									if(OtherComponentObjectInfo && OtherComponentObjectInfo->GetOuterId() == ActorObjectId)
									{
										bSetDrawSkeleton &= InTrack->bDrawSkeleton;
									}
								});

								SharedData.EnumerateSkeletalMeshPoseTracks([GameplayProvider, ActorObjectId, &bSetDrawSkeleton](const TSharedRef<FSkeletalMeshPoseTrack>& InTrack)
								{
									const FObjectInfo* OtherComponentObjectInfo = GameplayProvider->FindObjectInfo(InTrack->GetGameplayTrack().GetAssociatedObjectId());
									if(OtherComponentObjectInfo && OtherComponentObjectInfo->GetOuterId() == ActorObjectId)
									{
										InTrack->bDrawSkeleton = !bSetDrawSkeleton;
										InTrack->UpdateComponentVisibility();
									}
								});

								SharedData.InvalidateViewports();
							}),
							FCanExecuteAction(),
							FIsActionChecked::CreateLambda([this, ActorObjectId, GameplayProvider]()
							{
								TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

								bool bDrawSkeletonSet = true;
								SharedData.EnumerateSkeletalMeshPoseTracks([GameplayProvider, ActorObjectId, &bDrawSkeletonSet](const TSharedRef<FSkeletalMeshPoseTrack>& InTrack)
								{
									const FObjectInfo* OtherComponentObjectInfo = GameplayProvider->FindObjectInfo(InTrack->GetGameplayTrack().GetAssociatedObjectId());
									if(OtherComponentObjectInfo && OtherComponentObjectInfo->GetOuterId() == ActorObjectId)
									{
										bDrawSkeletonSet &= InTrack->bDrawSkeleton;
									}
								});

								return bDrawSkeletonSet;
							})),
						NAME_None,
						EUserInterfaceActionType::ToggleButton
					);
				}
				MenuBuilder.EndSection();
			}
		}
	}
#endif
}

#if WITH_ENGINE

void FSkeletalMeshPoseTrack::OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources)
{
	RemoveWorld(InWorld);
}

void FSkeletalMeshPoseTrack::RemoveWorld(UWorld* InWorld)
{
	FWorldComponentCache& CacheForWorld = GetWorldCache(InWorld);

	if(CacheForWorld.Component)
	{
		CacheForWorld.Component->UnregisterComponent();
		CacheForWorld.Component->MarkAsGarbage();
		CacheForWorld.Component = nullptr;
	}

	WorldCache.Remove(TWeakObjectPtr<UWorld>(InWorld));
}

USkeletalMeshComponent* FSkeletalMeshPoseTrack::GetComponent(UWorld* InWorld)
{
	if(InWorld)
	{
		return GetWorldCache(InWorld).GetComponent();
	}

	return nullptr;
}

void FSkeletalMeshPoseTrack::DrawPoses(UWorld* InWorld, double InTime, double InFrameStartTime, double InFrameEndTime)
{
	if(SharedData.IsAnalysisSessionValid())
	{
		const FAnimationProvider* AnimationProvider = SharedData.GetAnalysisSession().ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
		const FGameplayProvider* GameplayProvider = SharedData.GetAnalysisSession().ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);

		if(AnimationProvider && GameplayProvider)
		{
			FWorldComponentCache& CacheForWorld = GetWorldCache(InWorld);
			if(CacheForWorld.Component)
			{
				CacheForWorld.Component->SetVisibility(false);
			}

			TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);
			TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
			AnimationProvider->ReadSkeletalMeshPoseTimeline(GetGameplayTrack().GetObjectId(), [this, &CacheForWorld, &AnimationProvider, &GameplayProvider, &InFrameStartTime, &InFrameEndTime](const FAnimationProvider::SkeletalMeshPoseTimeline& InTimeline, bool bInHasCurves)
			{
				InTimeline.EnumerateEvents(InFrameStartTime, InFrameEndTime, [this, &CacheForWorld, &AnimationProvider, &GameplayProvider, &InFrameStartTime, &InFrameEndTime](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InMessage)
				{
					if(InStartTime >= InFrameStartTime && InStartTime <= InFrameEndTime)
					{
						const FSkeletalMeshInfo* SkeletalMeshInfo = AnimationProvider->FindSkeletalMeshInfo(InMessage.MeshId);
						const FObjectInfo* SkeletalMeshObjectInfo = GameplayProvider->FindObjectInfo(InMessage.MeshId);
						if(SkeletalMeshInfo && SkeletalMeshObjectInfo)
						{
							UInsightsSkeletalMeshComponent* Component = CacheForWorld.GetComponent();
							Component->SetVisibility(bDrawPose);

							if(CacheForWorld.Time != InFrameStartTime)
							{
								USkeletalMesh* SkeletalMesh = TSoftObjectPtr<USkeletalMesh>(FSoftObjectPath(SkeletalMeshObjectInfo->PathName)).LoadSynchronous();
								if(SkeletalMesh)
								{
									Component->SetSkeletalMesh(SkeletalMesh);
								}

								Component->SetPoseFromProvider(*AnimationProvider, InMessage, *SkeletalMeshInfo);

								CacheForWorld.Time = InFrameStartTime;
							}

							Component->SetDrawDebugSkeleton(bDrawSkeleton);
							Component->SetDebugDrawColor(Color);
						}
					}
					return TraceServices::EEventEnumerate::Continue;
				});
			});
		}
	}
}

FSkeletalMeshPoseTrack::FWorldComponentCache& FSkeletalMeshPoseTrack::GetWorldCache(UWorld* InWorld)
{
	FWorldComponentCache& Cache = WorldCache.FindOrAdd(TWeakObjectPtr<UWorld>(InWorld));
	Cache.World = InWorld;
	return Cache;
}

UInsightsSkeletalMeshComponent* FSkeletalMeshPoseTrack::FWorldComponentCache::GetComponent()
{
	if(Actor == nullptr)
	{
		FActorSpawnParameters ActorSpawnParameters;
		ActorSpawnParameters.bHideFromSceneOutliner = true;
		ActorSpawnParameters.ObjectFlags |= RF_Transient;

		Actor = World->SpawnActor<AActor>(ActorSpawnParameters);
		Actor->SetActorLabel(TEXT("Insights"));

		Time = 0.0;
	}

	if(Component == nullptr)
	{
		Component = NewObject<UInsightsSkeletalMeshComponent>(Actor);
		Component->PrimaryComponentTick.bStartWithTickEnabled = false;
		Component->PrimaryComponentTick.bCanEverTick = false;

		Actor->AddInstanceComponent(Component);

		Component->SetAnimationMode(EAnimationMode::AnimationCustomMode);
		Component->RegisterComponentWithWorld(World);

		Time = 0.0;
	}
	
	return Component;
}

void FSkeletalMeshPoseTrack::AddReferencedObjects(FReferenceCollector& Collector)
{
	for(auto& WorldCacheEntry : WorldCache)
	{
		Collector.AddReferencedObject(WorldCacheEntry.Value.Actor);
		Collector.AddReferencedObject(WorldCacheEntry.Value.Component);
	}
}

void FSkeletalMeshPoseTrack::UpdateComponentVisibility()
{
	for(auto& WorldCacheEntry : WorldCache)
	{
		if(WorldCacheEntry.Value.Component)
		{
			WorldCacheEntry.Value.Component->SetVisibility(bDrawPose);
		}
	}
}

#endif

#undef LOCTEXT_NAMESPACE
