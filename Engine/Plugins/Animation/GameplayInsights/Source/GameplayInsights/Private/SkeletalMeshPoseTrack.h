// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTrack.h"
#include "Insights/ViewModels/TimingEventsTrack.h"

#if WITH_ENGINE
#include "UObject/GCObject.h"
#include "Engine/World.h"
#endif

#if WITH_ENGINE
class USkeletalMeshComponent;
class UInsightsSkeletalMeshComponent;
class AActor;
#endif

class FAnimationSharedData;
class FTimingEventSearchParameters;
struct FSkeletalMeshPoseMessage;

class FSkeletalMeshPoseTrack : public FGameplayTimingEventsTrack
#if WITH_ENGINE
	, public FGCObject
#endif
{
	INSIGHTS_DECLARE_RTTI(FSkeletalMeshPoseTrack, FGameplayTimingEventsTrack)

public:
	FSkeletalMeshPoseTrack(const FAnimationSharedData& InSharedData, uint64 InObjectID, const TCHAR* InName);
	~FSkeletalMeshPoseTrack();

	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void Draw(const ITimingTrackDrawContext& Context) const override;
	virtual void InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const override;
	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;

	// Access drawing flags
	void SetDrawPose(bool bInDrawPose) { bDrawPose = bInDrawPose; }
	bool ShouldDrawPose() const { return bDrawPose; }
	bool ShouldDrawSkeleton() const { return bDrawSkeleton; }

	// Mark this track as potentially being debugged, so we might need to push through its transforms when time changes
	void MarkPotentiallyDebugged() { bPotentiallyDebugged = true; }
	bool IsPotentiallyDebugged() const { return bPotentiallyDebugged; }

#if WITH_ENGINE
	// Get the component for the specified world
	USkeletalMeshComponent* GetComponent(UWorld* InWorld);

	// Handle worlds being torn down
	void OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources);
	void RemoveWorld(UWorld* InWorld);

	// Draw poses at the specified time
	void DrawPoses(UWorld* InWorld, double InTime, double InFrameStartTime, double InFrameEndTime);

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("InsightsSkeletalMeshPoseTrack"); }
#endif

private:
	// Helper function used to find a skeletal mesh pose
	void FindSkeletalMeshPoseMessage(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FSkeletalMeshPoseMessage&)> InFoundPredicate) const;

#if WITH_ENGINE
	// Updates component visibility based on draw pose flag
	void UpdateComponentVisibility();
#endif
private:
	/** The shared data */
	const FAnimationSharedData& SharedData;

	/** The color to use to draw this track */
	FLinearColor Color;

	/** Whether to draw the pose */
	bool bDrawPose;

	/** Whether to draw the skeleton */
	bool bDrawSkeleton;

	/** Whether we might be being debugged */
	bool bPotentiallyDebugged;

#if WITH_ENGINE
	/** Cached data per-world */
	struct FWorldComponentCache
	{
		FWorldComponentCache()
			: World(nullptr)
			, Actor(nullptr)
			, Component(nullptr)
			, Time(0.0)
		{}

		/** Get a cached component for this world */
		UInsightsSkeletalMeshComponent* GetComponent();

		/** The world we populate */
		UWorld* World;

		/** Cached actor used to hang the component off of */
		TObjectPtr<AActor> Actor;

		/** Cached component used to visualize in this world */
		TObjectPtr<UInsightsSkeletalMeshComponent> Component;

		/** The time we last cached on this component */
		double Time;
	};

	// Get the cached data for a world
	FWorldComponentCache& GetWorldCache(UWorld* InWorld);

	/** Cached map of per-world data */
	TMap<TWeakObjectPtr<UWorld>, FWorldComponentCache> WorldCache;

	/** Handles used to deal with world switching */
	FDelegateHandle OnWorldCleanupHandle;
	FDelegateHandle OnWorldBeginTearDownHandle;
	FDelegateHandle OnPreWorldFinishDestroyHandle;
#endif
};