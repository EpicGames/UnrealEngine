// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariantTreeNode.h"
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "GameplayProvider.h"
#include "AnimationProvider.h"

#define LOCTEXT_NAMESPACE "FVariantTreeNode"

FString FVariantTreeNode::GetValueAsString(const TraceServices::IAnalysisSession& InAnalysisSession) const
{
	switch(Value.Type)
	{
	case EAnimNodeValueType::Bool:
		return (Value.Bool.bValue ? LOCTEXT("true", "true") : LOCTEXT("false", "false")).ToString();
	case EAnimNodeValueType::Int32:
		return FText::AsNumber(Value.Int32.Value).ToString();
	case EAnimNodeValueType::Float:
		return FText::AsNumber(Value.Float.Value).ToString();
	case EAnimNodeValueType::Vector2D:
		return FText::Format(LOCTEXT("Vector2DFormat", "{0}, {1}"), FText::AsNumber(Value.Vector2D.Value.X), FText::AsNumber(Value.Vector2D.Value.Y)).ToString();
	case EAnimNodeValueType::Vector:
		return FText::Format(LOCTEXT("VectorFormat", "{0}, {1}, {2}"), FText::AsNumber(Value.Vector.Value.X), FText::AsNumber(Value.Vector.Value.Y), FText::AsNumber(Value.Vector.Value.Z)).ToString();
	case EAnimNodeValueType::String:
		return Value.String.Value;
	case EAnimNodeValueType::Object:
		{
			const FGameplayProvider* GameplayProvider = InAnalysisSession.ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
			if(GameplayProvider)
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

				const FObjectInfo& ObjectInfo = GameplayProvider->GetObjectInfo(Value.Object.Value);
				return ObjectInfo.PathName;
			}

			break;
		}
	case EAnimNodeValueType::Class:
		{
			const FGameplayProvider* GameplayProvider = InAnalysisSession.ReadProvider<FGameplayProvider>(FGameplayProvider::ProviderName);
			if(GameplayProvider)
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

				const FClassInfo& ClassInfo = GameplayProvider->GetClassInfo(Value.Class.Value);
				return ClassInfo.PathName;
			}

			break;
		}
	case EAnimNodeValueType::AnimNode:
		{
			const FAnimationProvider* AnimationProvider = InAnalysisSession.ReadProvider<FAnimationProvider>(FAnimationProvider::ProviderName);
			if (AnimationProvider)
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*AnimationProvider);

				const FAnimNodeInfo* AnimNodeInfo = AnimationProvider->FindAnimNodeInfo(Value.AnimNode.Value, Value.AnimNode.AnimInstanceId);
				return AnimNodeInfo ? AnimNodeInfo->Name : TEXT("");
			}

			break;
		}
	}

	return FString();
}

#undef LOCTEXT_NAMESPACE
