// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAnimationProvider.h"
#include "IGameplayProvider.h"

namespace TraceServices { class IAnalysisSession; }

enum class EVariantTreeNodeFilterState
{
	Hidden,
	Visible,
	Highlighted,
};

/** A wrapper around a variant value, used to display collections of values in a tree */
struct FVariantTreeNode : TSharedFromThis<FVariantTreeNode>
{
	FVariantTreeNode(const FText& InName, const FVariantValue& InValue, uint64 InId = INDEX_NONE)
		: Name(InName)
		, Value(InValue)
		, PropertyNameId(0)
		, Id(InId)
	{}

	FVariantTreeNode(const FText& InName, const FVariantValue& InValue, uint32 InPropertyNamedId = 0, uint64 InId = INDEX_NONE)
		: Name(InName)
		, Value(InValue)
		, PropertyNameId(InPropertyNamedId)
		, Id(InId)
	{}
	
	const TSharedRef<FVariantTreeNode>& AddChild(const TSharedRef<FVariantTreeNode>& InChild)
	{
		check(!InChild->Parent.IsValid());
		InChild->Parent = SharedThis(this);
		return Children.Add_GetRef(InChild);
	}

	FText GetName() const { return Name; }

	const FVariantValue& GetValue() const { return Value; }

	FVariantValue& GetValue() { return Value; }

	TSharedPtr<FVariantTreeNode> GetParent() const { return Parent.Pin(); }

	const TArray<TSharedRef<FVariantTreeNode>>& GetChildren() const { return Children; }

	uint64 GetId() const { return Id; }

	uint32 GetPropertyNamedId() const { return PropertyNameId; }
	
	void SetPropertyNameId(uint32 InPropertyNameId) { PropertyNameId = InPropertyNameId; }
	
	void SetFilterState(EVariantTreeNodeFilterState InFilterState) { FilterState = InFilterState; }

	EVariantTreeNodeFilterState GetFilterState() const { return FilterState; }

	FString GetValueAsString(const TraceServices::IAnalysisSession& InAnalysisSession) const;

	static TSharedRef<FVariantTreeNode> MakeHeader(const FText& InName, uint64 InId)
	{
		FVariantValue Value;
		Value.Type = EAnimNodeValueType::String;
		Value.String.Value = TEXT("");

		return MakeShared<FVariantTreeNode>(InName, Value, InId);
	}

	static TSharedRef<FVariantTreeNode> MakeBool(const FText& InName, bool InValue, uint64 InId = INDEX_NONE)
	{
		FVariantValue Value;
		Value.Type = EAnimNodeValueType::Bool;
		Value.Bool.bValue = InValue;

		return MakeShared<FVariantTreeNode>(InName, Value, InId);
	}

	static TSharedRef<FVariantTreeNode> MakeInt32(const FText& InName, int32 InValue, uint64 InId = INDEX_NONE)
	{
		FVariantValue Value;
		Value.Type = EAnimNodeValueType::Int32;
		Value.Int32.Value = InValue;

		return MakeShared<FVariantTreeNode>(InName, Value, InId);
	}

	static TSharedRef<FVariantTreeNode> MakeFloat(const FText& InName, float InValue, uint64 InId = INDEX_NONE)
	{
		FVariantValue Value;
		Value.Type = EAnimNodeValueType::Float;
		Value.Float.Value = InValue;

		return MakeShared<FVariantTreeNode>(InName, Value, InId);
	}

	static TSharedRef<FVariantTreeNode> MakeVector2D(const FText& InName, const FVector2D& InValue, uint64 InId = INDEX_NONE)
	{
		FVariantValue Value;
		Value.Type = EAnimNodeValueType::Vector2D;
		Value.Vector2D.Value = InValue;

		return MakeShared<FVariantTreeNode>(InName, Value, InId);
	}


	static TSharedRef<FVariantTreeNode> MakeVector(const FText& InName, const FVector& InValue, uint64 InId = INDEX_NONE)
	{
		FVariantValue Value;
		Value.Type = EAnimNodeValueType::Vector;
		Value.Vector.Value = InValue;

		return MakeShared<FVariantTreeNode>(InName, Value, InId);
	}

	static TSharedRef<FVariantTreeNode> MakeString(const FText& InName, const TCHAR* InValue, uint64 InId = INDEX_NONE)
	{
		FVariantValue Value;
		Value.Type = EAnimNodeValueType::String;
		Value.String.Value = InValue;

		return MakeShared<FVariantTreeNode>(InName, Value, InId);
	}

	static TSharedRef<FVariantTreeNode> MakeObject(const FText& InName, uint64 InValue, uint64 InId = INDEX_NONE, float PlaybackTime = 0.0f, float BlendX = 0.0f, float BlendY = 0.0f)
	{
		FVariantValue Value;
		Value.Type = EAnimNodeValueType::Object;
		Value.Object.Value = InValue;
		Value.Object.PlaybackTime = PlaybackTime;
		Value.Object.BlendX = BlendX;
		Value.Object.BlendY = BlendY;

		return MakeShared<FVariantTreeNode>(InName, Value, InId);
	}

	static TSharedRef<FVariantTreeNode> MakeClass(const FText& InName, uint64 InValue, uint64 InId = INDEX_NONE)
	{
		FVariantValue Value;
		Value.Type = EAnimNodeValueType::Class;
		Value.Class.Value = InValue;

		return MakeShared<FVariantTreeNode>(InName, Value, InId);
	}

	static TSharedRef<FVariantTreeNode> MakeAnimNode(const FText& InName, uint64 InValue, uint64 InAnimInstanceId, uint64 InId = INDEX_NONE)
	{
		FVariantValue Value;
		Value.Type = EAnimNodeValueType::AnimNode;
		Value.AnimNode.Value = InValue;
		Value.AnimNode.AnimInstanceId = InAnimInstanceId;

		return MakeShared<FVariantTreeNode>(InName, Value, InId);
	}

private:
	FText Name;
	FVariantValue Value;
	TWeakPtr<FVariantTreeNode> Parent;
	TArray<TSharedRef<FVariantTreeNode>> Children;
	uint32 PropertyNameId;
	
	// Unique ID used to record expansion state
	uint64 Id;

	// Filter state given current filter text
	EVariantTreeNodeFilterState FilterState;
};
