// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Containers/Timelines.h"

struct FAnimTickRecordTrace;

struct FSkeletalMeshInfo
{
	uint64 ParentIndicesStartIndex = 0;
	uint64 Id = 0;
	uint64 SkeletonId = 0;
	uint32 BoneCount = 0;
};

struct FAnimNodeInfo
{
	int32 Id = 0;
	uint64 AnimInstanceId = 0;
	const TCHAR* Name = nullptr;
	const TCHAR* TypeName = nullptr;
};

struct FSkeletalMeshNamedCurve
{
	uint32 Id = 0;
	float Value = 0.0f;
};

struct FExternalMorphWeightMessage
{
	int32 Index = 0;
	int32 NumMorphs = 0;
	TArray<float> Weights;
};

struct FSkeletalMeshPoseMessage
{
	FTransform ComponentToWorld;
	double RecordingTime = 0.0;
	uint64 TransformStartIndex = 0;
	uint64 CurveStartIndex = 0;
	uint64 ComponentId = 0;	
	uint64 MeshId = 0;
	uint64 ExternalMorphStartIndex = 0;
	const TCHAR* MeshName = nullptr;
	uint16 NumTransforms = 0;
	uint16 NumCurves = 0;
	uint16 FrameCounter = 0;
	uint16 LodIndex = 0;
	uint16 NumExternalMorphSets = 0;
	bool bIsVisible = true;
};

struct FPoseWatchMessage
{
	FTransform WorldTransform;
	FColor Color = FColor::Black;
	double RecordingTime = 0.0;
	uint64 ComponentId = 0;
	uint64 AnimInstanceId = 0;
	uint64 PoseWatchId = 0;
	uint64 BoneTransformsStartIndex = 0;
	uint64 CurveStartIndex = 0;
	uint64 RequiredBonesStartIndex = 0;
	uint32 NameId = 0;
	uint16 NumBoneTransforms = 0;
	uint16 NumRequiredBones = 0;
	uint16 NumCurves = 0;
	bool bIsEnabled = false;
};

struct FSkeletalMeshFrameMessage
{
	uint64 ComponentId = 0;
	uint16 FrameCounter = 0;
};

struct FTickRecordMessage
{
	uint64 ComponentId = 0;
	uint64 AnimInstanceId = 0;
	uint64 AssetId = 0;
	double RecordingTime = 0.0;
	int32 NodeId = -1;
	float BlendWeight = 0.0f;
	float PlaybackTime = 0.0f;
	float RootMotionWeight = 0.0f;
	float PlayRate = 0.0f;
	float BlendSpacePositionX = 0.0f;
	float BlendSpacePositionY = 0.0f;
	float BlendSpaceFilteredPositionX = 0.0f;
	float BlendSpaceFilteredPositionY = 0.0f;
	uint16 FrameCounter = 0;
	bool bLooping = false;
	bool bIsBlendSpace = false;
	uint32 bSyncGroupNameId = 0;
	bool bIsSyncGroupLeader = false;
	float SyncLeaderScore = 0.0f;
	float TimeToPrevMarker = 0.0f;
	float TimeToNextMarker = 0.0f;
	int32 PrevMarkerIndex = -1;
	int32 NextMarkerIndex = -1;
};

enum class EAnimGraphPhase : uint8
{
	Initialize = 0,
	PreUpdate = 1,
	Update = 2,
	CacheBones = 3,
	Evaluate = 4,
};

struct FAnimGraphMessage
{
	uint64 AnimInstanceId = 0;
	int32 NodeCount = 0;
	uint16 FrameCounter = 0;
	EAnimGraphPhase Phase = EAnimGraphPhase::Initialize;
};

struct FAnimNodeMessage
{
	const TCHAR* NodeName = nullptr;
	uint64 AnimInstanceId = 0;
	int32 PreviousNodeId = -1;
	int32 NodeId = -1;
	float Weight = 0.0f;
	float RootMotionWeight = 0.0f;
	uint16 FrameCounter = 0;
	EAnimGraphPhase Phase = EAnimGraphPhase::Initialize;
	const TCHAR* NodeTypeName = nullptr;
};

enum class EAnimNodeValueType : uint8
{
	Bool,
	Int32,
	Float,
	Vector2D,
	Vector,
	String,
	Object,
	Class,
	AnimNode,
};

struct FVariantValue
{
	FVariantValue()
		: Vector(FVector::ZeroVector)
	{
	}

	struct FVector2DEntry
	{
		FVector2DEntry(const FVector2D& InVector)
			: Value(InVector)
		{}

		FVector2D Value;
	};

	struct FVectorEntry
	{
		FVectorEntry(const FVector& InVector)
			: Value(InVector)
		{}

		FVector Value;
	};

	union
	{
		struct
		{
			bool bValue;
		} Bool;
		struct
		{
			int32 Value;
		} Int32;
		struct
		{
			float Value;
		} Float;
		FVector2DEntry Vector2D;
		FVectorEntry Vector;
		struct
		{
			const TCHAR* Value;
		} String;
		struct
		{
			uint64 Value;
			// for animation assets, store the playback time so we can open the editor to a specific time
			float PlaybackTime;
			// for blend spaces, store the x/y parameters so we can open the editor at those parameters
			float BlendX;
			float BlendY;
		} Object;
		struct
		{
			uint64 Value;
		} Class;
		struct
		{
			int32 Value;
			uint64 AnimInstanceId;
		} AnimNode;
	};

	EAnimNodeValueType Type = EAnimNodeValueType::Bool;
};

struct FAnimNodeValueMessage
{
	const TCHAR* Key = nullptr;
	uint64 AnimInstanceId = 0;
	FVariantValue Value;
	int32 NodeId = -1;
	uint16 FrameCounter = 0;
	double RecordingTime = 0.0f;
};

struct FAnimSequencePlayerMessage
{
	uint64 AnimInstanceId = 0;
	int32 NodeId = -1;
	float Position = 0.0f;
	float Length = 0.0f;
	uint16 FrameCounter = 0;
};

struct FBlendSpacePlayerMessage
{
	uint64 AnimInstanceId = 0;
	uint64 BlendSpaceId = 0;
	int32 NodeId = -1;
	float PositionX = 0.0f;
	float PositionY = 0.0f;
	float PositionZ = 0.0f;
	float FilteredPositionX = 0.0f;
	float FilteredPositionY = 0.0f;
	float FilteredPositionZ = 0.0f;
};

struct FAnimStateMachineMessage
{
	uint64 AnimInstanceId = 0;
	int32 NodeId = -1;
	int32 StateMachineIndex = -1;
	int32 StateIndex = -1;
	float StateWeight = 0.0f;
	float ElapsedTime = 0.0f;
};

enum class EAnimNotifyMessageType : uint8
{
	Event = 0,
	Begin = 1,
	End = 2,
	Tick = 3,
	SyncMarker = 4	// We 'fake' sync markers with a notify type for convenience
};

struct FAnimNotifyMessage
{
	uint64 AnimInstanceId = 0;
	uint64 AssetId = 0;
	uint64 NotifyId = 0;
	const TCHAR* Name = nullptr;
	uint32 NameId = 0;
	double RecordingTime = 0.0f;
	float Time = 0.0f; 
	float Duration = 0.0f;
	EAnimNotifyMessageType NotifyEventType = EAnimNotifyMessageType::Event;
};

struct FAnimMontageMessage
{
	uint64 AnimInstanceId = 0;
	uint64 MontageId = 0;
	double RecordingTime = 0.0;
	uint32 CurrentSectionNameId = 0;
	uint32 NextSectionNameId = 0;
	float Weight = 0.0f;
	float DesiredWeight = 0.0f;
	float Position = 0.0f;
	uint16 FrameCounter = 0;
};

enum class EInertializationType : uint8
{
	Inertialization = 0,
	DeadBlending = 1
};

struct FInertializationMessage
{
	uint64 AnimInstanceId = 0;
	double ProfileTime;
	double RecordingTime;
	int32 NodeId;
	float Weight;
	EInertializationType Type;
};

struct FAnimAttributeMessage
{
	int32 SourceNodeId = 0;
	int32 TargetNodeId = 0;
	uint32 AttributeNameId = 0;
};

struct FAnimSyncMessage
{
	int32 SourceNodeId = 0;
	uint32 GroupNameId = 0;
};

class IAnimationProvider : public TraceServices::IProvider
{
public:
	typedef TraceServices::ITimeline<FTickRecordMessage> TickRecordTimeline;
	typedef TraceServices::ITimeline<FSkeletalMeshPoseMessage> SkeletalMeshPoseTimeline;
	typedef TraceServices::ITimeline<FAnimGraphMessage> AnimGraphTimeline;
	typedef TraceServices::ITimeline<FAnimNodeMessage> AnimNodesTimeline;
	typedef TraceServices::ITimeline<FAnimNodeValueMessage> AnimNodeValuesTimeline;
	typedef TraceServices::ITimeline<FAnimSequencePlayerMessage> AnimSequencePlayersTimeline;
	typedef TraceServices::ITimeline<FAnimStateMachineMessage> StateMachinesTimeline;
	typedef TraceServices::ITimeline<FBlendSpacePlayerMessage> BlendSpacePlayersTimeline;
	typedef TraceServices::ITimeline<FAnimNotifyMessage> AnimNotifyTimeline;
	typedef TraceServices::ITimeline<FAnimMontageMessage> AnimMontageTimeline;
	typedef TraceServices::ITimeline<FAnimAttributeMessage> AnimAttributeTimeline;
	typedef TraceServices::ITimeline<FAnimSyncMessage> AnimSyncTimeline;
	typedef TraceServices::ITimeline<FPoseWatchMessage> PoseWatchTimeline;
	typedef TraceServices::ITimeline<FInertializationMessage> InertializationTimeline;

	virtual void EnumerateSkeletalMeshPoseTimelines(TFunctionRef<void(uint64 ObjectId, const SkeletalMeshPoseTimeline&)> Callback) const =0;
	virtual bool ReadSkeletalMeshPoseTimeline(uint64 InObjectId, TFunctionRef<void(const SkeletalMeshPoseTimeline&, bool)> Callback) const = 0;
	virtual void GetSkeletalMeshComponentSpacePose(const FSkeletalMeshPoseMessage& InMessage, const FSkeletalMeshInfo& InMeshInfo, FTransform& OutComponentToWorld, TArray<FTransform>& OutTransforms) const = 0;
	virtual void EnumerateSkeletalMeshCurveIds(uint64 InObjectId, TFunctionRef<void(uint32)> Callback) const = 0;
	virtual void GetPoseWatchData(const FPoseWatchMessage& InMessage, TArray<FTransform>& BoneTransforms, TArray<uint16>& RequiredBones) const = 0;
	virtual void EnumerateSkeletalMeshCurves(const FSkeletalMeshPoseMessage& InMessage, TFunctionRef<void(const FSkeletalMeshNamedCurve&)> Callback) const = 0;
	virtual void EnumeratePoseWatchCurves(const FPoseWatchMessage& InMessage, TFunctionRef<void(const FSkeletalMeshNamedCurve&)> Callback) const = 0;
	virtual void EnumerateExternalMorphSets(const FSkeletalMeshPoseMessage& InMessage, TFunctionRef<void(const FExternalMorphWeightMessage&)> Callback) const = 0;
	virtual bool ReadTickRecordTimeline(uint64 InObjectId, TFunctionRef<void(const TickRecordTimeline&)> Callback) const = 0;
	virtual bool ReadInertializationTimeline(uint64 InObjectId, TFunctionRef<void(const InertializationTimeline&)> Callback) const = 0;
	virtual void EnumerateInertializationNodes(uint64 InObjectId, TFunctionRef<void(int32, EInertializationType)> Callback) const = 0;
	virtual void EnumerateTickRecordIds(uint64 InObjectId, TFunctionRef<void(uint64, int32)> Callback) const = 0;
	virtual void EnumerateAnimGraphTimelines(TFunctionRef<void(uint64 ObjectId, const AnimGraphTimeline&)> Callback) const =0;
	virtual bool ReadAnimGraphTimeline(uint64 InObjectId, TFunctionRef<void(const AnimGraphTimeline&)> Callback) const = 0;
	virtual bool ReadAnimNodesTimeline(uint64 InObjectId, TFunctionRef<void(const AnimNodesTimeline&)> Callback) const = 0;
	virtual bool ReadAnimNodeValuesTimeline(uint64 InObjectId, TFunctionRef<void(const AnimNodeValuesTimeline&)> Callback) const = 0;
	virtual bool ReadAnimAttributesTimeline(uint64 InObjectId, TFunctionRef<void(const AnimAttributeTimeline&)> Callback) const = 0;
	virtual bool ReadAnimSequencePlayersTimeline(uint64 InObjectId, TFunctionRef<void(const AnimSequencePlayersTimeline&)> Callback) const = 0;
	virtual bool ReadAnimBlendSpacePlayersTimeline(uint64 InObjectId, TFunctionRef<void(const BlendSpacePlayersTimeline&)> Callback) const = 0;
	virtual bool ReadStateMachinesTimeline(uint64 InObjectId, TFunctionRef<void(const StateMachinesTimeline&)> Callback) const = 0;
	virtual bool ReadNotifyTimeline(uint64 InObjectId, TFunctionRef<void(const AnimNotifyTimeline&)> Callback) const = 0;
	virtual void EnumerateNotifyStateTimelines(uint64 InObjectId, TFunctionRef<void(uint64, const AnimNotifyTimeline&)> Callback) const = 0;
	virtual bool ReadMontageTimeline(uint64 InObjectId, TFunctionRef<void(const AnimMontageTimeline&)> Callback) const = 0;
	virtual void EnumerateMontageIds(uint64 InObjectId, TFunctionRef<void(uint64)> Callback) const = 0;
	virtual bool ReadAnimSyncTimeline(uint64 InObjectId, TFunctionRef<void(const AnimSyncTimeline&)> Callback) const = 0;
	virtual bool ReadPoseWatchTimeline(uint64 InObjectId, TFunctionRef<void(const PoseWatchTimeline&)> Callback) const = 0;
	virtual const FSkeletalMeshInfo* FindSkeletalMeshInfo(uint64 InObjectId) const = 0;
	virtual const FAnimNodeInfo* FindAnimNodeInfo(int32 InNodeId, uint64 InAnimInstanceId) const = 0;
	virtual const TCHAR* GetName(uint32 InId) const = 0;
	virtual FText FormatNodeKeyValue(const FAnimNodeValueMessage& InMessage) const = 0;
	virtual FText FormatNodeValue(const FAnimNodeValueMessage& InMessage) const = 0;
};
