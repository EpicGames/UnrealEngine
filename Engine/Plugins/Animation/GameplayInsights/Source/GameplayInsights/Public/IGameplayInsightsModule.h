// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "UObject/NameTypes.h"

namespace GameplayInsightsTabs
{
	extern GAMEPLAYINSIGHTS_API const FName DocumentTab;
};

namespace TraceServices
{
	class IAnalysisSession;
}

class IGameplayInsightsModule
	: public IModuleInterface
{
public:

#if WITH_EDITOR
	// start or stop tracing all properties of this Object to Insights
	virtual void EnableObjectPropertyTrace(UObject* Object, bool bEnable = true) = 0;
	// check if an Object has it's properties tracing to Insights
	virtual bool IsObjectPropertyTraceEnabled(UObject* Object) = 0;
	// start tracing data
	virtual void StartTrace() = 0;
#endif
};
