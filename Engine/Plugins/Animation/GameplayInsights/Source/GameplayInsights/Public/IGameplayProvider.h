// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Math/Vector.h"
#include "Math/Rotator.h"
#include "Math/Range.h" // for deprecation only
#include "RewindDebuggerTypes.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Containers/Timelines.h"

struct FClassInfo
{
	uint64 Id = 0;
	uint64 SuperId = 0;
	const TCHAR* Name = nullptr;
	const TCHAR* PathName = nullptr;
};

enum class EObjectInfoFlags : uint8
{
	None = 0,
	TransientObject = 1 << 0,
	StructInstance = 1 << 1,
};

struct FObjectInfo
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FObjectInfo(const FObjectInfo& Other) = default;
	FObjectInfo(FObjectInfo&& Other) = default;
	FObjectInfo& operator=(const FObjectInfo& Other) = default;
	FObjectInfo& operator=(FObjectInfo&& Other) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FObjectInfo(const RewindDebugger::FObjectId& ObjectId, const RewindDebugger::FObjectId& OuterObjectId, const uint64 ClassId, const TCHAR* Name
				, const TCHAR* PathName, EObjectInfoFlags Flags)
		: ObjectId(ObjectId)
		, OuterObjectId(OuterObjectId)
		, ClassId(ClassId)
		, Name(Name)
		, PathName(PathName)
		, Flags(Flags)
	{
	}

	/** @return Id of an object recorded in the trace */
	const RewindDebugger::FObjectId& GetId() const
	{
		return ObjectId;
	}

	/** @return Id of the recorded object's parent */
	const RewindDebugger::FObjectId& GetOuterId() const
	{
		return OuterObjectId;
	}

	/** @return Part of the Id representing the UObject */
	uint64 GetUObjectId() const
	{
		return ObjectId.GetMainId();
	}

	/** @return Part of the Id of the parent object representing the UObject */
	uint64 GetOuterUObjectId() const
	{
		return OuterObjectId.GetMainId();
	}

private:
	RewindDebugger::FObjectId ObjectId;
	RewindDebugger::FObjectId OuterObjectId;

public:
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.7, "Use ObjectId instead")
	uint64 Id = 0;
	UE_DEPRECATED(5.7, "Use OuterObjectId instead")
	uint64 OuterId = 0;
#endif

	uint64 ClassId = 0;
	const TCHAR* Name = nullptr;
	const TCHAR* PathName = nullptr;
	EObjectInfoFlags Flags;
	
};

struct FObjectPropertiesMessage
{
	int64 PropertyValueStartIndex = INDEX_NONE;	// Inclusive
	int64 PropertyValueEndIndex = INDEX_NONE;	// Exclusive
	double ProfileTime;
	double ElapsedTime;
};

struct FObjectPropertyValue
{
	const TCHAR* Value = nullptr;
	int32 ParentId = 0;
	uint32 TypeStringId = 0;
	uint32 NameId = 0;
	uint32 ParentNameId = 0;
	float ValueAsFloat = 0.0f;
};

struct FObjectEventMessage
{
	RewindDebugger::FObjectId ObjectId;
	const TCHAR* Name = nullptr;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FObjectEventMessage() = default;
	FObjectEventMessage(const FObjectEventMessage& Other) = default;
	FObjectEventMessage(FObjectEventMessage&& Other) = default;
	FObjectEventMessage& operator=(const FObjectEventMessage& Other) = default;
	FObjectEventMessage& operator=(FObjectEventMessage&& Other) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.7, "Use Identifier instead")
	uint64 Id = 0;
#endif
};

struct FRecordingInfoMessage
{
	uint64 WorldId;
	double ProfileTime;
	double ElapsedTime;
	uint32 FrameIndex;
	uint32 RecordingIndex;
};

struct FViewMessage
{
	uint64 PlayerId;
	FVector Position;
	FRotator Rotation;
	float Fov;
	float AspectRatio;
};

struct FWorldInfo
{
	/** Types of worlds that we know about - synced with EngineTypes.h */
	enum class EType : uint8
	{
		None,
		Game,
		Editor,
		PIE,
		EditorPreview,
		GamePreview,
		GameRPC,
		Inactive
	};

	/** Types of net modes that we know about - synced with EngineBaseTypes.h */
	enum class ENetMode : uint8
	{
		Standalone,
		DedicatedServer,
		ListenServer,
		Client,

		MAX,
	};

	uint64 Id = 0;
	int32 PIEInstanceId = 0;
	EType Type = EType::None;
	ENetMode NetMode = ENetMode::Standalone;
	bool bIsSimulating = false;
};

struct FSlateIcon;

// Delegate fired when an object receives an end play event
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnObjectEndPlay, uint64 /*ObjectId*/, double /*Time*/, const FObjectInfo& /*ObjectInfo*/);

class IGameplayProvider : public TraceServices::IProvider, public TraceServices::IEditableProvider
{
public:
	typedef TraceServices::ITimeline<FObjectEventMessage> ObjectEventsTimeline;
	typedef TraceServices::ITimeline<FObjectPropertiesMessage> ObjectPropertiesTimeline;
	typedef TraceServices::ITimeline<FRecordingInfoMessage> RecordingInfoTimeline;
	typedef TraceServices::ITimeline<FViewMessage> ViewTimeline;

	virtual bool ReadObjectEventsTimeline(const RewindDebugger::FObjectId& InObjectId, TFunctionRef<void(const ObjectEventsTimeline&)> Callback) const = 0;
	virtual bool ReadObjectEvent(const RewindDebugger::FObjectId& InObjectId, uint64 InMessageId, TFunctionRef<void(const FObjectEventMessage&)> Callback) const = 0;
	virtual bool ReadObjectPropertiesTimeline(uint64 InObjectId, TFunctionRef<void(const ObjectPropertiesTimeline&)> Callback) const = 0;
	virtual bool ReadObjectPropertiesStorage(uint64 InObjectId, const FObjectPropertiesMessage& InMessage, TFunctionRef<void(const TConstArrayView<FObjectPropertyValue> &)> Callback) const = 0;
	virtual void EnumerateObjectPropertyValues(uint64 InObjectId, const FObjectPropertiesMessage& InMessage, TFunctionRef<void(const FObjectPropertyValue&)> Callback) const = 0;
	virtual void EnumerateObjects(TFunctionRef<void(const FObjectInfo&)> Callback) const = 0;
	virtual void EnumerateObjects(double StartTime, double EndTime, TFunctionRef<void(const FObjectInfo&)> Callback) const = 0;
	virtual void EnumerateWorlds(TFunctionRef<void(const FWorldInfo&)> Callback) const = 0;
	virtual void EnumerateSubobjects(const RewindDebugger::FObjectId& InObjectId, TFunctionRef<void(const RewindDebugger::FObjectId& SubObjectId)> Callback) const = 0;
	virtual const FObjectPropertyValue * FindPropertyValueFromStorageIndex(uint64 InObjectId, int64 InStorageIndex) const = 0;
	virtual const FClassInfo* FindClassInfo(uint64 InClassId) const = 0;
	virtual const UStruct* FindType(uint64 InTypeId, bool bSearchBlueprints) const = 0;
	UE_DEPRECATED(5.8, "Use FindType returning a UStruct instead.")
	virtual const UClass* FindClass(const uint64 InClassId, const bool bSearchBlueprints) const final
	{
		return Cast<UClass>(FindType(InClassId, bSearchBlueprints));
	}
	virtual const FClassInfo* FindClassInfo(const TCHAR* InClassPath) const = 0;
	virtual bool IsSubClassOf(uint64 InSubClassId, uint64 InParentClassId) const = 0;
	virtual const FObjectInfo* FindObjectInfo(const uint64 InObjectId) const = 0;
	virtual const FObjectInfo* FindObjectInfo(const RewindDebugger::FObjectId& InObjectId) const = 0;
	virtual const FWorldInfo* FindWorldInfo(uint64 InObjectId) const = 0;
	virtual const FWorldInfo* FindWorldInfoFromObject(uint64 InObjectId) const = 0;
	virtual bool IsWorld(uint64 InObjectId) const = 0;
	virtual const FClassInfo& GetClassInfo(uint64 InClassId) const = 0;
	virtual const FClassInfo& GetClassInfoFromObject(uint64 InObjectId) const = 0;
	virtual const FObjectInfo& GetObjectInfo(const uint64 InObjectId) const = 0;
	virtual const FObjectInfo& GetObjectInfo(const RewindDebugger::FObjectId& InObjectId) const = 0;
	virtual FOnObjectEndPlay& OnObjectEndPlay() = 0;
	virtual const TCHAR* GetPropertyName(uint32 InPropertyStringId) const = 0;
	virtual const RecordingInfoTimeline* GetRecordingInfo(uint32 RecordingId) const = 0; 
	virtual void ReadViewTimeline(TFunctionRef<void(const ViewTimeline&)> Callback) const = 0;
	virtual uint64 FindPossessingController(uint64 Pawn, double Time) const = 0;
	virtual TRange<double> GetObjectTraceLifetime(const RewindDebugger::FObjectId& InObjectId) const = 0;
	virtual TRange<double> GetObjectRecordingLifetime(const RewindDebugger::FObjectId& InObjectId) const = 0;
	virtual double GetRecordingDuration() const = 0;
	virtual FSlateIcon FindIconForClass(uint64 ClassId) const = 0;
	virtual bool GetObjectTransform(uint64 InObjectId, double InStartTime, double InEndTime, FTransform& OutTransform) const = 0;

	UE_DEPRECATED(5.7, "Use the version taking FObjectId instead")
	virtual bool ReadObjectEventsTimeline(uint64 InObjectId, TFunctionRef<void(const ObjectEventsTimeline&)> InCallback) const final
	{
		return ReadObjectEventsTimeline(RewindDebugger::FObjectId(InObjectId), InCallback);
	}

	UE_DEPRECATED(5.7, "Use the version taking FObjectId instead")
	virtual bool ReadObjectEvent(uint64 InObjectId, uint64 InMessageId, TFunctionRef<void(const FObjectEventMessage&)> InCallback) const final
	{
		return ReadObjectEvent(RewindDebugger::FObjectId(InObjectId), InMessageId, InCallback);
	}

	UE_DEPRECATED(5.7, "Use the version taking FObjectId instead")
	virtual void EnumerateSubobjects(uint64 InObjectId, TFunctionRef<void(uint64 SubobjectId)> InCallback) const final
	{
	}

	UE_DEPRECATED(5.7, "Use the version taking FObjectId instead")
	virtual TRange<double> GetObjectTraceLifetime(uint64 InObjectId) const final
	{
		return GetObjectTraceLifetime(RewindDebugger::FObjectId(InObjectId));
	}

	UE_DEPRECATED(5.7, "Use the version taking FObjectId instead")
	virtual TRange<double> GetObjectRecordingLifetime(uint64 InObjectId) const final
	{
		return GetObjectRecordingLifetime(RewindDebugger::FObjectId(InObjectId));
	}
};

inline const FObjectInfo* IGameplayProvider::FindObjectInfo(const uint64 InObjectId) const
{
	return FindObjectInfo(RewindDebugger::FObjectId(InObjectId));
}

inline const FObjectInfo& IGameplayProvider::GetObjectInfo(const uint64 InObjectId) const
{
	return GetObjectInfo(RewindDebugger::FObjectId(InObjectId));
}
