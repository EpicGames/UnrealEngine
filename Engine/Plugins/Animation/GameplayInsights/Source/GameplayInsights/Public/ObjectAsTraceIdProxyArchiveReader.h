// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_ENGINE
#include "ObjectAsTraceIdProxyArchive.h"

class UObject;

/**
 * Subclass of FObjectAsTraceIdProxyArchive which is also able to read object references as trace ids and resolve them
 *
 * Objects will only resolve if they are found by name based on the traced name data for the object - this will work for Assets and Classes
 * and may in some cases work for runtime UObject references if those UObjects still exist (eg during a rewind debugger recording in PIE)
 * 
 * @param InInnerArchive The actual FArchive object to serialize normal data types (FStrings, INTs, etc)
 */

struct FObjectAsTraceIdProxyArchiveReader : public FObjectAsTraceIdProxyArchive
{
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InInnerArchive - The inner archive to proxy.
	 * @param bInLoadIfFindFails - Indicates whether to try and load a ref'd object if we don't find it
	 */
	GAMEPLAYINSIGHTS_API FObjectAsTraceIdProxyArchiveReader(FArchive& InInnerArchive, const class IGameplayProvider* InGameplayProvider);

	GAMEPLAYINSIGHTS_API virtual ~FObjectAsTraceIdProxyArchiveReader();

	virtual FArchive& operator<<(UObject*& Obj) override;
	virtual FArchive& operator<<(FWeakObjectPtr& Obj) override;
	virtual FArchive& operator<<(FSoftObjectPtr& Value) override;
	virtual FArchive& operator<<(FSoftObjectPath& Value) override;
	virtual FArchive& operator<<(FObjectPtr& Obj) override;

	const class IGameplayProvider* GameplayProvider;
};

#endif

