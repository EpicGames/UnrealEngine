// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RewindDebuggerTrack.h"

class FRewindDebuggerPlaceholderTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:
	FRewindDebuggerPlaceholderTrack(const FName& InObjectName, const FText& InDisplayName)
		: ObjectName(InObjectName), DisplayName(InDisplayName)
	{
	}

private:
	virtual FName GetNameInternal() const override { return ObjectName; }
	virtual FText GetDisplayNameInternal() const override { return DisplayName; }
	
	FName ObjectName;
	FText DisplayName;
};
