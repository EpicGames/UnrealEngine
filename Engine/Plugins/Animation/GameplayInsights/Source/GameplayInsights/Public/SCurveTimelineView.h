// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API GAMEPLAYINSIGHTS_API

class FPaintArgs;
class FSlateWindowElementList;
class FTimeSliderController;
class SToolTip;

class SCurveTimelineView : public SCompoundWidget
{
public:
	struct FTimelineCurveData
	{
	public:
		struct CurvePoint
		{
			double Time;
			float Value;
		};

		TArray<CurvePoint> Points;
	};
	
	SLATE_BEGIN_ARGS(SCurveTimelineView)
		: _ViewRange(TRange<double>(0,10))
		, _DesiredSize(FVector2D(100.f,20.f))
		, _RenderFill(false)
	{}
    	/** View time range */
     	SLATE_ATTRIBUTE(TRange<double>, ViewRange);

		/** Data for curve to render */
		SLATE_ATTRIBUTE(TSharedPtr<FTimelineCurveData>, CurveData);

		/** Desired widget size */
		SLATE_ATTRIBUTE(FVector2D, DesiredSize);
	
		/** Curve Color */
		SLATE_ATTRIBUTE(FLinearColor, CurveColor);
	
		/** Fill Color */
		SLATE_ATTRIBUTE(FLinearColor, FillColor);
	
		/** Enable Fill rendering */
		SLATE_ATTRIBUTE(bool, RenderFill);

		/** Track Name for tool tip */
		SLATE_ATTRIBUTE(FText, TrackName);
     
	SLATE_END_ARGS()


	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	UE_API void Construct( const FArguments& InArgs );

	// render the curve with a fixed min/max range, rather than using the min/max from data
	void SetFixedRange(float Min, float Max)
	{
		bUseFixedRange = true;
		FixedRangeMin = Min;
		FixedRangeMax = Max;
	}

	FText GetTrackName() const { return TrackName.Get(); }
	FText GetCurveToolTipInputText() const { return CurveToolTipInputText; }
	FText GetCurveToolTipOutputText() const { return CurveToolTipOutputText; }
	
protected:
	// SWidget interface
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	UE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	
	UE_API int32 PaintCurve(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const;
	UE_API void UpdateCurveToolTip(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	
	TAttribute<TRange<double>> ViewRange;
	TAttribute<TSharedPtr<FTimelineCurveData>> CurveData;
	TAttribute<FVector2D> DesiredSize;
	TAttribute<FLinearColor> CurveColor;
	TAttribute<FLinearColor> FillColor;
	TAttribute<bool> RenderFill;
	TAttribute<FText> TrackName;
	
	TSharedPtr<SToolTip> CurveToolTip;	/** The tooltip control for the curve. */
	FText CurveToolTipInputText;		
	FText CurveToolTipOutputText;
	
	const FSlateBrush* FillBrush;

	bool bUseFixedRange;
	float FixedRangeMin;
	float FixedRangeMax;
};

#undef UE_API
