// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GameplayInsightsEditor : ModuleRules
	{
		public GameplayInsightsEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			});

            PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"GameplayInsights"
			});

            if (Target.bCompileAgainstEngine)
			{
				PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"CoreUObject",
				});
			}
		}
	}
}

