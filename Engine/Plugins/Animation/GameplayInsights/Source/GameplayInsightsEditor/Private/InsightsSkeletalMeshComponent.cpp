// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsSkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "IAnimationProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InsightsSkeletalMeshComponent)

void UInsightsSkeletalMeshComponent::SetPoseFromProvider(const IAnimationProvider& InProvider, const FSkeletalMeshPoseMessage& InMessage, const FSkeletalMeshInfo& SkeletalMeshInfo)
{
	// Grab transform and bone transforms from provider
	FTransform ComponentToWorldTransform;
	InProvider.GetSkeletalMeshComponentSpacePose(InMessage, SkeletalMeshInfo, ComponentToWorldTransform, GetEditableComponentSpaceTransforms());

	// Set transform
	SetRelativeTransform(ComponentToWorldTransform);

	// Force LOD to message LOD
	SetForcedLOD(InMessage.LodIndex + 1);

	// Flip buffers once to copy the directly-written component space transforms
	bNeedToFlipSpaceBaseBuffers = true;
	bHasValidBoneTransform = false;
	FlipEditableSpaceBases();
	bHasValidBoneTransform = true;

	InvalidateCachedBounds();
	UpdateBounds();
	MarkRenderTransformDirty();
	MarkRenderDynamicDataDirty();
}

void UInsightsSkeletalMeshComponent::InitAnim(bool bForceReInit)
{
	if(GetSkeletalMeshAsset())
	{
		const FReferenceSkeleton& SkeletalMeshRefSkeleton = GetSkeletalMeshAsset()->GetRefSkeleton();
		// set up bone visibility states as this gets skipped since we allocate the component array before registration
		for (int32 BaseIndex = 0; BaseIndex < 2; ++BaseIndex)
		{
			BoneVisibilityStates[BaseIndex].SetNum(SkeletalMeshRefSkeleton.GetNum());

			for (int32 BoneIndex = 0; BoneIndex < SkeletalMeshRefSkeleton.GetNum(); BoneIndex++)
			{
				BoneVisibilityStates[BaseIndex][BoneIndex] = BVS_Visible;
			}
		}

		MorphTargetWeights.SetNumZeroed(GetSkeletalMeshAsset()->GetMorphTargets().Num());
	}

	USkeletalMeshComponent::InitAnim(bForceReInit);
}
