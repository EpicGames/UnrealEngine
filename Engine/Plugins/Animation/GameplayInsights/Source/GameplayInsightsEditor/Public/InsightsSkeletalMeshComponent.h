// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SkeletalMeshComponent.h"
#include "InsightsSkeletalMeshComponent.generated.h"

#define UE_API GAMEPLAYINSIGHTSEDITOR_API

class IAnimationProvider;
struct FSkeletalMeshPoseMessage;
struct FSkeletalMeshInfo;

UCLASS(MinimalAPI, Hidden)
class UInsightsSkeletalMeshComponent : public USkeletalMeshComponent
{
	GENERATED_BODY()

public:
	// Set this component up from a provider & message
	UE_API void SetPoseFromProvider(const IAnimationProvider& InProvider, const FSkeletalMeshPoseMessage& InMessage, const FSkeletalMeshInfo& SkeletalMeshInfo);

	// USkeletalMeshComponent interface
	UE_API virtual void InitAnim(bool bForceReInit) override;
};

#undef UE_API
