// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimInstanceHelpers.h"
#include "Common/ProviderLock.h"
#include "Modules/ModuleManager.h"
#include "ObjectTrace.h"
#include "IGameplayProvider.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Editor.h"
#include "EdGraph/EdGraph.h"
#include "RewindDebuggerAnimation.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "IAnimationBlueprintEditor.h"
#include "RewindDebugger.h"
#include "RewindDebuggerModule.h"
#include "Insights/IUnrealInsightsModule.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "SRewindDebuggerAnimBPTools"

static bool OpenAnimBlueprintAndAttachDebugger(const TraceServices::IAnalysisSession* Session, uint64 ObjectId)
{
	if (const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider"))
	{
		// Cache the class path name under a tight lock scope
		FString ClassPathName;
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			if (const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(ObjectId))
			{
				if (const FClassInfo* ClassInfo = GameplayProvider->FindClassInfo(ObjectInfo->ClassId))
				{
					ClassPathName = ClassInfo->PathName;
				}
			}
		}

		// Perform asset loading and editor operations outside the lock
		if (!ClassPathName.IsEmpty())
		{
			if (UAnimBlueprintGeneratedClass* InstanceClass = TSoftObjectPtr<UAnimBlueprintGeneratedClass>(FSoftObjectPath(ClassPathName)).LoadSynchronous())
			{
				if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(InstanceClass->ClassGeneratedBy))
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AnimBlueprint);

					UObject* SelectedInstance = nullptr;
#if OBJECT_TRACE_ENABLED
					SelectedInstance = FObjectTrace::GetObjectFromId(ObjectId);
#endif
					if (SelectedInstance == nullptr)
					{
						if (FRewindDebuggerAnimation* RewindDebuggerAnimation = FRewindDebuggerAnimation::GetInstance())
						{
							SelectedInstance = RewindDebuggerAnimation->GetDebugAnimInstance(ObjectId);
						}
					}

					if(SelectedInstance)
					{
						AnimBlueprint->SetObjectBeingDebugged(SelectedInstance);
					}

					if(IAnimationBlueprintEditor* AnimBlueprintEditor = static_cast<IAnimationBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AnimBlueprint, true)))
					{
						// navigate the opened editor to the AnimGraph
						if (AnimBlueprint->FunctionGraphs.Num() > 0)
						{
							AnimBlueprintEditor->JumpToHyperlink(AnimBlueprint->FunctionGraphs[0]);
						}
					}

					return true;
				}
			}
		}
	}
	return false;
}

bool FAnimInstanceDoubleClickHandler::HandleDoubleClick(IRewindDebugger* RewindDebugger)
{
	TSharedPtr<FDebugObjectInfo> SelectedObject = RewindDebugger->GetSelectedObject();
	if (SelectedObject.IsValid())
	{
		return OpenAnimBlueprintAndAttachDebugger(RewindDebugger->GetAnalysisSession(), SelectedObject->GetUObjectId());
	}
	return false;
}

FName FAnimInstanceDoubleClickHandler::GetTargetTypeName() const
{
	static const FName ObjectTypeName = "AnimInstance";
	return ObjectTypeName;
}

void FAnimInstanceMenu::Register()
{
#if OBJECT_TRACE_ENABLED
	UToolMenu* Menu = UToolMenus::Get()->FindMenu(FRewindDebuggerModule::TrackContextMenuName);
	FToolMenuSection& Section = Menu->FindOrAddSection("Blueprint");
	Section.AddDynamicEntry("DebugAnimInstanceEntry", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		URewindDebuggerTrackContextMenuContext* Context = InSection.FindContext<URewindDebuggerTrackContextMenuContext>();
		if (Context && Context->SelectedObject.IsValid() && Context->TypeHierarchy.Contains("AnimInstance"))
		{
			InSection.AddMenuEntry(NAME_None,
						LOCTEXT("Open AnimBP", "Open/Debug AnimGraph"),
							LOCTEXT("Open AnimBP ToolTip", "Open this Animation Blueprint and attach the debugger to this instance"),
							FSlateIcon(),
							FExecuteAction::CreateLambda([ObjectId = Context->SelectedObject->GetUObjectId()]()
							{
								const TraceServices::IAnalysisSession* Session = FRewindDebugger::Instance() ? FRewindDebugger::Instance()->GetAnalysisSession() : nullptr;
								if (Session)
								{
									OpenAnimBlueprintAndAttachDebugger(Session, ObjectId);
								}
							}));
		}
	}));
#endif
}

#undef LOCTEXT_NAMESPACE
