// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IRewindDebuggerDoubleClickHandler.h"

class FAnimInstanceDoubleClickHandler : public IRewindDebuggerDoubleClickHandler
{
	virtual FName GetTargetTypeName() const override;
	bool HandleDoubleClick(IRewindDebugger* RewindDebugger) override;
};

class FAnimInstanceMenu
{
public:
	static void Register();
};