// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

// Event based properties for syncronizing widgets with Models
// In future, this should be replaced by slate ViewModels, and/or Slate Attributes once those have an OnChanged delegate
//
// Usage
//
// Widget Header:
// 	SLATE_BEGIN_ARGS(SMyWidget) { }
//		SLATE_ARGUMENT(TBindablePropertyInitializer<float>, MyProperty);
//	SLATE_END_ARGS()
//
//  TBindableProperty<float> MyProperty
//
// Widget Cpp:
// void SMyWidget::Construct(const FArguments& InArgs, ...
// {
//      MyProperty.Initialize(InArgs._MyProperty);
//
//      optionally:
//      MyProperty.OnPropertyChanged = TBindableProperty<float>::FOnPropertyChanged::Create...
//
//
// Outside the Widget:
//
// TBindableProperty<float> MyModelsProperty;
// ...
// SNew(SMyWidget,...)
//   .MyProperty(&MyModelsProperty);
//  or: .MyProperty(0.5f); 

enum EBindingType
{
    BindingType_In,    // default - for value to be copied in to widget on change
    BindingType_Out,   // value copied out of widget on change
    BindingType_TwoWay // value copied both ways on change
};

template <typename T, EBindingType BindingType>
class TBindableProperty;

template <typename T, EBindingType BindingType = BindingType_In>
struct TBindablePropertyInitializer
{
    TBindablePropertyInitializer()
    {
        Source = nullptr;
        Value = T();
    }

    TBindablePropertyInitializer(TBindableProperty<T, BindingType>* InSource)
    {
        Source = InSource;
        Value = T();
    }

	TBindablePropertyInitializer(TBindableProperty<T, BindingType>* InSource, const T& InitialValue)
    {
		Source = InSource;
		Value = InitialValue;
    }
    
    TBindablePropertyInitializer(const T& InValue)
    {
        Source = nullptr;
        Value = InValue;
    }

    TBindableProperty<T, BindingType>* Source;
    T Value;
};

template <typename T, EBindingType BindingType = BindingType_In>
class TBindableProperty
{
    public:
        DECLARE_DELEGATE_OneParam(FOnPropertyChanged, T NewValue);

        // delete copy/move constructors and equals operator because we are storing raw pointers to other properties, so they can't move around in memory
        // todo: handle move constructor correctly so that Bindable properties can be stored in arrays
        TBindableProperty(const TBindableProperty<T,BindingType>&) =delete;
        TBindableProperty(TBindableProperty<T,BindingType>&&) =delete;
        TBindableProperty& operator=(const TBindableProperty<T,BindingType>&) =delete;

        TBindableProperty()
            : InternalValue() // make sure POD types get zero initialized
        {
        }

        ~TBindableProperty()
        {
            ClearBinding();

            for(TBindableProperty<T, BindingType>* BoundProperty : BoundTo)
            {
                BoundProperty->BoundFrom = nullptr;
            }
        }

        bool IsBound()
        {
            return BoundFrom != nullptr;
        }

        void Set(const T& NewValue)
        {
            if (!IsBound() || BindingType == BindingType_TwoWay)
            {
                SetInternal(NewValue);
            }
        }

        const T& Get() const
        {
            return InternalValue;
        }

        void ClearBinding()
        {
            if (BoundFrom)
            {
                BoundFrom->RemoveBinding(this);
                BoundFrom = nullptr;
            }
        }

        void Bind(TBindableProperty<T, BindingType>* Source)
        {
            if (Source)
            {
                if (BindingType == BindingType_Out)
                {
                    ClearBinding();
                    AddBinding(Source);
                    Source->BoundFrom = this; 
                }
                else
                {
                    ClearBinding();
                    Source->AddBinding(this);
                    BoundFrom = Source; 
                }
            }
        }

        // either set an initial value, or bind to another property (for Out properties, maybe need to support both?)
        void Initialize(const TBindablePropertyInitializer<T, BindingType> &Initializer)
        {
            if (Initializer.Source)
            {
            	InternalValue = Initializer.Value;
                Bind(Initializer.Source);
            }
            else
            {
                Set(Initializer.Value);
            }
        }

        FOnPropertyChanged OnPropertyChanged;

    private:
        T InternalValue;

        TArray<TBindableProperty<T, BindingType>*> BoundTo;     // properties I will write to on write if BindingType is TwoWay or In
        TBindableProperty<T, BindingType>* BoundFrom = nullptr; // a Property that will write to me if BindingType is From, and I write to if BindingType is TwoWay or Out

        void AddBinding(TBindableProperty<T, BindingType>* Target)
        {
            BoundTo.Add(Target);            
            Target->SetInternal(InternalValue);
        }

        void RemoveBinding(TBindableProperty<T, BindingType>* Target)
        {
            BoundTo.Remove(Target);            
        }

        void SetInternal(const T& NewValue)
        {
            InternalValue = NewValue;
            OnPropertyChanged.ExecuteIfBound(InternalValue);

			for (TBindableProperty<T, BindingType>* BoundProperty : BoundTo)
			{
				BoundProperty->SetInternal(InternalValue);
			}

            if (BindingType == BindingType_TwoWay)
            {
				if (BoundFrom)
				{
					BoundFrom->SetInternal(InternalValue);
				}
            }
        }
};
