// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintDoubleClickHandler.h"

#include "IGameplayProvider.h"
#include "Common/ProviderLock.h"
#include "ObjectTrace.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"

#if OBJECT_TRACE_ENABLED

/** Open blueprint editor given an object id. If no editor found, it will use the properties editor. */
static bool OpenBlueprintAndAttachDebugger(const TraceServices::IAnalysisSession* Session, uint64 ObjectId)
{
	if (const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider"))
	{
		// Cache the class path name under a tight lock scope
		FString ClassPathName;
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			if (const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(ObjectId))
			{
				if (const FClassInfo* ClassInfo = GameplayProvider->FindClassInfo(ObjectInfo->ClassId))
				{
					ClassPathName = ClassInfo->PathName;
				}
			}
		}

		// Perform asset loading and editor operations outside the lock
		if (!ClassPathName.IsEmpty())
		{
			if (const UBlueprintGeneratedClass* InstanceClass = TSoftObjectPtr<UBlueprintGeneratedClass>(FSoftObjectPath(ClassPathName)).LoadSynchronous())
			{
				if (UBlueprint* Blueprint = Cast<UBlueprint>(InstanceClass->ClassGeneratedBy))
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint);

					if (UObject* SelectedInstance = FObjectTrace::GetObjectFromId(ObjectId))
					{
						Blueprint->SetObjectBeingDebugged(SelectedInstance);
					}

					return true;
				}
			}
		}
	}
	return false;
}

#endif

bool FBlueprintDoubleClickHandler::HandleDoubleClick(IRewindDebugger* RewindDebugger)
{
#if OBJECT_TRACE_ENABLED
	const TSharedPtr<FDebugObjectInfo> SelectedObject = RewindDebugger->GetSelectedObject();
	if (SelectedObject.IsValid())
	{
		return OpenBlueprintAndAttachDebugger(RewindDebugger->GetAnalysisSession(), SelectedObject->GetUObjectId());
	}
#endif
	
	return false;
}

FName FBlueprintDoubleClickHandler::GetTargetTypeName() const
{
	static const FName ObjectTypeName = "Object";
	return ObjectTypeName;
}