// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyTraceMenu.h"
#include "IGameplayInsightsModule.h"
#include "Modules/ModuleManager.h"
#include "ObjectTrace.h"
#include "RewindDebuggerModule.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "RewindDebuggerComponentHelpers"

void FPropertyTraceMenu::Register()
{
#if OBJECT_TRACE_ENABLED

	UToolMenu* Menu = UToolMenus::Get()->FindMenu(FRewindDebuggerModule::TrackContextMenuName);
	FToolMenuSection& Section = Menu->FindOrAddSection("Object");
	Section.AddDynamicEntry("AssetManagerEditorViewCommands", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		URewindDebuggerTrackContextMenuContext* Context = InSection.FindContext<URewindDebuggerTrackContextMenuContext>();
		if (Context && Context->SelectedObject.IsValid())
		{
			InSection.AddMenuEntry(
				NAME_None,
				LOCTEXT("Trace Object Properties", "Trace Object Properties"),
				LOCTEXT("Trace Object Properties Tooltip", "Record this object's properties so they will show in the Rewind Debugger when scrubbing."),
				FSlateIcon(), 
				FUIAction(FExecuteAction::CreateLambda([SelectedObject = Context->SelectedObject]()
					{
						IGameplayInsightsModule& GameplayInsightsModule = FModuleManager::GetModuleChecked<IGameplayInsightsModule>("GameplayInsights");
						if (SelectedObject.IsValid())
						{
							if (UObject* Object = FObjectTrace::GetObjectFromId(SelectedObject->GetUObjectId()))
							{
								GameplayInsightsModule.EnableObjectPropertyTrace(Object, !GameplayInsightsModule.IsObjectPropertyTraceEnabled(Object));
							}
						}
					}),
					FCanExecuteAction(),
					FGetActionCheckState::CreateLambda([SelectedObject = Context->SelectedObject]()
					{
						IGameplayInsightsModule& GameplayInsightsModule = FModuleManager::GetModuleChecked<IGameplayInsightsModule>("GameplayInsights");
						bool bEnabled = false;
	 
						if (SelectedObject.IsValid())
						{
							if (UObject* Object = FObjectTrace::GetObjectFromId(SelectedObject->GetUObjectId()))
							{
								bEnabled = GameplayInsightsModule.IsObjectPropertyTraceEnabled(Object);
							}
						}
						 return bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}
					),
					FIsActionButtonVisible()
				),
				EUserInterfaceActionType::Check
			);
		}
	}));
#endif
}

#undef LOCTEXT_NAMESPACE
