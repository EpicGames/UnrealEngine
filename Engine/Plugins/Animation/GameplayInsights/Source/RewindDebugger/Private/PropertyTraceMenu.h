// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IRewindDebuggerDoubleClickHandler.h"

class FPropertyTraceMenu
{
public:
	static void Register();
};