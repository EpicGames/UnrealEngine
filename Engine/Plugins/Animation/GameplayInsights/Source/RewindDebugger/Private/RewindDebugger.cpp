// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebugger.h"

#include "ActorPickerMode.h"
#include "Common/ProviderLock.h"
#include "DesktopPlatformModule.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "Editor/UnrealEdEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformFileManager.h"
#include "IDesktopPlatform.h"
#include "IPIEAuthorizer.h"
#include "IRewindDebuggerExtension.h"
#include "Kismet2/DebuggerCommands.h"
#include "LevelEditor.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "ObjectTrace.h"
#include "RewindDebuggerCommands.h"
#include "RewindDebuggerEngineEditorBridge.h"
#include "RewindDebuggerModule.h"
#include "RewindDebuggerObjectTrack.h"
#include "RewindDebuggerPlaceholderTrack.h"
#include "RewindDebuggerRecordingControls.h"
#include "RewindDebuggerRuntime/RewindDebuggerRuntime.h"
#include "RewindDebuggerSettings.h"
#include "RewindDebuggerStyle.h"
#include "RewindDebuggerTrackCreators.h"
#include "SessionInfo.h"
#include "ToolMenuDelegates.h"
#include "ToolMenus.h"
#include "TraceSessionsManager.h"
#include "UnrealEdGlobals.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "RewindDebugger"

namespace UE::RewindDebugger
{
constexpr uint32 NumDefaultDebugTracks = 2;

static void IterateExtensions(TFunction<void(IRewindDebuggerExtension* Extension)> IteratorFunction)
{
	// update extensions
	IModularFeatures& ModularFeatures = IModularFeatures::Get();

	const int32 NumExtensions = ModularFeatures.GetModularFeatureImplementationCount(IRewindDebuggerExtension::ModularFeatureName);
	for (int32 ExtensionIndex = 0; ExtensionIndex < NumExtensions; ++ExtensionIndex)
	{
		IRewindDebuggerExtension* Extension = static_cast<IRewindDebuggerExtension*>(ModularFeatures.GetModularFeatureImplementation(IRewindDebuggerExtension::ModularFeatureName, ExtensionIndex));
		IteratorFunction(Extension);
	}
}

static void TraceSubobjects(const UObject* OuterObject)
{
	TArray<UObject*> Subobjects;
	GetObjectsWithOuter(OuterObject, Subobjects, EGetObjectsFlags::IncludeNestedObjects);
	for (const UObject* Subobject : Subobjects)
	{
		TRACE_OBJECT_LIFETIME_BEGIN(Subobject);
	}
}
} // UE::RewindDebugger

FRewindDebugger::FRewindDebugger()
{
	using namespace RewindDebugger;

#if WITH_TRACE_BASED_DEBUGGERS
	using namespace UE::TraceBasedDebuggers;

	if (FRewindDebuggerRuntime::Instance() == nullptr)
	{
		FRewindDebuggerRuntime::Initialize();
	}

	if (FRewindDebuggerRuntime* Runtime = FRewindDebuggerRuntime::Instance())
	{
		ClearRecordingHandle = Runtime->ClearRecording.AddRaw(this, &FRewindDebugger::OnClearRecording);
	}
#endif // WITH_TRACE_BASED_DEBUGGERS

	FRewindDebuggerTrackCreators::EnumerateCreators([this](const IRewindDebuggerTrackCreator* Creator)
		{
			Creator->GetTrackTypes(TrackTypes);
		});

	RecordingDuration.Set(0);

	PreBeginPIEHandle = FEditorDelegates::PreBeginPIE.AddRaw(this, &FRewindDebugger::OnPIEStarted);
	PausePIEHandle = FEditorDelegates::PausePIE.AddRaw(this, &FRewindDebugger::OnPIEPaused);
	ResumePIEHandle = FEditorDelegates::ResumePIE.AddRaw(this, &FRewindDebugger::OnPIEResumed);
	SingleStepPIEHandle = FEditorDelegates::SingleStepPIE.AddRaw(this, &FRewindDebugger::OnPIESingleStepped);
	// Using ShutdownPIE instead of EndPIE to make sure all traces emitted during world EndPlay get processed before disabling channels
	ShutdownPIEHandle = FEditorDelegates::ShutdownPIE.AddRaw(this, &FRewindDebugger::OnPIEStopped);
	SIESwitchHandle = FEditorDelegates::OnSwitchBeginPIEAndSIE.AddRaw(this, &FRewindDebugger::OnSIESwitch);

	RootObjectName.OnPropertyChanged = RootObjectName.OnPropertyChanged.CreateLambda([this](FString Target)
		{
			URewindDebuggerSettings& Settings = URewindDebuggerSettings::Get();
			if (Settings.DebugTargetActor != Target)
			{
				Settings.DebugTargetActor = Target;
				Settings.Modify();
				Settings.SaveConfig();
			}

			GetTargetObjectIds(CandidateIds);
			// make sure all the SubObjects of the root object have been traced
#if OBJECT_TRACE_ENABLED
			for (const FObjectId& TargetObjectId : CandidateIds)
			{
				if (const UObject* TargetObject = FObjectTrace::GetObjectFromId(TargetObjectId.GetMainId()))
				{
					UE::RewindDebugger::TraceSubobjects(TargetObject);
				}
			}
#endif

			RefreshDebugTracks();
		});

	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("RewindDebugger"), 0.0f, [this](float DeltaTime)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FRewindDebuggerModule_Tick);

			Tick(DeltaTime);

			return true;
		});

#if WITH_TRACE_BASED_DEBUGGERS
	using namespace UE::RewindDebugger;
	if (const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = FRewindDebuggerEngineEditorBridge::Get().GetSessionsManager())
	{
		// Create remote sessions recording controls and register them to the main toolbar
		RecordingControls = MakeShared<FDebuggerRecordingControls>(FRewindDebuggerModule::MainStatusBarName
			, RemoteSessionManager.ToSharedRef()
			, DebuggerGuid
			, LogRewindDebugger);
		RecordingControls->Initialize();
		RecordingControls->AddToMenu(FRewindDebuggerModule::MainToolBarName);

		RecordingControls->OnRecordingStarted.BindRaw(this, &FRewindDebugger::HandleRecordingStarted);
		RecordingControls->OnRecordingStopped.BindRaw(this, &FRewindDebugger::HandleRecordingStopped);
		RecordingControls->OnSessionSelected.BindRaw(this, &FRewindDebugger::HandleSessionSelected);
	}
#endif // WITH_TRACE_BASED_DEBUGGERS
}

FRewindDebugger::~FRewindDebugger()
{
	FEditorDelegates::PreBeginPIE.Remove(PreBeginPIEHandle);
	FEditorDelegates::PausePIE.Remove(PausePIEHandle);
	FEditorDelegates::ResumePIE.Remove(ResumePIEHandle);
	FEditorDelegates::SingleStepPIE.Remove(SingleStepPIEHandle);
	FEditorDelegates::ShutdownPIE.Remove(ShutdownPIEHandle);
	FEditorDelegates::OnSwitchBeginPIEAndSIE.Remove(SIESwitchHandle);

	FTSTicker::RemoveTicker(TickerHandle);

#if WITH_TRACE_BASED_DEBUGGERS
	if (RewindDebugger::FRewindDebuggerRuntime* Runtime = RewindDebugger::FRewindDebuggerRuntime::Instance())
	{
		Runtime->ClearRecording.Remove(ClearRecordingHandle);
	}

	if (RecordingControls)
	{
		RecordingControls->OnRecordingStarted.Unbind();
		RecordingControls->OnRecordingStopped.Unbind();
		RecordingControls->OnSessionSelected.Unbind();
	}

	// Close any active analysis sessions
	using namespace UE::TraceBasedDebuggers;
	if (ActiveAnalysisDescriptors.Num())
	{
		if (const TSharedPtr<FTraceSessionsManager> TraceManager = FRewindDebuggerModule::Get().GetTraceSessionsManager())
		{
			for (const FTraceSessionDescriptor& ActiveAnalysisDescriptor : ActiveAnalysisDescriptors)
			{
				if (ActiveAnalysisDescriptor.IsValid())
				{
					TraceManager->CloseSession(ActiveAnalysisDescriptor.SessionName);
				}
			}
		}
	}
#endif // WITH_TRACE_BASED_DEBUGGERS
}

void FRewindDebugger::Initialize()
{
	FRewindDebugger* Instance = new FRewindDebugger;
	InternalInstance = Instance;

	// Triggering callbacks after assigning the global instance
	// so code called from the callback don't track to initialize the instance again.
	if (GEditor->bIsSimulatingInEditor || GEditor->PlayWorld)
	{
		Instance->OnPIEStarted(true);
	}
}

void FRewindDebugger::Shutdown()
{
	delete InternalInstance;
}

void FRewindDebugger::SetTrackListChangedDelegate(const FOnTrackListChanged& InTrackListChangedDelegate)
{
	TrackListChangedDelegate = InTrackListChangedDelegate;
}

void FRewindDebugger::SetTrackCursorDelegate(const FOnTrackCursor& InTrackCursorDelegate)
{
	TrackCursorDelegate = InTrackCursorDelegate;
}

void FRewindDebugger::OnPIEStarted(bool bSimulating)
{
	bPIEStarted = true;
	bPIESimulating = true;

	if (ShouldAutoRecordOnPIE())
	{
		StartRecordingLocalSession();

#if WITH_TRACE_BASED_DEBUGGERS
		// Wait for trace channels to be enabled before PIE continues to BeginPlay.
		// StartRecordingLocalSession triggers an async chain:
		//   FRuntimeModule::StartRecording -> Trace worker thread -> HandleTraceConnectionEstablished -> EnableRequiredTraceChannels
		// Without this wait, early lifecycle trace events (OnBeginPlay, etc.) are lost.
		if (::RewindDebugger::FRewindDebuggerRuntime* Runtime = ::RewindDebugger::FRewindDebuggerRuntime::Instance())
		{
			constexpr float MaxWaitTimeSeconds = 3.0f;
			Runtime->WaitForTraceChannelsEnabled(MaxWaitTimeSeconds);
		}
#endif // WITH_TRACE_BASED_DEBUGGERS
	}
}

void FRewindDebugger::OnPIEPaused(bool bSimulating)
{
	bPIESimulating = false;
	ControlState = EControlState::Pause;

	if (ShouldAutoEject() && FPlayWorldCommandCallbacks::IsInPIE())
	{
		bool CanEject = false;
		for (auto It = GUnrealEd->SlatePlayInEditorMap.CreateIterator(); It; ++It)
		{
			CanEject = CanEject || It.Value().DestinationSlateViewport.IsValid();
		}

		if (CanEject)
		{
			GEditor->RequestToggleBetweenPIEandSIE();
		}
	}
}

void FRewindDebugger::OnPIEResumed(bool bSimulating)
{
	bPIESimulating = true;

	if (IsRecordingLocalSession())
	{
		// Reactivate auto-scroll assuming user is done inspecting recorded data
		bAutoScrollToLastItem = true;
	}

	if (ShouldAutoEject() && FPlayWorldCommandCallbacks::IsInSIE())
	{
		GEditor->RequestToggleBetweenPIEandSIE();
	}
}

void FRewindDebugger::OnPIESingleStepped(bool bSimulating)
{
	if (IsRecordingLocalSession())
	{
		// Reactivate auto-scroll assuming user wants to visualize data for the next frame
		bAutoScrollToLastItem = true;
	}
}

void FRewindDebugger::OnSIESwitch(bool bSimulating)
{
	if (!bSimulating)
	{
		const FActorPickerModeModule& ActorPickerMode = FModuleManager::Get().GetModuleChecked<FActorPickerModeModule>("ActorPickerMode");
		if (ActorPickerMode.IsInActorPickingMode())
		{
			ActorPickerMode.EndActorPickingMode();
		}
	}
}

void FRewindDebugger::OnPIEStopped(bool bSimulating)
{
	if (IsRecordingLocalSession())
	{
		StopRecordingLocalSession();
	}

	bPIEStarted = false;
	bPIESimulating = false;
	bDisplayWorldIdValid = false;
}

bool FRewindDebugger::GetRootObjectPosition(FVector& OutPosition) const
{
	OutPosition = RootObjectPosition.Get(FVector::ZeroVector);
	return RootObjectPosition.IsSet();
}

void FRewindDebugger::SetRootObjectPosition(const TOptional<FVector>& InPosition)
{
	RootObjectPosition = InPosition;
}

void FRewindDebugger::GetTargetObjectIds(TArray<RewindDebugger::FObjectId>& OutTargetObjectIds) const
{
	// We are looking for the object ID matching RootObjectName and currently supporting a single root track.
	OutTargetObjectIds.Empty(1);

	if (RootObjectName.Get() == "")
	{
		return;
	}

	if (const TraceServices::IAnalysisSession* Session = GetAnalysisSession())
	{
		if (const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider"))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			GameplayProvider->EnumerateObjects(CurrentTraceRange.GetLowerBoundValue(), CurrentTraceRange.GetUpperBoundValue(), [this, &OutTargetObjectIds](const FObjectInfo& InObjectInfo)
				{
					if (RootObjectName.Get() == InObjectInfo.Name)
					{
						OutTargetObjectIds.Add(InObjectInfo.GetId());
					}
				});
		}
	}

	// make sure all the SubObjects of the root object have been traced
#if OBJECT_TRACE_ENABLED
	if (IsRecordingLocalSession())
	{
		for (const RewindDebugger::FObjectId& CandidateId : CandidateIds)
		{
			if (const UObject* TargetObject = FObjectTrace::GetObjectFromId(CandidateId.GetMainId()))
			{
				UE::RewindDebugger::TraceSubobjects(TargetObject);
			}
		}
	}
#endif
}

void FRewindDebugger::RefreshDebugTracks()
{
	static const FName DebugMessageTrackName = "DebugMessageDummyTrack";
	TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebugger::RefreshDebugTracks);

	if (CandidateIds.Num() == 0)
	{
		GetTargetObjectIds(CandidateIds);
	}

	const FString SelectionName = RootObjectName.Get();

	if (CandidateIds.Num() == 0 && !SelectionName.IsEmpty())
	{
		// fallback code path for when the target object is not found

		if (DebugTracks.Num() != UE::RewindDebugger::NumDefaultDebugTracks)
		{
			// clear tracks so we don't show data from previous recordings
			DebugTracks.SetNum(0);
			DebugTracks.SetNum(UE::RewindDebugger::NumDefaultDebugTracks);
		}

		if (DebugTracks[1] == nullptr || DebugTracks[0] == nullptr || DebugTracks[0]->GetName().ToString() != SelectionName)
		{
			DebugTracks[0] = MakeShared<FRewindDebuggerPlaceholderTrack>(FName(SelectionName), FText::FromString(SelectionName));
			DebugTracks[1] = MakeShared<FRewindDebuggerPlaceholderTrack>(DebugMessageTrackName, NSLOCTEXT("RewindDebugger", "No Debug Data", " - Start a recording to debug"));
			OnTrackListChanged();
		}
	}
	else if (DebugTracks.Num() || CandidateIds.Num())
	{
		bool bChanged = false;

		// remove any existing tracks that don't match the current list of object ids
		for (int TrackIndex = DebugTracks.Num() - 1; TrackIndex >= 0; TrackIndex--)
		{
			if (!CandidateIds.Contains(DebugTracks[TrackIndex]->GetAssociatedObjectId()))
			{
				DebugTracks.RemoveAt(TrackIndex);
			}
		}

		// add new tracks for current list of object identifiers if they don't already exist
		for (const RewindDebugger::FObjectId& CandidateIdentifier : CandidateIds)
		{
			const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>* FoundTrack = DebugTracks.FindByPredicate(
				[CandidateIdentifier](const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& Track)
				{
					return Track->GetAssociatedObjectId() == CandidateIdentifier;
				});

			if (!FoundTrack)
			{
				DebugTracks.Add(MakeShared<RewindDebugger::FRewindDebuggerObjectTrack>(CandidateIdentifier, RootObjectName.Get(), true));
				bChanged = true;
			}
		}

		// update all tracks
		for (const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& DebugTrack : DebugTracks)
		{
			if (DebugTrack->Update())
			{
				UE_LOGF(LogRewindDebugger, Verbose, "List changed by: '%ls'", *DebugTrack->GetDisplayName().ToString());
				bChanged = true;
			}
		}

		if (bChanged)
		{
			OnTrackListChanged();
		}
	}
}

bool FRewindDebugger::CanStartRecording() const
{
	// Preserving legacy behavior of IRewindDebugger interface by monitoring the local session only.
	return CanStartRecordingLocalSession();
}

void FRewindDebugger::StartRecording() const
{
	// Preserving legacy behavior of IRewindDebugger interface by recording the local session only.
	StartRecordingLocalSession();
}

bool FRewindDebugger::CanStartRecordingLocalSession() const
{
	// Relying on the recording controls using the
	// sessions manager which also handles active sessions from other debuggers.
	return RecordingControls
		&& RecordingControls->CanStartRecordingLocalSession();
}

void FRewindDebugger::StartRecordingLocalSession() const
{
	if (!CanStartRecordingLocalSession())
	{
		return;
	}

	if (RecordingControls)
	{
		RecordingControls->StartRecordingLocalSession();
	}
}

#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS
void FRewindDebugger::HandleRecordingStarted(const TSharedPtr<UE::TraceBasedDebuggers::FSessionInfo>& InSessionInfo)
{
	if (RecordingControls
		&& RecordingControls->IsRecordingLocalSession())
	{
		bAutoScrollToLastItem = true;
	}
}

void FRewindDebugger::HandleRecordingStopped(const TSharedPtr<UE::TraceBasedDebuggers::FSessionInfo>& InSessionInfo)
{
	using namespace UE::TraceBasedDebuggers;
	const FTraceSessionDescriptor* ExistingDescriptor = ActiveAnalysisDescriptors.FindByPredicate(
		[SessionInfo = InSessionInfo](const FTraceSessionDescriptor& Descriptor)
		{
			return Descriptor.RemoteSessionID == SessionInfo->InstanceId;
		});

	if (ExistingDescriptor)
	{
		StopAnalysisSession(*ExistingDescriptor);
	}
}

void FRewindDebugger::HandleSessionSelected(const TSharedPtr<UE::TraceBasedDebuggers::FSessionInfo>& InSessionInfo)
{
	using namespace UE::TraceBasedDebuggers;

	// If a file analysis is active we don't want to stop it
	if (SelectedDescriptor.IsValid()
		&& !SelectedDescriptor.RemoteSessionID.IsValid())
	{
		return;
	}

	const FTraceSessionDescriptor* ExistingDescriptor = ActiveAnalysisDescriptors.FindByPredicate(
		[SessionInfo = InSessionInfo](const FTraceSessionDescriptor& Descriptor)
		{
			return Descriptor.RemoteSessionID == SessionInfo->InstanceId;
		});

	ClearTrackData();

	if (ExistingDescriptor)
	{
		SetSelectedAnalysisSession(*ExistingDescriptor);
	}
	else
	{
		SetSelectedAnalysisSession(FTraceSessionDescriptor{});
	}
}
#endif // UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS

void FRewindDebugger::StopRecordingLocalSession() const
{
	if (RecordingControls)
	{
		RecordingControls->StopRecordingLocalSession();
	}
}

bool FRewindDebugger::IsRecordingLocalSession() const
{
	return RecordingControls
		&& RecordingControls->IsRecordingLocalSession();
}

#if WITH_TRACE_BASED_DEBUGGERS
void FRewindDebugger::StopLocalRecording() const
{
	RewindDebugger::FRewindDebuggerRuntime* Runtime = RewindDebugger::FRewindDebuggerRuntime::Instance();
	if (Runtime != nullptr && Runtime->IsRecording())
	{
		Runtime->StopRecording();
	}
}
#endif // WITH_TRACE_BASED_DEBUGGERS

bool FRewindDebugger::CanOpenTrace() const
{
	return !IsRecordingLocalSession();
}

void FRewindDebugger::OpenTrace(const FString& FilePath)
{
	ClearAnalysis();

	bDisplayWorldIdValid = false;

#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FTraceSessionsManager> TraceSessionsManager = FRewindDebuggerModule::Get().GetTraceSessionsManager())
	{
		FTraceSessionDescriptor NewSessionFromFileDescriptor;
		NewSessionFromFileDescriptor.SessionName = TraceSessionsManager->LoadTraceFile(FilePath);
		NewSessionFromFileDescriptor.bIsLiveSession = false;

		if (NewSessionFromFileDescriptor.IsValid())
		{
			OpenAnalysisSession(NewSessionFromFileDescriptor);
		}
	}
#endif // UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS

	// todo: optionally open the map the trace file was recorded in
}

void FRewindDebugger::OpenTrace()
{
	const FString FolderPath = "";

	TArray<FString> OutOpenFilenames;
	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		FString ExtensionStr;
		ExtensionStr += TEXT("Unreal Trace|*.utrace|");

		DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("OpenDialogTitle", "Open Rewind Debugger Recording").ToString(),
			FolderPath,
			TEXT(""),
			*ExtensionStr,
			EFileDialogFlags::None,
			OutOpenFilenames
		);
	}

	if (OutOpenFilenames.Num() > 0)
	{
		if (OutOpenFilenames[0].EndsWith(TEXT("utrace")))
		{
			OpenTrace(OutOpenFilenames[0]);
		}
	}
}



bool FRewindDebugger::CanClearAnalysis() const
{
	return GetAnalysisSession() != nullptr || RecordingDuration.Get() > 0.f;
}

void FRewindDebugger::ClearAnalysis()
{
	ClearActiveAnalysisSession();
	ClearTrackData();
}

void FRewindDebugger::ClearTrackData()
{
	RecordingDuration.Set(0);

	CandidateIds.Empty();
	CurrentTraceRange.SetLowerBoundValue(0);
	CurrentTraceRange.SetUpperBoundValue(0);
	RecordingDuration.Set(0.0);
	SetCurrentScrubTime(0.0, EAutoScrollChange::PreserveCurrentState);

	ClearTrackSelection();

	// update extensions
	UE::RewindDebugger::IterateExtensions([this](IRewindDebuggerExtension* Extension)
		{
			Extension->Clear(this);
		}
	);

	RefreshDebugTracks();
}

void FRewindDebugger::OnClearRecording()
{
	RootObjectPosition.Reset();

	ClearTrackData();
}

void FRewindDebugger::ToggleAutoScroll()
{
	bAutoScrollToLastItem = !bAutoScrollToLastItem;
	if (bAutoScrollToLastItem)
	{
		SetCurrentScrubTime(RecordingDuration.Get(), EAutoScrollChange::PreserveCurrentState);
		TrackCursorDelegate.ExecuteIfBound(/*bReverse*/false);
	}
}

bool FRewindDebugger::CanEnableAutoScroll() const
{
	return GetAnalysisSession() != nullptr;
}

bool FRewindDebugger::IsAutoScrollEnabled() const
{
	return bAutoScrollToLastItem;
}

bool FRewindDebugger::ShouldAutoRecordOnPIE() const
{
	return URewindDebuggerSettings::Get().bShouldAutoRecordOnPIE;
}

void FRewindDebugger::SetShouldAutoRecordOnPIE(bool value)
{
	URewindDebuggerSettings& RewindDebuggerSettings = URewindDebuggerSettings::Get();
	RewindDebuggerSettings.Modify();
	RewindDebuggerSettings.bShouldAutoRecordOnPIE = value;
	RewindDebuggerSettings.SaveConfig();
}

bool FRewindDebugger::ShouldAutoEject() const
{
	return URewindDebuggerSettings::Get().bShouldAutoEject;
}

void FRewindDebugger::SetShouldAutoEject(bool value)
{
	URewindDebuggerSettings& RewindDebuggerSettings = URewindDebuggerSettings::Get();
	RewindDebuggerSettings.Modify();
	RewindDebuggerSettings.bShouldAutoEject = value;
	RewindDebuggerSettings.SaveConfig();
}

bool FRewindDebugger::CanUsePlaybackControls() const
{
	// We now allow playback controls at any time and using them will disable auto-scroll
	return true;
}

bool FRewindDebugger::CanPause() const
{
	return ControlState != EControlState::Pause;
}

void FRewindDebugger::Pause()
{
	if (CanPause())
	{
		if (bPIESimulating)
		{
			// pause PIE
		}

		ControlState = EControlState::Pause;
	}
}

bool FRewindDebugger::IsPlaying() const
{
	return ControlState == EControlState::Play && CanUsePlaybackControls();
}

bool FRewindDebugger::CanPlay() const
{
	return ControlState != EControlState::Play && CanUsePlaybackControls() && RecordingDuration.Get() > 0;
}

void FRewindDebugger::Play()
{
	if (CanPlay())
	{
		// Loop from the beginning if already at the end,
		// otherwise keep the current time but make sure to disable the auto-scroll
		const double AdjustedTime = (ScrubTimeInformation.ElapsedTime >= RecordingDuration.Get()) ? 0 : ScrubTimeInformation.ElapsedTime;
		SetCurrentScrubTime(AdjustedTime, EAutoScrollChange::Disable);
		TrackCursorDelegate.ExecuteIfBound(/*bReverse*/false);

		ControlState = EControlState::Play;
	}
}

bool FRewindDebugger::CanPlayReverse() const
{
	return ControlState != EControlState::PlayReverse && CanUsePlaybackControls() && RecordingDuration.Get() > 0;
}

void FRewindDebugger::PlayReverse()
{
	if (CanPlayReverse())
	{
		// Loop from the end if already at the beginning,
		// otherwise keep the current time but make sure to disable the auto-scroll
		const double AdjustedTime = (ScrubTimeInformation.ElapsedTime <= 0) ? RecordingDuration.Get() : ScrubTimeInformation.ElapsedTime;
		SetCurrentScrubTime(AdjustedTime, EAutoScrollChange::Disable);
		TrackCursorDelegate.ExecuteIfBound(/*bReverse*/true);

		ControlState = EControlState::PlayReverse;
	}
}

bool FRewindDebugger::CanScrub() const
{
	return CanUsePlaybackControls() && RecordingDuration.Get() > 0;
}

void FRewindDebugger::ScrubToStart()
{
	if (CanScrub())
	{
		Pause();
		SetCurrentScrubTime(0, EAutoScrollChange::Disable);
		TrackCursorDelegate.ExecuteIfBound(/*bReverse*/false);
	}
}

void FRewindDebugger::ScrubToEnd()
{
	if (CanScrub())
	{
		Pause();
		SetCurrentScrubTime(RecordingDuration.Get(), EAutoScrollChange::Disable);
		TrackCursorDelegate.ExecuteIfBound(/*bReverse*/false);
	}
}

void FRewindDebugger::Step(const int32 InNumberOfFrames)
{
	if (CanScrub())
	{
		Pause();

		if (FMath::Abs(InNumberOfFrames) == 1
			&& SelectedTrack)
		{
			const TOptional<double> NewScrubTime = SelectedTrack->GetStepFrameTime(
				InNumberOfFrames == 1
					? RewindDebugger::EStepMode::Forward
					: RewindDebugger::EStepMode::Backward
				, ScrubTimeInformation);

			if (NewScrubTime.IsSet())
			{
				SetCurrentScrubTime(NewScrubTime.GetValue(), EAutoScrollChange::Disable);
				TrackCursorDelegate.ExecuteIfBound(/*bReverse*/false);
				return;
			}
		}

		if (const TraceServices::IAnalysisSession* Session = GetAnalysisSession())
		{
			if (const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider"))
			{
				bool bEventFound = false;
				double ElapsedTime = 0;
				{
					TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
					if (const IGameplayProvider::RecordingInfoTimeline* Recording = GameplayProvider->GetRecordingInfo(RecordingIndex))
					{
						const uint64 EventCount = Recording->GetEventCount();

						if (EventCount > 0)
						{
							ScrubTimeInformation.FrameIndex = FMath::Clamp<int64>(ScrubTimeInformation.FrameIndex + InNumberOfFrames, 0, (int64)EventCount - 1);
							const FRecordingInfoMessage& Event = Recording->GetEvent(ScrubTimeInformation.FrameIndex);
							bEventFound = true;
							ElapsedTime = Event.ElapsedTime;
						}
					}
				}

				if (bEventFound)
				{
					SetCurrentScrubTime(ElapsedTime, EAutoScrollChange::Disable);
					TrackCursorDelegate.ExecuteIfBound(/*bReverse*/false);
				}
			}
		}
	}
}

void FRewindDebugger::StepForward()
{
	Step(1);
}

void FRewindDebugger::StepBackward()
{
	Step(-1);
}

void FRewindDebugger::ScrubToTime(const double ScrubTime, bool bIsScrubbing)
{
	if (CanScrub())
	{
		Pause();
		SetCurrentScrubTime(ScrubTime, EAutoScrollChange::Disable);

		// No need to broadcast TrackCursorDelegate here since this is the callback
		// handling the timeline cursor updates
	}
}

UWorld* FRewindDebugger::GetWorldToVisualize() const
{
	// we probably want to replace this with a world selector widget, if we are going to support tracing from anything other thn the PIE world

	UWorld* World = nullptr;

	UEditorEngine* EditorEngine = Cast<UEditorEngine>(GEngine);
	if (GIsEditor && EditorEngine != nullptr && World == nullptr)
	{
		// let's use PlayWorld during PIE/Simulate and regular world from editor otherwise, to draw debug information
		World = EditorEngine->PlayWorld != nullptr ? ToRawPtr(EditorEngine->PlayWorld) : EditorEngine->GetEditorWorldContext().World();
	}

	return World;
}

bool FRewindDebugger::IsRecording() const
{
	// Preserving legacy behavior of IRewindDebugger interface by monitoring the local session only.
	return RecordingControls ? RecordingControls->IsRecordingLocalSession() : false;
}

bool FRewindDebugger::IsAnalyzingLocalSession() const
{
	return SelectedDescriptor.IsValid()
		&& SelectedDescriptor.RemoteSessionID.IsValid()
		&& SelectedDescriptor.RemoteSessionID == UE::TraceBasedDebuggers::FRemoteSessionsManager::LocalEditorSessionID;
}

bool FRewindDebugger::IsAnalyzingRemoteSession() const
{
	return SelectedDescriptor.IsValid()
		&& SelectedDescriptor.RemoteSessionID.IsValid()
		&& SelectedDescriptor.RemoteSessionID != UE::TraceBasedDebuggers::FRemoteSessionsManager::LocalEditorSessionID;
}

bool FRewindDebugger::IsAnalyzingTraceFile() const
{
	return SelectedDescriptor.IsValid()
		&& !SelectedDescriptor.RemoteSessionID.IsValid();
}

void FRewindDebugger::SetCurrentViewRange(const TRange<double>& Range)
{
	CurrentViewRange = Range;
	if (const TraceServices::IAnalysisSession* Session = GetAnalysisSession())
	{
		GetScrubTimeInformation(CurrentViewRange.GetLowerBoundValue(), LowerBoundViewTimeInformation, RecordingIndex, Session);
		GetScrubTimeInformation(CurrentViewRange.GetUpperBoundValue(), UpperBoundViewTimeInformation, RecordingIndex, Session);

		CurrentTraceRange.SetLowerBoundValue(LowerBoundViewTimeInformation.ProfileTime);
		CurrentTraceRange.SetUpperBoundValue(UpperBoundViewTimeInformation.ProfileTime);
	}
}

void FRewindDebugger::SetCurrentScrubTime(const double Time, const EAutoScrollChange AutoScrollChange)
{
	if (const TraceServices::IAnalysisSession* Session = GetAnalysisSession())
	{
		GetScrubTimeInformation(Time, ScrubTimeInformation, RecordingIndex, Session);

		TraceTime.Set(ScrubTimeInformation.ProfileTime);
	}

	// Enforce the specific Time that was provided
	ScrubTimeInformation.ElapsedTime = Time;

	if (bAutoScrollToLastItem
		&& AutoScrollChange == EAutoScrollChange::Disable)
	{
		bAutoScrollToLastItem = false;
	}
}

void FRewindDebugger::GetScrubTimeInformation(double InDebugTime, FScrubTimeInformation& InOutTimeInformation, uint16 InRecordingIndex, const TraceServices::IAnalysisSession* AnalysisSession)
{
	const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider");

	if (GameplayProvider)
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
		if (const IGameplayProvider::RecordingInfoTimeline* Recording = GameplayProvider->GetRecordingInfo(InRecordingIndex))
		{
			const uint64 EventCount = Recording->GetEventCount();

			if (EventCount > 0)
			{
				int ScrubFrameIndex = InOutTimeInformation.FrameIndex;
				const FRecordingInfoMessage& FirstEvent = Recording->GetEvent(0);
				const FRecordingInfoMessage& LastEvent = Recording->GetEvent(EventCount - 1);

				// Check if we are outside the recorded range, and apply the first or last frame
				if (InDebugTime <= FirstEvent.ElapsedTime)
				{
					ScrubFrameIndex = FMath::Min<uint64>(1, EventCount - 1);
				}
				else if (InDebugTime >= LastEvent.ElapsedTime)
				{
					ScrubFrameIndex = EventCount - 1;
				}
				// Find the two keys surrounding the InDebugTime, and pick the nearest to update InOutTimeInformation
				else
				{
					const FRecordingInfoMessage& ScrubEvent = Recording->GetEvent(ScrubFrameIndex);
					constexpr float MaxTimeDifferenceInSeconds = 15.0f / 60.0f;

					// Use linear search on smaller time differences
					if (FMath::Abs(InDebugTime - ScrubEvent.ElapsedTime) <= MaxTimeDifferenceInSeconds)
					{
						if (Recording->GetEvent(ScrubFrameIndex).ElapsedTime > InDebugTime)
						{
							for (uint64 EventIndex = ScrubFrameIndex; EventIndex > 0; EventIndex--)
							{
								const FRecordingInfoMessage& Event = Recording->GetEvent(EventIndex);
								const FRecordingInfoMessage& NextEvent = Recording->GetEvent(EventIndex - 1);
								if (Event.ElapsedTime >= InDebugTime && NextEvent.ElapsedTime <= InDebugTime)
								{
									if (Event.ElapsedTime - InDebugTime < InDebugTime - NextEvent.ElapsedTime)
									{
										ScrubFrameIndex = EventIndex;
									}
									else
									{
										ScrubFrameIndex = EventIndex - 1;
									}
									break;
								}
							}
						}
						else
						{
							for (uint64 EventIndex = ScrubFrameIndex; EventIndex < EventCount - 1; EventIndex++)
							{
								const FRecordingInfoMessage& Event = Recording->GetEvent(EventIndex);
								const FRecordingInfoMessage& NextEvent = Recording->GetEvent(EventIndex + 1);
								if (Event.ElapsedTime <= InDebugTime && NextEvent.ElapsedTime >= InDebugTime)
								{
									if (InDebugTime - Event.ElapsedTime < NextEvent.ElapsedTime - InDebugTime)
									{
										ScrubFrameIndex = EventIndex;
									}
									else
									{
										ScrubFrameIndex = EventIndex + 1;
									}
									break;
								}
							}
						}
					}
					// Binary search for surrounding keys on big time differences
					else
					{
						uint64 StartEventIndex = 0;
						uint64 EndEventIndex = EventCount - 1;

						while (EndEventIndex - StartEventIndex > 1)
						{
							const uint64 MiddleEventIndex = ((StartEventIndex + EndEventIndex) / 2);
							const FRecordingInfoMessage& MiddleEvent = Recording->GetEvent(MiddleEventIndex);
							if (InDebugTime < MiddleEvent.ElapsedTime)
							{
								EndEventIndex = MiddleEventIndex;
							}
							else
							{
								StartEventIndex = MiddleEventIndex;
							}
						}

						// Ensure there is not frames between start and end index
						check(EndEventIndex == StartEventIndex + 1);

						const FRecordingInfoMessage& Event = Recording->GetEvent(StartEventIndex);
						const FRecordingInfoMessage& NextEvent = Recording->GetEvent(EndEventIndex);

						// Ensure debug time is between both frames time range
						check(Event.ElapsedTime <= InDebugTime && NextEvent.ElapsedTime >= InDebugTime);

						// Choose frame that is nearest to the debug time
						if (InDebugTime - Event.ElapsedTime < NextEvent.ElapsedTime - InDebugTime)
						{
							ScrubFrameIndex = StartEventIndex;
						}
						else
						{
							ScrubFrameIndex = EndEventIndex;
						}
					}
				}

				const FRecordingInfoMessage& Event = Recording->GetEvent(ScrubFrameIndex);
				InOutTimeInformation.FrameIndex = ScrubFrameIndex;
				InOutTimeInformation.ProfileTime = Event.ProfileTime;
				InOutTimeInformation.ElapsedTime = Event.ElapsedTime;
			}
		}
	}
}

TSharedPtr<UE::TraceBasedDebuggers::FSessionInfo> FRewindDebugger::GetSelectedDebugSessionInfo() const
{
	return RecordingControls ? RecordingControls->GetCurrentSessionInfo() : nullptr;
}

const TraceServices::IAnalysisSession* FRewindDebugger::GetAnalysisSession() const
{
	return GetAnalysisSessionAsShared().Get();
}

TSharedPtr<const TraceServices::IAnalysisSession> FRewindDebugger::GetAnalysisSessionAsShared() const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = nullptr;

#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS
	if (SelectedDescriptor.IsValid())
	{
		using namespace UE::TraceBasedDebuggers;
		if (const TSharedPtr<FTraceSessionsManager> TraceManager = FRewindDebuggerModule::Get().GetTraceSessionsManager())
		{
			Session = TraceManager->GetSession(SelectedDescriptor.SessionName);
		}
	}
#endif // UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS

	return Session;
}

uint64 FRewindDebugger::GetRootObjectId() const
{
	return DebuggedObjects.Num() ? DebuggedObjects[0]->GetUObjectId() : RewindDebugger::FObjectId::InvalidId;
}

const FObjectInfo* FRewindDebugger::FindTypedOuterInfo(TNotNull<const UStruct*> InType, TNotNull<const IGameplayProvider*> InGameplayProvider, const uint64 InObjectId) const
{
	const FClassInfo* TypeInfo = InGameplayProvider->FindClassInfo(*InType->GetPathName());

	uint64 ObjectId(InObjectId);
	while (true)
	{
		const FObjectInfo& ObjectInfo = InGameplayProvider->GetObjectInfo(ObjectId);
		if (InGameplayProvider->IsSubClassOf(ObjectInfo.ClassId, TypeInfo->Id))
		{
			return &ObjectInfo;
		}

		if (!ObjectInfo.GetOuterId().IsSet())
		{
			return nullptr;
		}

		ObjectId = ObjectInfo.GetOuterUObjectId();
	}
}

#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS

bool FRewindDebugger::ConnectToLiveSession(const uint32 SessionID, const FStringView InSessionAddress)
{
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FTraceSessionsManager> TraceSessionsManager = FRewindDebuggerModule::Get().GetTraceSessionsManager())
	{
		FTraceSessionDescriptor NewSessionDescriptor;
		NewSessionDescriptor.bIsLiveSession = true;
		NewSessionDescriptor.SessionName = TraceSessionsManager->ConnectToLiveSession(InSessionAddress, SessionID);

		if (NewSessionDescriptor.IsValid())
		{
			OpenAnalysisSession(NewSessionDescriptor);
			return true;
		}
	}

	return false;
}

bool FRewindDebugger::ConnectToLiveSession_Direct(const FGuid& InRemoteSessionID)
{
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FTraceSessionsManager> TraceSessionsManager = FRewindDebuggerModule::Get().GetTraceSessionsManager())
	{
		FTraceSessionDescriptor NewSessionDescriptor;
		NewSessionDescriptor.RemoteSessionID = InRemoteSessionID;
		NewSessionDescriptor.bIsLiveSession = true;
		NewSessionDescriptor.SessionName = TraceSessionsManager->ConnectToLiveSession_Direct(InRemoteSessionID, NewSessionDescriptor.SessionPort);

		if (NewSessionDescriptor.IsValid())
		{
			OpenAnalysisSession(NewSessionDescriptor);
			return true;
		}
	}

	return false;
}

bool FRewindDebugger::ConnectToLiveSession_Relay(const FGuid& InRemoteSessionID)
{
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FTraceSessionsManager> TraceSessionsManager = FRewindDebuggerModule::Get().GetTraceSessionsManager())
	{
		FTraceSessionDescriptor NewSessionDescriptor;
		NewSessionDescriptor.bIsLiveSession = true;
		NewSessionDescriptor.RemoteSessionID = InRemoteSessionID;
		NewSessionDescriptor.SessionName = TraceSessionsManager->ConnectToLiveSession_Relay(InRemoteSessionID);

		if (NewSessionDescriptor.IsValid())
		{
			OpenAnalysisSession(NewSessionDescriptor);
			return true;
		}
	}

	return false;
}

void FRewindDebugger::SetSelectedAnalysisSession(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& InSessionDescriptor)
{
	SelectedDescriptor = InSessionDescriptor;

	// A new analysis session was selected so make sure to read all the data
	bAutoScrollToLastItem = true;
}

void FRewindDebugger::OpenAnalysisSession(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& InSessionDescriptor)
{
	using namespace UE::TraceBasedDebuggers;

	const TSharedPtr<FTraceSessionsManager> TraceManager = FRewindDebuggerModule::Get().GetTraceSessionsManager();
	ActiveAnalysisDescriptors.RemoveAll([&InSessionDescriptor, TraceManager](const FTraceSessionDescriptor& Element)
		{
			// We currently don't have a way to reselect an analysis session from a trace file
			// so we can clear it when a new analysis starts (from file or remote).
			// In the case or remote sessions, we close only if from the same remote session Id.
			const bool bClosePreviousSession = !Element.RemoteSessionID.IsValid()
				|| Element.RemoteSessionID == InSessionDescriptor.RemoteSessionID;

			if (bClosePreviousSession)
			{
				if (TraceManager)
				{
					TraceManager->CloseSession(Element.SessionName);
				}
				return true;
			}

			return false;
		});

	ActiveAnalysisDescriptors.Emplace(InSessionDescriptor);

	ClearTrackData();

	SetSelectedAnalysisSession(InSessionDescriptor);

	// Notify extensions
	UE::RewindDebugger::IterateExtensions([this](IRewindDebuggerExtension* Extension)
		{
			Extension->AnalysisSessionOpened(this);
		});

	// Invalidate world Id to reselect one from the new analysis
	bDisplayWorldIdValid = false;
}

void FRewindDebugger::ClearAnalysisSessionLinkedToRemoteSessionID(const FGuid& InRemoteSessionID)
{
	using namespace UE::TraceBasedDebuggers;

	if (const FTraceSessionDescriptor* ExistingDescriptor = ActiveAnalysisDescriptors.FindByPredicate(
		[InRemoteSessionID](const FTraceSessionDescriptor& Descriptor)
		{
			return Descriptor.RemoteSessionID == InRemoteSessionID;
		}))
	{
		ClearAnalysisSession(*ExistingDescriptor);
	}
}

void FRewindDebugger::StopAnalysisSession(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& InSessionDescriptor)
{
	using namespace UE::TraceBasedDebuggers;

	if (InSessionDescriptor.IsValid())
	{
		if (const TSharedPtr<FTraceSessionsManager> TraceManager = FRewindDebuggerModule::Get().GetTraceSessionsManager())
		{
			TraceManager->StopSession(InSessionDescriptor.SessionName);
		}

		// Notify extensions
		UE::RewindDebugger::IterateExtensions([this](IRewindDebuggerExtension* Extension)
			{
				Extension->AnalysisSessionClosed(this);
			});
	}
}

void FRewindDebugger::ClearAnalysisSession(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& InSessionDescriptor)
{
	using namespace UE::TraceBasedDebuggers;

	if (InSessionDescriptor.IsValid())
	{
		if (const TSharedPtr<FTraceSessionsManager> TraceManager = FRewindDebuggerModule::Get().GetTraceSessionsManager())
		{
			TraceManager->CloseSession(InSessionDescriptor.SessionName);
		}

		// Notify extensions
		UE::RewindDebugger::IterateExtensions([this](IRewindDebuggerExtension* Extension)
			{
				Extension->AnalysisSessionClosed(this);
			});

		ActiveAnalysisDescriptors.RemoveAll([&InSessionDescriptor](const FTraceSessionDescriptor& Element)
			{
				return Element.RemoteSessionID == InSessionDescriptor.RemoteSessionID;
			});

		if (SelectedDescriptor == InSessionDescriptor)
		{
			SetSelectedAnalysisSession(FTraceSessionDescriptor{});
		}
	}
}

#endif // UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS

void FRewindDebugger::ClearActiveAnalysisSession()
{
#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS
	using namespace UE::TraceBasedDebuggers;

	if (SelectedDescriptor.IsValid())
	{
		ClearAnalysisSession(SelectedDescriptor);
	}
#endif // UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS
}

void FRewindDebugger::Tick(float DeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebugger::Tick);

	if (const TraceServices::IAnalysisSession* Session = GetAnalysisSession())
	{
		double RecordingDurationValue = 0;

		if (const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider"))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

			RecordingDurationValue = GameplayProvider->GetRecordingDuration();

			// set a default display world when analyzing a session (first client/standalone world)
			if (!bDisplayWorldIdValid && (GetAnalysisSession() != nullptr))
			{
				GameplayProvider->EnumerateWorlds([this](const FWorldInfo& WorldInfo)
					{
						if (WorldInfo.Type == FWorldInfo::EType::PIE)
						{
							if (WorldInfo.NetMode == FWorldInfo::ENetMode::Client && WorldInfo.PIEInstanceId == 1)
							{
								DisplayWorldId = WorldInfo.Id;
								bDisplayWorldIdValid = true;
							}
							if (WorldInfo.NetMode == FWorldInfo::ENetMode::Standalone && WorldInfo.PIEInstanceId == 0)
							{
								DisplayWorldId = WorldInfo.Id;
								bDisplayWorldIdValid = true;
							}
						}
						else if (WorldInfo.Type == FWorldInfo::EType::Game)
						{
							DisplayWorldId = WorldInfo.Id;
							bDisplayWorldIdValid = true;
						}
					});
			}
		}

		bool bNewDataAvailable = false;
		if (RecordingDurationValue > RecordingDuration.Get()
			&& (IsAnalyzingTraceFile() || IsAnalyzingRemoteSession()))
		{
			// while trace file is loading up or analyzing a live session, force the trace range to update.
			SetCurrentViewRange(GetCurrentViewRange());
			bNewDataAvailable = true;
		}
		RecordingDuration.Set(RecordingDurationValue);

		RefreshDebugTracks();

		if (bAutoScrollToLastItem)
		{
			// Only need to update scrub time and notify
			// when recording a PIE session or analyzing data from a remote session or trace file.
			if (IsRecordingLocalSession()
				|| bNewDataAvailable)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebugger::Tick_UpdateSimulating);
				SetCurrentScrubTime(RecordingDuration.Get(), EAutoScrollChange::PreserveCurrentState);
				TrackCursorDelegate.ExecuteIfBound(/*bReverse*/false);
			}

			// The debug position is only expected to be set outside of PIE
			RootObjectPosition.Reset();
		}
		else
		{
			const double CurrentScrubTime = ScrubTimeInformation.ElapsedTime;
			if (RecordingDuration.Get() > 0 && CurrentScrubTime <= RecordingDuration.Get())
			{
				if (ControlState == EControlState::Play || ControlState == EControlState::PlayReverse)
				{
					const float PlaybackRate = URewindDebuggerSettings::Get().PlaybackRate;
					TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebugger::Tick_UpdatePlayback);
					const float Rate = PlaybackRate * (ControlState == EControlState::Play ? 1 : -1);
					SetCurrentScrubTime(FMath::Clamp(CurrentScrubTime + Rate * DeltaTime, 0.0f, RecordingDuration.Get()), EAutoScrollChange::Disable);
					TrackCursorDelegate.ExecuteIfBound(/*bReverse*/Rate < 0);

					if (CurrentScrubTime == 0 || CurrentScrubTime == RecordingDuration.Get())
					{
						// pause at end.
						ControlState = EControlState::Pause;
					}
				}
			}
		}

		// update extensions
		UE::RewindDebugger::IterateExtensions([DeltaTime, this](IRewindDebuggerExtension* Extension)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*Extension->GetName());
				Extension->Update(DeltaTime, this);
			}
		);
	}
}

void FRewindDebugger::OnTrackListChanged()
{
	TrackListChangedDelegate.ExecuteIfBound();

	UE::RewindDebugger::IterateExtensions([this](IRewindDebuggerExtension* Extension)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*Extension->GetName());
			Extension->OnTrackListChanged(this);
		}
	);
}

void FRewindDebugger::OpenDetailsPanel()
{
	bIsDetailsPanelOpen = true;
	TrackSelectionChanged(SelectedTrack);
}

void FRewindDebugger::ClearTrackSelection()
{
	SelectedTrack.Reset();

	if (bIsDetailsPanelOpen)
	{
		const FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		const TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

		if (DetailsPanelWidget.IsValid())
		{
			// avoid actually summoning the tab if we are only clearing it
				
			if (EmptyDetails == nullptr)
			{
				EmptyDetails = SNew(SSpacer);
			}
			DetailsPanelWidget->SetContent(EmptyDetails.ToSharedRef());
		}
	}
}

void FRewindDebugger::TrackSelectionChanged(const TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InSelectedTrack)
{
	SelectedTrack = InSelectedTrack;
	if (!SelectedTrack.IsValid())
	{
		// when a selected track goes out of view, we get track selection changed notifications but we don't want to invoke any tab
		return;
	}

	if (bIsDetailsPanelOpen)
	{
		const FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		const TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

		const TSharedPtr<SDockTab> DetailsTab = LevelEditorTabManager->TryInvokeTab(FRewindDebuggerModule::DetailsTabName );

		if (DetailsTab.IsValid())
		{
			UpdateDetailsPanel(DetailsTab.ToSharedRef());
		}
	}
}

void FRewindDebugger::UpdateDetailsPanel(TSharedRef<SDockTab> DetailsTab)
{
	if (bIsDetailsPanelOpen)
	{
		if (!DetailsPanelWidget.IsValid())
		{
			DetailsPanelWidget = SNew(SBox);
		}

		if (DetailsTab->GetContent() != DetailsPanelWidget)
		{
			DetailsTab->SetContent(DetailsPanelWidget.ToSharedRef());
		}
		
		TSharedPtr<SWidget> DetailsView;

		if (SelectedTrack)
		{
			DetailsView = SelectedTrack->GetDetailsView();
		}

		if (!DetailsView.IsValid())
		{
			if (EmptyDetails == nullptr)
			{
				EmptyDetails = SNew(SSpacer);
			}
			DetailsView = EmptyDetails;
		}
		
		DetailsPanelWidget->SetContent(DetailsView.ToSharedRef());
	}
}

void FRewindDebugger::RegisterTrackContextMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->FindMenu(FRewindDebuggerModule::TrackContextMenuName);

	FToolMenuSection& Section = Menu->FindOrAddSection("SelectedTrack");

	Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			const URewindDebuggerTrackContextMenuContext* Context = InSection.FindContext<URewindDebuggerTrackContextMenuContext>();
			if (Context && Context->SelectedTrack.IsValid())
			{
				Context->SelectedTrack->BuildContextMenu(InSection);
			}
		}));
}

void FRewindDebugger::MakeOtherWorldsMenu(UToolMenu* Menu)
{
	const FRewindDebugger* RewindDebugger = Instance();

	FToolMenuSection& Section = Menu->AddSection("Other Worlds", LOCTEXT("Other Worlds", "Other Worlds"));

	if (const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession())
	{
		const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider");

		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
		GameplayProvider->EnumerateWorlds([GameplayProvider, &Section](const FWorldInfo& WorldInfo)
			{
				const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(WorldInfo.Id);
				FString Name = ObjectInfo->Name;

				if (WorldInfo.NetMode == FWorldInfo::ENetMode::DedicatedServer)
				{
					return;
				}
				else if (WorldInfo.Type == FWorldInfo::EType::Game || WorldInfo.Type == FWorldInfo::EType::PIE)
				{
					return;
				}
				else
				{
					if (WorldInfo.Type == FWorldInfo::EType::Editor)
					{
						Name = Name + " (Editor)";
					}
					else if (WorldInfo.Type == FWorldInfo::EType::Inactive)
					{
						Name = Name + " (Editor)";
					}
					else if (WorldInfo.Type == FWorldInfo::EType::EditorPreview)
					{
						Name = Name + " (Editor Preview)";
					}
					else if (WorldInfo.Type == FWorldInfo::EType::GamePreview)
					{
						Name = Name + " (Game Preview)";
					}
					else if (WorldInfo.Type == FWorldInfo::EType::GameRPC)
					{
						Name = Name + " (Game RPC)";
					}
				}

				Section.AddMenuEntry(FName(ObjectInfo->Name, WorldInfo.Id),
					FText::FromString(Name),
					FText(),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([World = WorldInfo.Id]()
						{
							Instance()->SetDisplayWorld(World);
						}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([World = WorldInfo.Id]()
							{
								return Instance()->DisplayWorldId == World;
							})),
					EUserInterfaceActionType::Check
				);

			});
	}
}

void FRewindDebugger::SetDisplayWorld(uint64 WorldId)
{
	DisplayWorldId = WorldId;

	UE::RewindDebugger::IterateExtensions([this](IRewindDebuggerExtension* Extension)
		{
			Extension->Clear(this);
			Extension->Update(0.0, this);
		});
}

void FRewindDebugger::MakeWorldsMenu(UToolMenu* Menu)
{
	const FRewindDebugger* RewindDebugger = Instance();

	FToolMenuSection& ServerWorldsSection = Menu->AddSection("Server Worlds", LOCTEXT("Server", "Server"));
	FToolMenuSection& GameWorldsSection = Menu->AddSection("Game Worlds", LOCTEXT("Game Worlds", "Game Worlds"));
	FToolMenuSection& OtherWorldsSection = Menu->AddSection("Other Worlds", LOCTEXT("Other Worlds", "Other Worlds"));

	OtherWorldsSection.AddSubMenu("Other Worlds",
		LOCTEXT("Other Worlds", "Other Worlds"),
		LOCTEXT("Other Worlds Tooltip", "Additional worlds such as  Editor Preview worlds"),
		FNewToolMenuChoice(
			FNewToolMenuDelegate::CreateStatic(MakeOtherWorldsMenu)
		));

	if (const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession())
	{
		const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider");
		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
		GameplayProvider->EnumerateWorlds([GameplayProvider, &GameWorldsSection, &OtherWorldsSection, &ServerWorldsSection](const FWorldInfo& WorldInfo)
			{
				const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(WorldInfo.Id);
				if (ObjectInfo == nullptr)
				{
					UE_LOGF(LogRewindDebugger, Error, "Unable to find Object information associated to the provided World Id. Might be caused by missing data in the recording.");
					return;
				}
				FString Name = ObjectInfo->Name;

				FToolMenuSection* Section = &OtherWorldsSection;

				if (WorldInfo.NetMode == FWorldInfo::ENetMode::DedicatedServer)
				{
					Section = &ServerWorldsSection;
					Name = Name + " (Server)";
				}
				else if (WorldInfo.Type == FWorldInfo::EType::Game || WorldInfo.Type == FWorldInfo::EType::PIE)
				{
					Section = &GameWorldsSection;
					if (WorldInfo.NetMode == FWorldInfo::ENetMode::Client && WorldInfo.PIEInstanceId >= 0)
					{
						Name = Name + " (Client " + FString::FromInt(WorldInfo.PIEInstanceId) + ")";
					}
					if (WorldInfo.NetMode == FWorldInfo::ENetMode::Standalone && WorldInfo.PIEInstanceId >= 0)
					{
						Name = Name + " (Standalone " + FString::FromInt(WorldInfo.PIEInstanceId) + ")";
					}
				}
				else
				{
					return;
				}

				Section->AddMenuEntry(FName(ObjectInfo->Name, WorldInfo.Id),
					FText::FromString(Name),
					FText(),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([World = WorldInfo.Id]()
						{
							Instance()->SetDisplayWorld(World);
						}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([World = WorldInfo.Id]()
							{
								return Instance()->DisplayWorldId == World;
							})),
					EUserInterfaceActionType::Check
				);
			});
	}
}

void FRewindDebugger::RegisterPreviewMenu()
{
	if (!UToolMenus::Get()->IsMenuRegistered(FRewindDebuggerModule::PreviewMenuName))
	{
		UToolMenu* PreviewMenu = UToolMenus::Get()->RegisterMenu(FRewindDebuggerModule::PreviewMenuName);
		PreviewMenu->AddDynamicSection("Worlds", FNewSectionConstructChoice(FNewToolMenuDelegate::CreateStatic(&FRewindDebugger::MakeWorldsMenu)));
	}
}

void FRewindDebugger::RegisterToolBar()
{
	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(FRewindDebuggerModule::MainToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);

	const FRewindDebuggerCommands& Commands = FRewindDebuggerCommands::Get();

	// Play back controls
	{
		FToolMenuSection& PlaybackSection = Menu->FindOrAddSection("PlaybackControls");
		PlaybackSection.AddEntry(FToolMenuEntry::InitToolBarButton(
			Commands.FirstFrame,
			FText(),
			TAttribute<FText>(),
			FSlateIcon("RewindDebuggerStyle", "RewindDebugger.FirstFrame")));

		PlaybackSection.AddEntry(FToolMenuEntry::InitToolBarButton(
			Commands.PreviousFrame,
			FText(),
			TAttribute<FText>::CreateLambda([]()
				{
					FText Override;
					if (Instance()->SelectedTrack.IsValid())
					{
						Override = Instance()->SelectedTrack->GetStepCommandTooltip(RewindDebugger::EStepMode::Backward);
					}

					return Override.IsEmpty() ? FRewindDebuggerCommands::Get().PreviousFrame->GetDescription() : Override;
				}),
			FSlateIcon("RewindDebuggerStyle", "RewindDebugger.PreviousFrame")));

		PlaybackSection.AddEntry(FToolMenuEntry::InitToolBarButton(
			Commands.ReversePlay,
			FText(),
			TAttribute<FText>(),
			FSlateIcon("RewindDebuggerStyle", "RewindDebugger.ReversePlay")));

		PlaybackSection.AddEntry(FToolMenuEntry::InitToolBarButton(
			Commands.Pause,
			FText(),
			FText::Format(LOCTEXT("PauseButtonTooltip", "{0} ({1})"), Commands.Pause->GetDescription(), Commands.PauseOrPlay->GetInputText()),
			FSlateIcon("RewindDebuggerStyle", "RewindDebugger.Pause")));

		PlaybackSection.AddEntry(FToolMenuEntry::InitToolBarButton(
			Commands.Play,
			FText(),
			FText::Format(LOCTEXT("PlayButtonTooltip", "{0} ({1})"), Commands.Play->GetDescription(), Commands.PauseOrPlay->GetInputText()),
			FSlateIcon("RewindDebuggerStyle", "RewindDebugger.Play")));

		FToolMenuEntry& PlayBackOptions = PlaybackSection.AddEntry(FToolMenuEntry::InitComboButton(
			"PlaybackOptions",
			FUIAction(),
			FNewToolMenuChoice(FNewToolMenuWidget::CreateLambda([](const FToolMenuContext& Context)
				{
					static const FName MenuName = "RewindDebugger.Playback.OptionsMenu";
					
					if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
					{
						UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);
						FToolMenuSection& OptionsSection = Menu->FindOrAddSection("PlaybackSpeed", LOCTEXT("PlaybackSpeedLabel", "Playback Speed"));
						
						// Preset play rate options
						for (const float Rate : { 0.1f, 0.25f, 0.5f, 1.0f, 2.0f })
						{
							const FText RateLabel = FText::FromString(FString::SanitizeFloat(Rate));
							OptionsSection.AddEntry(
								FToolMenuEntry::InitMenuEntry(
									FName(*FString::SanitizeFloat(Rate)),
									RateLabel,
									FText::Format(LOCTEXT("SetPlaybackSpeedTooltipFmt", "Set playback speed to {0}"), RateLabel),
									FSlateIcon(),
									FUIAction(
										FExecuteAction::CreateLambda([Rate]()
											{
												URewindDebuggerSettings::Get().PlaybackRate = Rate;
											}
										),
										FCanExecuteAction(),
										FIsActionChecked::CreateLambda([Rate]()
											{
												return FMath::IsNearlyEqual(URewindDebuggerSettings::Get().PlaybackRate, Rate);
											}
										)
									),
									EUserInterfaceActionType::RadioButton
								)
							);
						}
						
						// Manually set the play rate
						OptionsSection.AddEntry(
							FToolMenuEntry::InitWidget(
								"ManuallySetPlayRate",
								SNew(SNumericEntryBox<float>)
								.Value_Lambda([]()
									{
										return URewindDebuggerSettings::Get().PlaybackRate;
									})
								.OnValueChanged_Lambda([](float Value)
									{
										URewindDebuggerSettings::Get().PlaybackRate = Value;
									}),
								FText::GetEmpty(),
								true, false, true
							)
						);
					}
					
					return UToolMenus::Get()->GenerateWidget(MenuName, Context);
				})
			),
			FText::GetEmpty(),
			LOCTEXT("PlaybackOptionsTooltip", "Playback options")
		));
		PlayBackOptions.StyleNameOverride = FName("RewindDebugger.Toolbar.Primary");
		PlayBackOptions.ToolBarData.bSimpleComboBox = true;

		PlaybackSection.AddEntry(FToolMenuEntry::InitToolBarButton(
			Commands.NextFrame,
			FText(),
			TAttribute<FText>::CreateLambda([]()
				{
					FText Override;
					if (Instance()->SelectedTrack.IsValid())
					{
						Override = Instance()->SelectedTrack->GetStepCommandTooltip(RewindDebugger::EStepMode::Forward);
					}

					return Override.IsEmpty() ? FRewindDebuggerCommands::Get().NextFrame->GetDescription() : Override;
				}),
			FSlateIcon("RewindDebuggerStyle", "RewindDebugger.NextFrame")));

		PlaybackSection.AddEntry(FToolMenuEntry::InitToolBarButton(
			Commands.LastFrame,
			FText(),
			TAttribute<FText>(),
			FSlateIcon("RewindDebuggerStyle", "RewindDebugger.LastFrame")));
	}
	
	// Trace actions
	{
		FToolMenuSection& TraceSection = Menu->FindOrAddSection("TraceActions");
		TraceSection.AddEntry(FToolMenuEntry::InitToolBarButton(
			Commands.OpenTrace,
			FText(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FolderOpen")));

		TraceSection.AddEntry(FToolMenuEntry::InitToolBarButton(
			Commands.ClearAnalysis,
			FText(),
			TAttribute<FText>(),
			FSlateIcon("RewindDebuggerStyle", "RewindDebugger.ClearAnalysis")));	
	}

	// PIE Controls
	{
		FToolMenuSection& PIESection = Menu->FindOrAddSection("PIEControls", FText::GetEmpty(), FToolMenuInsert("PlaybackControls", EToolMenuInsertType::Before));
		PIESection.AddDynamicEntry("PIEControls", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				const TValueOrError<bool, FText> AuthorizationResult = IPIEAuthorizer::IsPIEAuthorized(/*bIsSimulateInEditor*/false);
				if (AuthorizationResult.HasValue()
					&& AuthorizationResult.GetValue())
				{
					const FRewindDebuggerCommands& Commands = FRewindDebuggerCommands::Get();

					InSection.AddEntry(FToolMenuEntry::InitToolBarButton(
						Commands.AutoEject,
						FText(),
						TAttribute<FText>(),
						FSlateIcon("RewindDebuggerStyle", "RewindDebugger.AutoEject")));

					InSection.AddEntry(FToolMenuEntry::InitToolBarButton(
						Commands.AutoRecord,
						FText(),
						TAttribute<FText>(),
						FSlateIcon("RewindDebuggerStyle", "RewindDebugger.AutoRecord")));

					InSection.AddSeparator(NAME_None);
				}
			}));
	}

	// Preview world selector — sits after TraceActions
	{
		RegisterPreviewMenu();

		FToolMenuSection& PreviewSection = Menu->FindOrAddSection("PreviewControls", FText::GetEmpty(), FToolMenuInsert("TraceActions", EToolMenuInsertType::After));
		PreviewSection.AddEntry(FToolMenuEntry::InitComboButton(
			"PreviewWorld",
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction::CreateLambda([]() { return !Instance()->IsPIESimulating(); })
			),
			FNewToolMenuChoice(FNewToolMenuWidget::CreateLambda([](const FToolMenuContext& Context)
				{
					return UToolMenus::Get()->GenerateWidget(FRewindDebuggerModule::PreviewMenuName, Context);
				}
			)),
			LOCTEXT("DisplayWorldLabel", "Preview"),
			LOCTEXT("DisplayWorldTooltip", "Select which world to use when spawning preview objects (e.g. Skeletal Meshes) from a trace file"),
			FSlateIcon("RewindDebuggerStyle", "RewindDebugger.Preview")
		));
	}
	
	// Categories section — populated by the Visual Logger plugin with per-category record/display filters.
	{
		FToolMenuSection& CategoriesSection = Menu->FindOrAddSection("CategoriesControls");
		CategoriesSection.AddEntry(FToolMenuEntry::InitComboButton(
			"Categories",
			FUIAction(),
			FNewToolMenuChoice(FNewToolMenuWidget::CreateLambda([](const FToolMenuContext& Context)
				{
					return UToolMenus::Get()->GenerateWidget(FRewindDebuggerModule::CategoriesMenuName, Context);
				}
			)),
			LOCTEXT("CategoriesLabel", "Categories"),
			LOCTEXT("CategoriesTooltip", "Filter which log categories are recorded and displayed in the viewport"),
			FSlateIcon("RewindDebuggerStyle", "RewindDebugger.Filter")
		));
	}

	Menu->SetStyleSet(&FRewindDebuggerStyle::Get());
	Menu->StyleName = "RewindDebugger.Toolbar";
	
	// Right toolbar
	UToolMenu* RightMenu = UToolMenus::Get()->RegisterMenu(FRewindDebuggerModule::RightToolBarName, NAME_None, EMultiBoxType::ToolBar);
	FToolMenuSection& RightSection = RightMenu->FindOrAddSection("PlaybackControls");
	RightSection.AddEntry(FToolMenuEntry::InitToolBarButton(
		Commands.AutoScroll,
		FText(),
		TAttribute<FText>(),
		FSlateIcon("RewindDebuggerStyle", "RewindDebugger.AutoScroll"))
	);
	
	RightMenu->SetStyleSet(&FRewindDebuggerStyle::Get());
	RightMenu->StyleName = "RewindDebugger.Toolbar";
}

void FRewindDebugger::TrackDoubleClicked(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InSelectedTrack)
{
	if (!InSelectedTrack.IsValid())
	{
		return;
	}

	SelectedTrack = InSelectedTrack;
	SelectedTrack->HandleDoubleClick();
}

TSharedPtr<SWidget> FRewindDebugger::BuildTrackContextMenu() const
{
	URewindDebuggerTrackContextMenuContext* MenuContext = NewObject<URewindDebuggerTrackContextMenuContext>();
	MenuContext->SelectedObject = GetSelectedObject();
	MenuContext->SelectedTrack = SelectedTrack;

	if (SelectedTrack.IsValid())
	{
		// build a list of class hierarchy names to make it easier for extensions to enable menu entries by type
		if (const TraceServices::IAnalysisSession* Session = GetAnalysisSession())
		{
			const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider");
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

			const FObjectInfo& ObjectInfo = GameplayProvider->GetObjectInfo(SelectedTrack->GetAssociatedObjectId());
			uint64 ClassId = ObjectInfo.ClassId;
			while (ClassId != 0)
			{
				const FClassInfo& ClassInfo = GameplayProvider->GetClassInfo(ClassId);
				MenuContext->TypeHierarchy.Add(ClassInfo.Name);
				ClassId = ClassInfo.SuperId;
			}
		}
	}

	return UToolMenus::Get()->GenerateWidget(FRewindDebuggerModule::TrackContextMenuName, FToolMenuContext(MenuContext));
}

TSharedPtr<FDebugObjectInfo> FRewindDebugger::GetSelectedObject() const
{
	if (SelectedTrack.IsValid())
	{
		if (!SelectedObject.IsValid())
		{
			SelectedObject = MakeShared<FDebugObjectInfo>();
		}

		SelectedObject->Id = SelectedTrack->GetAssociatedObjectId();
		SelectedObject->ObjectName = SelectedTrack->GetDisplayName().ToString();
		return SelectedObject;
	}

	return TSharedPtr<FDebugObjectInfo>();
}

void FRewindDebugger::SetObjectToDebug(const RewindDebugger::FObjectId ObjectId)
{
	if (IsObjectCurrentlyDebugged(ObjectId.GetMainId()))
	{
		return;
	}

	if (GetAnalysisSession() == nullptr)
	{
		UE_LOGF(LogRewindDebugger, Log, "Unable to set the object to debug since there is no active session");
		return;
	}

	CandidateIds = {ObjectId};

	RefreshDebugTracks();
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FRewindDebugger::GetSelectedTrack() const
{
	return SelectedTrack;
}

void FRewindDebugger::SelectTrack(RewindDebugger::FObjectId ObjectId)
{
	using namespace RewindDebugger;
	for (const TSharedPtr<FRewindDebuggerTrack>& Track : DebugTracks)
	{
		if (FRewindDebuggerTrack::Visit(Track, [this, ObjectId](const TSharedPtr<FRewindDebuggerTrack>& Track)
			{
				if (Track->GetAssociatedObjectId() == ObjectId)
				{
					TrackSelectionChanged(Track);
					return FRewindDebuggerTrack::EVisitResult::Break;
				}

				return FRewindDebuggerTrack::EVisitResult::Continue;
			}) == FRewindDebuggerTrack::EVisitResult::Break)
		{
			break;
		}
	}
}

// build a tree that's compatible with the public api from 5.0 for GetDebuggedObjects.
void FRewindDebugger::RefreshDebuggedObjects(TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>& InTracks, TArray<TSharedPtr<FDebugObjectInfo>>& OutObjects)
{
	using namespace RewindDebugger;

	OutObjects.SetNum(0, EAllowShrinking::No);
	for (const TSharedPtr<FRewindDebuggerTrack>& Track : InTracks)
	{
		FRewindDebuggerTrack::Visit(Track, [&OutObjects](const TSharedPtr<FRewindDebuggerTrack>& Track)
		{
			OutObjects.Add(MakeShared<FDebugObjectInfo>(Track->GetAssociatedObjectId(), Track->GetDisplayName().ToString()));
			return FRewindDebuggerTrack::EVisitResult::Continue;
		});
	}
}

TArray<TSharedPtr<FDebugObjectInfo>>& FRewindDebugger::GetDebuggedObjects()
{
	RefreshDebuggedObjects(DebugTracks, DebuggedObjects);
	return DebuggedObjects;
}

bool FRewindDebugger::IsObjectCurrentlyDebugged(uint64 InObjectId) const
{
	using namespace RewindDebugger;
	for (const TSharedPtr<FRewindDebuggerTrack>& Track : DebugTracks)
	{
		if (FRewindDebuggerTrack::Visit(Track, [InObjectId](TSharedPtr<FRewindDebuggerTrack> Track)
			{
				if (Track->GetUObjectId() == InObjectId)
				{
					return FRewindDebuggerTrack::EVisitResult::Break;
				}

				return FRewindDebuggerTrack::EVisitResult::Continue;
			}) == FRewindDebuggerTrack::EVisitResult::Break)
		{
			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
