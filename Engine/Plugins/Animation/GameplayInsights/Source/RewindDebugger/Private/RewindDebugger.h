// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7

#include "BindableProperty.h"
#include "Containers/Ticker.h"
#include "IRewindDebugger.h"
#include "IRewindDebuggerTrackCreator.h"
#include "RewindDebuggerTrack.h"
#include "TraceSessionsManager.h"
#include "Widgets/Layout/SBox.h"

namespace TraceServices
{
class IAnalysisSession;
}

namespace UE::RewindDebugger
{
struct FDebuggerRecordingControls;
}

namespace UE::TraceBasedDebuggers
{
struct FSessionInfo;
}

class FMenuBuilder;
class IGameplayProvider;
class SDockTab;
class SWidget;
class USkeletalMeshComponent;
struct FObjectInfo;

/**
 * Singleton class that handles the logic for the Rewind Debugger:
 * - Playback/Scrubbing state
 * - Start/Stop recording
 * - Keeping track of the current Debug Target object, and outputting a list of its sub objects/elements for the UI
 */
class FRewindDebugger : public IRewindDebugger
{
public:
	FRewindDebugger();
	virtual ~FRewindDebugger();

	//~ IRewindDebugger interface
	virtual double CurrentTraceTime() const override
	{
		return TraceTime.Get();
	}

	virtual double GetScrubTime() const override
	{
		return ScrubTimeInformation.ElapsedTime;
	}

	virtual void SetScrubTime(double Time) override
	{
		// Clamp to the recording range so the interface contract holds; SetCurrentScrubTime stores
		// ElapsedTime verbatim and would otherwise let callers push the cursor out of range.
		const double ClampedTime = FMath::Clamp(Time, 0.0, RecordingDuration.Get());
		SetCurrentScrubTime(ClampedTime, EAutoScrollChange::Disable);
		TrackCursorDelegate.ExecuteIfBound(/*bReverse*/false);
	}

	virtual void CenterViewOnTime(double Time) override
	{
		CenterViewOnTimeDelegate.ExecuteIfBound(Time);
	}

	virtual const TRange<double>& GetCurrentTraceRange() const override
	{
		return CurrentTraceRange;
	}

	virtual const TRange<double>& GetCurrentViewRange() const override
	{
		return CurrentViewRange;
	}

	virtual TSharedPtr<UE::TraceBasedDebuggers::FSessionInfo> GetSelectedDebugSessionInfo() const override;
	virtual const TraceServices::IAnalysisSession* GetAnalysisSession() const override;
	virtual TSharedPtr<const TraceServices::IAnalysisSession> GetAnalysisSessionAsShared() const override;
	virtual uint64 GetRootObjectId() const override;
	virtual bool GetRootObjectPosition(FVector& OutPosition) const override;
	virtual void SetRootObjectPosition(const TOptional<FVector>& InPosition) override;
	virtual UWorld* GetWorldToVisualize() const override;
	virtual bool IsRecording() const override;
	virtual bool IsPIESimulating() const override
	{
		return bPIESimulating;
	}
	bool IsPIEStarted() const
	{
		return bPIEStarted;
	}

	virtual double GetRecordingDuration() const override
	{
		return RecordingDuration.Get();
	}

	virtual TSharedPtr<FDebugObjectInfo> GetSelectedObject() const override;
	virtual TSharedPtr<RewindDebugger::FRewindDebuggerTrack> GetSelectedTrack() const override;
	virtual void SelectTrack(RewindDebugger::FObjectId ObjectId) override;
	virtual void SetObjectToDebug(RewindDebugger::FObjectId ObjectId) override;
	virtual TArray<TSharedPtr<FDebugObjectInfo>>& GetDebuggedObjects() override;
	virtual bool IsObjectCurrentlyDebugged(uint64 ObjectId) const override;
	virtual bool ShouldDisplayWorld(uint64 WorldId) override
	{
		return DisplayWorldId == WorldId;
	}

	/** @return Whether the debugger is currently analyzing a trace session from the current process (e.g., PIE) */
	bool IsAnalyzingLocalSession() const;

	/** @return Whether the debugger is currently analyzing a remote trace session (e.g., external process like standalone game, client, server) */
	bool IsAnalyzingRemoteSession() const;

	/** @return Whether the debugger is currently analyzing a trace file */
	bool IsAnalyzingTraceFile() const;

	void GetTargetObjectIds(TArray<RewindDebugger::FObjectId>& OutTargetObjectIds) const;

	TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>& GetTracks()
	{
		return DebugTracks;
	}

	/** create singleton instance */
	static void Initialize();

	/** destroy singleton instance */
	static void Shutdown();

	/** get singleton instance */
	static FRewindDebugger* Instance()
	{
		return static_cast<FRewindDebugger*>(InternalInstance);
	}

	virtual bool CanStartRecording() const override;

	/** Starts a new recording for the local session */
	virtual void StartRecording() const override;

	void StartRecordingLocalSession() const;
	bool CanStartRecordingLocalSession() const;
	bool IsRecordingLocalSession() const;

	void OnClearRecording();

	bool CanOpenTrace() const;
	void OpenTrace(const FString& FilePath);
	void OpenTrace();

	/** @return Whether there is an active analysis session to close and/or tracks data to clear */
	bool CanClearAnalysis() const;

	/** Closes the currently selected analysis session and clears debugger and tracks data */
	void ClearAnalysis();

	/** Clears all debugger and track data associated to the current, or last, analysis session */
	void ClearTrackData();

	void ToggleAutoScroll();
	bool CanEnableAutoScroll() const;
	bool IsAutoScrollEnabled() const;

	bool ShouldAutoRecordOnPIE() const;
	void SetShouldAutoRecordOnPIE(bool InValue);

	bool ShouldAutoEject() const;
	void SetShouldAutoEject(bool InValue);

	void StopRecordingLocalSession() const;
	bool CanStopRecordingLocalSession() const
	{
		return IsRecordingLocalSession();
	}

	//~ Playback controls

	bool CanPause() const;
	void Pause();

	bool CanPlay() const;
	void Play();
	bool IsPlaying() const;

	bool CanPlayReverse() const;
	void PlayReverse();

	void ScrubToStart();
	void ScrubToEnd();
	void Step(int32 InNumberOfFrames);
	void StepForward();
	void StepBackward();

	bool CanScrub() const;
	void ScrubToTime(double ScrubTime, bool bIsScrubbing);

	/** Tick function: While recording, update recording duration.  While paused, and we have recorded data, update skinned mesh poses for the current frame, and handle playback. */
	void Tick(float DeltaTime);

	/** update the list of tracks for the currently selected debug target */
	void RefreshDebugTracks();

	void SetCurrentViewRange(const TRange<double>& Range);

	DECLARE_DELEGATE(FOnTrackListChanged)
	void SetTrackListChangedDelegate(const FOnTrackListChanged& InTrackListChangedDelegate);

	void TrackDoubleClicked(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InSelectedTrack);
	void TrackSelectionChanged(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InSelectedTrack);
	void ClearTrackSelection();

	TSharedPtr<SWidget> BuildTrackContextMenu() const;

	void UpdateDetailsPanel(TSharedRef<SDockTab> DetailsTab);
	static void RegisterTrackContextMenu();
	static void MakeOtherWorldsMenu(class UToolMenu* Menu);
	void SetDisplayWorld(uint64 WorldId);
	static void MakeWorldsMenu(class UToolMenu* Menu);
	static void RegisterPreviewMenu();
	static void RegisterToolBar();

	DECLARE_DELEGATE_OneParam(FOnTrackCursor, bool)
	void SetTrackCursorDelegate(const FOnTrackCursor& InTrackCursorDelegate);

	DECLARE_DELEGATE_OneParam(FOnCenterViewOnTime, double)
	void SetCenterViewOnTimeDelegate(const FOnCenterViewOnTime& InCenterViewOnTimeDelegate)
	{ 
		CenterViewOnTimeDelegate = InCenterViewOnTimeDelegate;
	}

	TSharedPtr<SBox> DetailsPanelWidget;
	TSharedPtr<SWidget> EmptyDetails;

	TBindableProperty<double>* GetTraceTimeProperty()
	{
		return &TraceTime;
	}

	TBindableProperty<double>* GetRecordingDurationProperty()
	{
		return &RecordingDuration;
	}

	TBindableProperty<FString, BindingType_Out>* GetRootObjectNameProperty()
	{
		return &RootObjectName;
	}

	UE_DEPRECATED(5.7, "Use GetRootObjectNameProperty instead")
	TBindableProperty<FString, BindingType_Out>* GetDebugTargetActorProperty()
	{
		return GetRootObjectNameProperty();
	}

	virtual void OpenDetailsPanel() override;
	void SetIsDetailsPanelOpen(bool bIsOpen)
	{
		bIsDetailsPanelOpen = bIsOpen;
	}

	virtual const FObjectInfo* FindTypedOuterInfo(TNotNull<const UStruct*> InType, TNotNull<const IGameplayProvider*> InGameplayProvider, uint64 InObjectId) const override;

	TArrayView<RewindDebugger::FRewindDebuggerTrackType> GetTrackTypes()
	{
		return TrackTypes;
	};

#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS
public:
	bool ConnectToLiveSession(uint32 SessionID, FStringView InSessionAddress);
	bool ConnectToLiveSession_Direct(const FGuid& InRemoteSessionID);
	bool ConnectToLiveSession_Relay(const FGuid& InRemoteSessionID);

	/**
	 * Finds the trace session analysis linked to the provided remote session ID,
	 * then calls ClearAnalysisSession.
	 * @see ClearAnalysisSession
	 */
	void ClearAnalysisSessionLinkedToRemoteSessionID(const FGuid& InRemoteSessionID);

	const UE::TraceBasedDebuggers::FTraceSessionDescriptor& GetCurrentSessionDescriptor() const
	{
		return SelectedDescriptor;
	}

private:
	/** Sets the analysis session as the one used by the debugger */
	void SetSelectedAnalysisSession(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& InSessionDescriptor);

	/**
	 * Adds the provided session descriptor to the list of active analysis sessions,
	 * sets the selected analysis session that the debugger will represent and notifies extensions.
	 * @see SetSelectedAnalysisSession
	 */
	void OpenAnalysisSession(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& InSessionDescriptor);

	/**
	 * Stops the trace analysis session associated to the remote session specified in the
	 * provided session descriptor and notifies the extensions.
	 * @note Unlike ClearAnalysisSession this method will keep the session available by not removing it
	 * from the Trace Sessions Manager.
	 * The current session selection is not affected.
	 * @see ClearAnalysisSession
	 */
	void StopAnalysisSession(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& InSessionDescriptor);

	/**
	 * Closes the trace analysis session (i.e., stop and remove from the Trace Sessions Manager)
	 * associated to the remote session specified in the provided session descriptor,
	 * notifies the extensions and removes the session from the active sessions.
	 * Clears the current selected session, if applicable.
	 * @note Consider using StopAnalysisSession if the session information still need to be accessed after stopping the analysis.
	 * @see StopAnalysisSession
	 * @see SetSelectedAnalysisSession
	 */
	void ClearAnalysisSession(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& InSessionDescriptor);

	void HandleRecordingStarted(const TSharedPtr<UE::TraceBasedDebuggers::FSessionInfo>& InSessionInfo);
	void HandleRecordingStopped(const TSharedPtr<UE::TraceBasedDebuggers::FSessionInfo>& InSessionInfo);
	void HandleSessionSelected(const TSharedPtr<UE::TraceBasedDebuggers::FSessionInfo>& InSessionInfo);
#endif // UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS

private:

	void ClearActiveAnalysisSession();

#if WITH_TRACE_BASED_DEBUGGERS
	void StopLocalRecording() const;
#endif

	static void RefreshDebuggedObjects(TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>& InTracks, TArray<TSharedPtr<FDebugObjectInfo>>& OutObjects);

	void OnTrackListChanged();

	void OnPIEStarted(bool bSimulating);
	void OnPIEPaused(bool bSimulating);
	void OnPIEResumed(bool bSimulating);
	void OnPIEStopped(bool bSimulating);
	void OnPIESingleStepped(bool bSimulating);
	void OnSIESwitch(bool bSimulating);

	bool CanUsePlaybackControls() const;

	FDelegateHandle PreBeginPIEHandle;
	FDelegateHandle PausePIEHandle;
	FDelegateHandle ResumePIEHandle;
	FDelegateHandle SingleStepPIEHandle;
	FDelegateHandle ShutdownPIEHandle;
	FDelegateHandle SIESwitchHandle;

	FDelegateHandle ClearRecordingHandle;

	enum class EAutoScrollChange : int8
	{
		PreserveCurrentState,
		Disable
	};

	void SetCurrentScrubTime(double Time, EAutoScrollChange AutoScrollChange);

	TBindableProperty<double> TraceTime;
	TBindableProperty<double> RecordingDuration;
	TBindableProperty<FString, BindingType_Out> RootObjectName;

	enum class EControlState : int8
	{
		Play,
		PlayReverse,
		Pause
	};

	EControlState ControlState = EControlState::Pause;

	FOnTrackListChanged TrackListChangedDelegate;
	FOnTrackCursor TrackCursorDelegate;
	FOnCenterViewOnTime CenterViewOnTimeDelegate;

	bool bPIEStarted = false;
	bool bPIESimulating = false;

	/** Instructs the debugger to always scrub to the last received event */
	bool bAutoScrollToLastItem = false;

	double PreviousTraceTime = -1;
	TRange<double> CurrentViewRange{ 0, 10 };
	TRange<double> CurrentTraceRange{ 0, 0 };
	uint16 RecordingIndex = 0;

	FScrubTimeInformation ScrubTimeInformation;
	FScrubTimeInformation LowerBoundViewTimeInformation;
	FScrubTimeInformation UpperBoundViewTimeInformation;

	static void GetScrubTimeInformation(double InDebugTime, FScrubTimeInformation& InOutTimeInformation, uint16 InRecordingIndex, const TraceServices::IAnalysisSession* AnalysisSession);

#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS
	/** List of session descriptors associated to active trace analysis sessions */
	TArray<UE::TraceBasedDebuggers::FTraceSessionDescriptor> ActiveAnalysisDescriptors;

	/**
	 * Trace analysis descriptor associated to the selected remote session
	 * An invalid descriptor indicates that no active recording/analysis is active
	 * for the currently selected session.
	 */
	UE::TraceBasedDebuggers::FTraceSessionDescriptor SelectedDescriptor;
#endif // UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS
	TArray<TSharedPtr<FDebugObjectInfo>> DebuggedObjects;
	mutable TSharedPtr<FDebugObjectInfo> SelectedObject;

	TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>> DebugTracks;
	TSharedPtr<RewindDebugger::FRewindDebuggerTrack> SelectedTrack;
	TSharedPtr<UE::RewindDebugger::FDebuggerRecordingControls> RecordingControls;

	TArray<RewindDebugger::FObjectId> CandidateIds;

	FTSTicker::FDelegateHandle TickerHandle;

	TOptional<FVector> RootObjectPosition;

	TArray<RewindDebugger::FRewindDebuggerTrackType> TrackTypes;

	bool bIsDetailsPanelOpen = true;

	uint64 DisplayWorldId = 0;
	bool bDisplayWorldIdValid = false;
};
