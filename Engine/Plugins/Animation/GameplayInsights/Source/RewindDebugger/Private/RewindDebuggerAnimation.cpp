// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerAnimation.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Common/ProviderLock.h"
#include "Editor.h"
#include "Engine/PoseWatch.h"
#include "IAnimationProvider.h"
#include "IGameplayProvider.h"
#include "Insights/IUnrealInsightsModule.h"
#include "RewindDebugger.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "SLevelViewport.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ObjectTrace.h"
#include "TraceServices/Model/Frames.h"
#include "Engine/SkeletalMesh.h"
#include "UObject/SoftObjectPath.h"

#define LOCTEXT_NAMESPACE "RewindDebuggerAnimation"

FRewindDebuggerAnimation* FRewindDebuggerAnimation::Instance;

FRewindDebuggerAnimation::FRewindDebuggerAnimation()
{
}

void FRewindDebuggerAnimation::ClearSpawnedComponents()
{
	for(auto& MeshComponentInfo : SpawnedMeshComponents)
	{
		if(MeshComponentInfo.Value.Actor.IsValid())
		{
			if(MeshComponentInfo.Value.Component)
			{
				MeshComponentInfo.Value.Component->UnregisterComponent();
				MeshComponentInfo.Value.Component->MarkAsGarbage();
				MeshComponentInfo.Value.Component = nullptr;
			}

			MeshComponentInfo.Value.Actor->Destroy();
			MeshComponentInfo.Value.Actor = nullptr;
		}
	}
	SpawnedMeshComponents.Empty();
	SpawnedAnimInstances.Empty();
}

void FRewindDebuggerAnimation::Clear(IRewindDebugger* RewindDebugger)
{
	ClearSpawnedComponents();
	LastScrubTime = -1;
}

void FRewindDebuggerAnimation::Initialize()
{
	FEditorDelegates::ResumePIE.AddRaw(this, &FRewindDebuggerAnimation::OnPIEResumed);
	FEditorDelegates::EndPIE.AddRaw(this, &FRewindDebuggerAnimation::OnPIEStopped);
	FEditorDelegates::SingleStepPIE.AddRaw(this, &FRewindDebuggerAnimation::OnPIESingleStepped);

	Instance = this;
}

void FRewindDebuggerAnimation::Shutdown()
{
	FEditorDelegates::ResumePIE.RemoveAll(this);
	FEditorDelegates::EndPIE.RemoveAll(this);
	FEditorDelegates::SingleStepPIE.RemoveAll(this);

	Instance = nullptr;
}

void FRewindDebuggerAnimation::OnPIEResumed(bool bSimulating)
{
	// restore all relative transforms of any meshes that may have been moved while scrubbing
	for (TTuple<uint64, FMeshComponentResetData>& MeshData : MeshComponentsToReset)
	{
		if (USkeletalMeshComponent* MeshComponent = MeshData.Value.Component.Get())
		{
			MeshComponent->SetRelativeTransform(MeshData.Value.RelativeTransform, false, nullptr, ETeleportType::TeleportPhysics);
			MeshComponent->SetForcedLOD(MeshData.Value.ForcedLod);
			MeshComponent->SetVisibility(MeshData.Value.bIsVisible);
		}
	}

	MeshComponentsToReset.Empty();
}

void FRewindDebuggerAnimation::OnPIESingleStepped(bool bSimulating)
{
	// restore all relative transforms of any meshes that may have been moved while scrubbing
	for (TTuple<uint64, FMeshComponentResetData>& MeshData : MeshComponentsToReset)
	{
		if (USkeletalMeshComponent* MeshComponent = MeshData.Value.Component.Get())
		{
			MeshComponent->SetRelativeTransform(MeshData.Value.RelativeTransform, false, nullptr, ETeleportType::TeleportPhysics);
		}
	}

	MeshComponentsToReset.Empty();
}


void FRewindDebuggerAnimation::OnPIEStopped(bool bSimulating)
{
	MeshComponentsToReset.Empty();
	
	// clear last scrub time so that poses will reapply
	LastScrubTime = -1;
}


void FRewindDebuggerAnimation::ApplyPoseToMesh(const IAnimationProvider* AnimationProvider, const IGameplayProvider* GameplayProvider, const TraceServices::FFrame& Frame,
	const IAnimationProvider::SkeletalMeshPoseTimeline& TimelineData, USkeletalMeshComponent* MeshComponent, uint64 ObjectId, bool bQueueForReset, bool bApplyMesh)
{
	const FSkeletalMeshPoseMessage * PoseMessage = nullptr;
	
	// Get last pose in frame
	TimelineData.EnumerateEvents(Frame.StartTime, Frame.EndTime,
		[&PoseMessage](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InPoseMessage)
		{
			PoseMessage = &InPoseMessage;
			return TraceServices::EEventEnumerate::Continue;
		});

	// Update mesh based on pose
	if (PoseMessage)
	{
		if (const FSkeletalMeshInfo* SkeletalMeshInfo = AnimationProvider->FindSkeletalMeshInfo(PoseMessage->MeshId))
		{
			if (bApplyMesh)
			{
				if (const FObjectInfo* SkeletalMeshObjectInfo = GameplayProvider->FindObjectInfo(PoseMessage->MeshId))
				{
					USkeletalMesh* SkeletalMesh = TSoftObjectPtr<USkeletalMesh>(FSoftObjectPath(SkeletalMeshObjectInfo->PathName)).LoadSynchronous();
										
					if (SkeletalMesh == nullptr)
					{
						// if the skeletal mesh asset was not found, try the skeleton asset preview mesh as a fallback
						if (SkeletalMeshInfo->SkeletonId != 0)
						{
							const FObjectInfo& SkeletonInfo = GameplayProvider->GetObjectInfo(SkeletalMeshInfo->SkeletonId);
						
							USkeleton* Skeleton = TSoftObjectPtr<USkeleton>(FSoftObjectPath(SkeletonInfo.PathName)).LoadSynchronous();
							if (Skeleton)
							{
								SkeletalMesh = Skeleton->GetPreviewMesh(true);
							}
						}
					}
				
					if(SkeletalMesh)
					{
						MeshComponent->SetSkeletalMesh(SkeletalMesh);
					}
				}
			}
			
			FTransform ComponentWorldTransform;
			AnimationProvider->GetSkeletalMeshComponentSpacePose(*PoseMessage, *SkeletalMeshInfo, ComponentWorldTransform, MeshComponent->GetEditableComponentSpaceTransforms());
			MeshComponent->ApplyEditedComponentSpaceTransforms();

			GameplayProvider->GetObjectTransform(ObjectId, Frame.StartTime, Frame.EndTime, ComponentWorldTransform);
			
			if (bQueueForReset)
			{
				if (MeshComponentsToReset.Find(ObjectId) == nullptr)
				{
					FMeshComponentResetData ResetData;
					ResetData.Component = MeshComponent;
					ResetData.RelativeTransform = MeshComponent->GetRelativeTransform();
					ResetData.ForcedLod = MeshComponent->GetForcedLOD();
					ResetData.bIsVisible = MeshComponent->GetVisibleFlag();
					MeshComponentsToReset.Add(ObjectId, ResetData);
				}
			}

			MeshComponent->SetWorldTransform(ComponentWorldTransform, false, nullptr, ETeleportType::TeleportPhysics);
			
			MeshComponent->SetForcedLOD(PoseMessage->LodIndex + 1);
			MeshComponent->UpdateLODStatus();
			MeshComponent->UpdateChildTransforms(EUpdateTransformFlags::None, ETeleportType::TeleportPhysics);
			MeshComponent->SetVisibility(PoseMessage->bIsVisible);
			MeshComponent->MarkRenderStateDirty();
		}
	}
}

FRewindDebuggerAnimation::FSpawnedMeshComponentInfo* FRewindDebuggerAnimation::SpawnMesh(uint64 ObjectId, const IGameplayProvider* GameplayProvider)
{
	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	check(RewindDebugger);
	if (UWorld* World = RewindDebugger->GetWorldToVisualize())
	{
		FSpawnedMeshComponentInfo* MeshComponentInfo = &SpawnedMeshComponents.Add(ObjectId);

		FActorSpawnParameters ActorSpawnParameters;
		ActorSpawnParameters.bHideFromSceneOutliner = true;
		ActorSpawnParameters.ObjectFlags |= RF_Transient;

		MeshComponentInfo->Actor = World->SpawnActor<AActor>(ActorSpawnParameters);

		if (const FObjectInfo* ActorInfo = RewindDebugger->FindTypedOuterInfo<AActor>(GameplayProvider, ObjectId))
		{
			MeshComponentInfo->Actor->SetActorLabel(FString(TEXT("RewindDebugger: ") + FString(ActorInfo->Name)));
		}

		FName ComponentName = NAME_None;
		if (const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(ObjectId))
		{
			ComponentName = ObjectInfo->Name;
		}
		
		MeshComponentInfo->Component = NewObject<USkeletalMeshComponent>(MeshComponentInfo->Actor.Get(), ComponentName);
		MeshComponentInfo->Component->PrimaryComponentTick.bStartWithTickEnabled = false;
		MeshComponentInfo->Component->PrimaryComponentTick.bCanEverTick = false;

		MeshComponentInfo->Actor->AddInstanceComponent(MeshComponentInfo->Component);

		MeshComponentInfo->Component->SetAnimationMode(EAnimationMode::AnimationCustomMode);
		MeshComponentInfo->Component->RegisterComponentWithWorld(World);
		return MeshComponentInfo;
	}
	return nullptr;
}

UAnimInstance* FRewindDebuggerAnimation::SpawnAnimInstance(uint64 ObjectId, const IGameplayProvider* GameplayProvider)
{
	FSpawnedAnimInstanceInfo* AnimInstanceInfo = SpawnedAnimInstances.Find(ObjectId);
	if (AnimInstanceInfo)
	{
		if (AnimInstanceInfo->AnimInstance.IsValid())
		{
			return AnimInstanceInfo->AnimInstance.Get();
		}
	}

	if (AnimInstanceInfo == nullptr)
	{
		AnimInstanceInfo = &SpawnedAnimInstances.Add(ObjectId);
	}

	if (const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(ObjectId))
	{
		if (const FClassInfo* ClassInfo = GameplayProvider->FindClassInfo(ObjectInfo->ClassId))
		{
			TSoftObjectPtr<UAnimBlueprintGeneratedClass> ClassPtr =	TSoftObjectPtr<UAnimBlueprintGeneratedClass>(FSoftObjectPath(FStringView(ClassInfo->PathName)));
			if(ClassPtr.LoadSynchronous())
			{
				if (const FSpawnedMeshComponentInfo* MeshInfo = SpawnedMeshComponents.Find(ObjectInfo->GetOuterUObjectId()))
				{
					AnimInstanceInfo->AnimInstance = NewObject<UAnimInstance>(MeshInfo->Component, ClassPtr.Get());
					return AnimInstanceInfo->AnimInstance.Get();
				}
			}
		}
	}

	SpawnedAnimInstances.Remove(ObjectId);
	return nullptr;
}

UAnimInstance* FRewindDebuggerAnimation::GetDebugAnimInstance(uint64 ObjectId)
{
	if (FSpawnedAnimInstanceInfo* FoundInfo = SpawnedAnimInstances.Find(ObjectId))
	{
		return FoundInfo->AnimInstance.Get();
	}
	return nullptr;
}


void FRewindDebuggerAnimation::Update(float DeltaTime, IRewindDebugger* RewindDebugger)
{
	check(RewindDebugger);

	if (RewindDebugger->IsPIESimulating() || RewindDebugger->GetRecordingDuration() == 0.0)
	{
		return;
	}

	if (const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession())
	{
		const double CurrentTraceTime = RewindDebugger->CurrentTraceTime();
		if (CurrentTraceTime != LastScrubTime)
		{
			bool bFrameFound;
			TraceServices::FFrame Frame;
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session);
				const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*Session);
				bFrameFound = FrameProvider.GetFrameFromTime(TraceFrameType_Game, CurrentTraceTime, Frame);
			}

			if (bFrameFound)
			{
				const IAnimationProvider* AnimationProvider = Session->ReadProvider<IAnimationProvider>("AnimationProvider");
				const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider");

				if (AnimationProvider && GameplayProvider)
				{
					const TArray<TSharedPtr<FDebugObjectInfo>>& DebuggedObjects = RewindDebugger->GetDebuggedObjects();
					bool bSearchForActor = !TargetActorIdForMesh.IsSet() || !DebuggedObjects.ContainsByPredicate(
						[Id = TargetActorIdForMesh](const TSharedPtr<FDebugObjectInfo>& ObjectInfo)
						{
							return ObjectInfo->Id == Id;
						});
					bool bNewActor = false;
					TargetActorPosition.Reset();

					// update pose on all SkeletalMeshComponents:
					// - enumerate all skeletal mesh pose timelines
					// - check if the corresponding mesh component still exists
					// - apply the recorded pose for the current Frame
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebugger::Tick_UpdatePoses);

						const bool ResetOrSpawnMeshes = !RewindDebugger->IsPIESimulating() && RewindDebugger->GetAnalysisSession() != nullptr;

						TraceServices::FProviderReadScopeLock AnimProviderReadScope(*AnimationProvider);
						TraceServices::FProviderReadScopeLock GameProviderReadScope(*GameplayProvider);
						AnimationProvider->EnumerateSkeletalMeshPoseTimelines(
						[this, ResetOrSpawnMeshes, RewindDebugger, &Frame, AnimationProvider, GameplayProvider, &bSearchForActor, &DebuggedObjects, &bNewActor](uint64 ObjectId, const IAnimationProvider::SkeletalMeshPoseTimeline& TimelineData)
						{
							if (bSearchForActor)
							{
								// until we have actor transforms traced out, the first (from a non-server) skeletal mesh component transform on the target actor be used as the actor position
								if (const FWorldInfo* WorldInfo = GameplayProvider->FindWorldInfoFromObject(ObjectId))
								{
									if (WorldInfo->NetMode != FWorldInfo::ENetMode::DedicatedServer)
									{
										if (const FObjectInfo* ActorInfo = RewindDebugger->FindTypedOuterInfo<AActor>(GameplayProvider, ObjectId))
										{
											if (DebuggedObjects.ContainsByPredicate([Id = ActorInfo->GetId()](const TSharedPtr<FDebugObjectInfo>& ObjectInfo)
												{
													return ObjectInfo->Id == Id;
												}))
											{
												bSearchForActor = false;
												bNewActor = true;
												TargetActorIdForMesh = ActorInfo->GetId();
												TargetActorMeshId = RewindDebugger::FObjectId{ ObjectId };
											}
										}
									}
								}
							}

							USkeletalMeshComponent* MeshComponent = nullptr;
							bool bQueueForReset = false;
							bool bLoadMesh = false;
#if OBJECT_TRACE_ENABLED
							if (ResetOrSpawnMeshes)
							{
								if(UObject* ObjectInstance = FObjectTrace::GetObjectFromId(ObjectId))
								{
									MeshComponent = Cast<USkeletalMeshComponent>(ObjectInstance);
									if (MeshComponent)
									{
										bQueueForReset = true;
									}
								}
							}
#endif // OBJECT_TRACE_ENABLED
							if (MeshComponent == nullptr)
							{
								// display pose on a spawned mesh component
								FSpawnedMeshComponentInfo* MeshComponentInfo = SpawnedMeshComponents.Find(ObjectId);
								if (MeshComponentInfo)
								{
									if (!MeshComponentInfo->Actor.IsValid())
									{
										// if the actor has been deleted, clear cached data and create it again
										SpawnedMeshComponents.Remove(ObjectId);
										MeshComponentInfo = nullptr;
									}
								}
									
								if (MeshComponentInfo == nullptr)
								{
									if (ResetOrSpawnMeshes)
									{
										if (const FWorldInfo* ObjectWorldInfo = GameplayProvider->FindWorldInfoFromObject(ObjectId))
										{
											if (RewindDebugger->ShouldDisplayWorld(ObjectWorldInfo->Id))
											{
												MeshComponentInfo = SpawnMesh(ObjectId, GameplayProvider);
											}
										}
									}
									bLoadMesh = true;
									LastScrubTime = -1;
								}

								if (MeshComponentInfo)
								{
									MeshComponent = MeshComponentInfo->Component;
								}
							}

							if (MeshComponent)
							{
								ApplyPoseToMesh(AnimationProvider, GameplayProvider, Frame, TimelineData, MeshComponent, ObjectId, bQueueForReset, bLoadMesh);
							}
						}
						);
					}

					{
						TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebuggerAnimation::UpdateActorPosition);

						// mark the target position as invalid for a frame when the actor changes, so it will be treated as a teleport by the camera system
						if (!bNewActor)
						{
							TraceServices::FProviderReadScopeLock AnimProviderReadScope(*AnimationProvider);
							AnimationProvider->ReadSkeletalMeshPoseTimeline(TargetActorMeshId.GetMainId(), [this, &Frame, bNewActor](const IAnimationProvider::SkeletalMeshPoseTimeline& TimelineData, bool bHasCurves)
							{
								const FSkeletalMeshPoseMessage* PoseMessage = nullptr;

								// Get last pose in frame
								TimelineData.EnumerateEvents(Frame.StartTime, Frame.EndTime,
									[&PoseMessage](double InStartTime, double InEndTime, uint32 InDepth, const FSkeletalMeshPoseMessage& InPoseMessage)
									{
										PoseMessage = &InPoseMessage;
										return TraceServices::EEventEnumerate::Continue;
									});

								// Update position based on pose
								if (PoseMessage)
								{
									TargetActorPosition = PoseMessage->ComponentToWorld.GetTranslation();
								}
							});
						}

						RewindDebugger->SetRootObjectPosition(TargetActorPosition);
					}

					{
						TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebugger::Tick_AnimBlueprintsDebug);
						// Apply Animation Blueprint Debugging Data:
						// - enumerate over all anim graph timelines
						// - check if their instance class still exists and is the debugging target for the Animation Blueprint Editor
						// - if it is copy that debug data into the class debug data for the blueprint debugger

						TraceServices::FProviderReadScopeLock AnimProviderReadScope(*AnimationProvider);
						TraceServices::FProviderReadScopeLock GameProviderReadScope(*GameplayProvider);
						AnimationProvider->EnumerateAnimGraphTimelines([this, &Frame, AnimationProvider, GameplayProvider](uint64 ObjectId, const IAnimationProvider::AnimGraphTimeline& AnimGraphTimeline)
						{
							UAnimInstance* AnimInstance = nullptr;
#if OBJECT_TRACE_ENABLED
							if(UObject* ObjectInstance = FObjectTrace::GetObjectFromId(ObjectId))
							{
								AnimInstance = Cast<UAnimInstance>(ObjectInstance);
							}
#endif
							if (AnimInstance == nullptr)
							{
								AnimInstance = SpawnAnimInstance(ObjectId, GameplayProvider);
							}
							
							if (AnimInstance)
							{
								if(UAnimBlueprintGeneratedClass* InstanceClass = Cast<UAnimBlueprintGeneratedClass>(AnimInstance->GetClass()))
								{
									if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(InstanceClass->ClassGeneratedBy))
									{
										// for child Animation Blueprints, we actually want to debug the root blueprint (since the child doesn't contain any anim graphs)
										if (UAnimBlueprint* RootAnimBP = UAnimBlueprint::FindRootAnimBlueprint(AnimBlueprint))
										{
											if (UAnimBlueprintGeneratedClass* RootInstanceClass = Cast<UAnimBlueprintGeneratedClass>(RootAnimBP->GeneratedClass))
											{
												AnimBlueprint = RootAnimBP;
												InstanceClass = RootInstanceClass;
											}
										}

										if(AnimBlueprint->IsObjectBeingDebugged(AnimInstance))
										{
											TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebugger::Tick_UpdateBlueprintDebug);
											// update debug info for attached Animation Blueprint editors
											const int32 NodeCount = InstanceClass->GetAnimNodeProperties().Num();
					
											FAnimBlueprintDebugData& DebugData = InstanceClass->GetAnimBlueprintDebugData();
											{
												TRACE_CPUPROFILER_EVENT_SCOPE(ResetNodeVisitStates);
												DebugData.ResetNodeVisitSites();
											}
											
											// Anim node values can come from all phases
											AnimationProvider->ReadAnimNodeValuesTimeline(ObjectId, [&Frame,AnimationProvider, &DebugData](const IAnimationProvider::AnimNodeValuesTimeline& InNodeValuesTimeline)
											{
												TRACE_CPUPROFILER_EVENT_SCOPE(AnimGraphNodeValues);
												InNodeValuesTimeline.EnumerateEvents(Frame.StartTime, Frame.EndTime, [AnimationProvider, &DebugData](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNodeValueMessage& InMessage)
												{
													// don't send "Name" Node value for display in the graph
													if (FPlatformString::Strcmp(InMessage.Key, TEXT("Name")) != 0)
													{
														FText Text = AnimationProvider->FormatNodeKeyValue(InMessage);
														DebugData.RecordNodeValue(InMessage.NodeId, Text.ToString());
													}
													return TraceServices::EEventEnumerate::Continue;
												});
											});

											DebugData.DisableAllPoseWatches();
				
											AnimGraphTimeline.EnumerateEvents(Frame.StartTime, Frame.EndTime, [ObjectId, AnimationProvider, GameplayProvider, &DebugData, NodeCount](double InGraphStartTime, double InGraphEndTime, uint32 InDepth, const FAnimGraphMessage& InMessage)
											{
												TRACE_CPUPROFILER_EVENT_SCOPE(AnimGraphTimelineEvent);
														
												// Basic verification - check node count is the same
												// @TODO: could add some form of node hash/CRC to the class to improve this
												if(InMessage.NodeCount == NodeCount)
												{
													// Check for an update phase (which contains weights)
													if(InMessage.Phase == EAnimGraphPhase::Update)
													{
														AnimationProvider->ReadAnimNodesTimeline(ObjectId, [InGraphStartTime, InGraphEndTime, &DebugData](const IAnimationProvider::AnimNodesTimeline& InNodesTimeline)
														{
															TRACE_CPUPROFILER_EVENT_SCOPE(AnimGraphDebugNodeVisits);
															InNodesTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [&DebugData](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNodeMessage& InMessage)
															{
																DebugData.RecordNodeVisit(InMessage.NodeId, InMessage.PreviousNodeId, InMessage.Weight);
																return TraceServices::EEventEnumerate::Continue;
															});
														});
				
														AnimationProvider->ReadStateMachinesTimeline(ObjectId, [InGraphStartTime, InGraphEndTime, &DebugData](const IAnimationProvider::StateMachinesTimeline& InStateMachinesTimeline)
														{
															TRACE_CPUPROFILER_EVENT_SCOPE(AnimGraphDebugStateMachine);
															InStateMachinesTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [&DebugData](double InStartTime, double InEndTime, uint32 InDepth, const FAnimStateMachineMessage& InMessage)
															{
																DebugData.RecordStateData(InMessage.StateMachineIndex, InMessage.StateIndex, InMessage.StateWeight, InMessage.ElapsedTime);
																return TraceServices::EEventEnumerate::Continue;
															});
														});
				
														AnimationProvider->ReadAnimSequencePlayersTimeline(ObjectId, [InGraphStartTime, InGraphEndTime, GameplayProvider, &DebugData](const IAnimationProvider::AnimSequencePlayersTimeline& InSequencePlayersTimeline)
														{
															TRACE_CPUPROFILER_EVENT_SCOPE(AnimGraphDebugSequencePlayers);
															InSequencePlayersTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [&DebugData](double InStartTime, double InEndTime, uint32 InDepth, const FAnimSequencePlayerMessage& InMessage)
															{
																DebugData.RecordSequencePlayer(InMessage.NodeId, InMessage.Position, InMessage.Length, InMessage.FrameCounter);
																return TraceServices::EEventEnumerate::Continue;
															});
														});
				
														AnimationProvider->ReadAnimBlendSpacePlayersTimeline(ObjectId, [InGraphStartTime, InGraphEndTime, GameplayProvider, &DebugData](const IAnimationProvider::BlendSpacePlayersTimeline& InBlendSpacePlayersTimeline)
														{
															TRACE_CPUPROFILER_EVENT_SCOPE(AnimGraphBlendSpaces);
															InBlendSpacePlayersTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [GameplayProvider, &DebugData](double InStartTime, double InEndTime, uint32 InDepth, const FBlendSpacePlayerMessage& InMessage)
															{
																UBlendSpace* BlendSpace = nullptr;
																const FObjectInfo* BlendSpaceInfo = GameplayProvider->FindObjectInfo(InMessage.BlendSpaceId);
																if(BlendSpaceInfo)
																{
																	BlendSpace = TSoftObjectPtr<UBlendSpace>(FSoftObjectPath(BlendSpaceInfo->PathName)).LoadSynchronous();
																}
				
																DebugData.RecordBlendSpacePlayer(InMessage.NodeId, BlendSpace, FVector(InMessage.PositionX, InMessage.PositionY, InMessage.PositionZ), FVector(InMessage.FilteredPositionX, InMessage.FilteredPositionY, InMessage.FilteredPositionZ));
																return TraceServices::EEventEnumerate::Continue;
															});
														});
				
														AnimationProvider->ReadAnimSyncTimeline(ObjectId, [InGraphStartTime, InGraphEndTime, AnimationProvider, &DebugData](const IAnimationProvider::AnimSyncTimeline& InAnimSyncTimeline)
														{
															TRACE_CPUPROFILER_EVENT_SCOPE(AnimGraphAnimSync);
															InAnimSyncTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [AnimationProvider, &DebugData](double InStartTime, double InEndTime, uint32 InDepth, const FAnimSyncMessage& InMessage)
															{
																const TCHAR* GroupName = AnimationProvider->GetName(InMessage.GroupNameId);
																if(GroupName)
																{
																	DebugData.RecordNodeSync(InMessage.SourceNodeId, FName(GroupName));
																}
													
																return TraceServices::EEventEnumerate::Continue;
															});
														});
													}
				
													// Some traces come from both update and evaluate phases
													if(InMessage.Phase == EAnimGraphPhase::Update || InMessage.Phase == EAnimGraphPhase::Evaluate)
													{
														AnimationProvider->ReadAnimAttributesTimeline(ObjectId, [InGraphStartTime, InGraphEndTime, AnimationProvider, &DebugData](const IAnimationProvider::AnimAttributeTimeline& InAnimAttributeTimeline)
														{
															TRACE_CPUPROFILER_EVENT_SCOPE(AnimGraphAttributes);
															InAnimAttributeTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [AnimationProvider, &DebugData](double InStartTime, double InEndTime, uint32 InDepth, const FAnimAttributeMessage& InMessage)
															{
																const TCHAR* AttributeName = AnimationProvider->GetName(InMessage.AttributeNameId);
																if(AttributeName)
																{
																	DebugData.RecordNodeAttribute(InMessage.TargetNodeId, InMessage.SourceNodeId, FName(AttributeName));
																}
													
																return TraceServices::EEventEnumerate::Continue;
															});
														});

														
														AnimationProvider->ReadPoseWatchTimeline(ObjectId, [InGraphStartTime, InGraphEndTime, AnimationProvider, &DebugData](const IAnimationProvider::PoseWatchTimeline& InPoseWatchTimeline)
															{
																InPoseWatchTimeline.EnumerateEvents(InGraphStartTime, InGraphEndTime, [AnimationProvider, &DebugData](double InStartTime, double InEndTime, uint32 InDepth, const FPoseWatchMessage& InMessage)
																	{
																		for (FAnimNodePoseWatch& PoseWatch : DebugData.AnimNodePoseWatch)
																		{
																			if (PoseWatch.NodeID == InMessage.PoseWatchId)
																			{
																				TArray<FBoneIndexType> RequiredBones;
																				TArray<FTransform> BoneTransforms;
																				AnimationProvider->GetPoseWatchData(InMessage, BoneTransforms, RequiredBones);

																				PoseWatch.SetPose(RequiredBones, BoneTransforms);
																				PoseWatch.SetWorldTransform(InMessage.WorldTransform);

																				PoseWatch.PoseWatch->SetIsNodeEnabled(true);
																				break;
																			}
																		}
																		return TraceServices::EEventEnumerate::Continue;
																	});
															});

													}
				
												}
												return TraceServices::EEventEnumerate::Continue;
											});
										}
									}
								}
							}
							return TraceServices::EEventEnumerate::Continue;
						});
					}
				}
			}
			LastScrubTime = CurrentTraceTime;
		}
	}
}

#undef LOCTEXT_NAMESPACE