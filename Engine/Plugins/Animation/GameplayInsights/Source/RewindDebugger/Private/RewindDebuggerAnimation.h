// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAnimationProvider.h"
#include "IGameplayProvider.h"
#include "IRewindDebuggerExtension.h"
#include "RewindDebuggerSettings.h"
#include "UObject/WeakObjectPtr.h"


namespace TraceServices
{
	struct FFrame;
}

class UAnimInstance;

// Rewind debugger extension for animation support
//  replay of animated pose data
//  updating animation blueprint debugger

class FRewindDebuggerAnimation : public IRewindDebuggerExtension
{
public:

	FRewindDebuggerAnimation();
	virtual ~FRewindDebuggerAnimation() = default;
	void Initialize();
	void Shutdown();

	virtual void Update(float DeltaTime, IRewindDebugger* RewindDebugger) override;
	virtual void Clear(IRewindDebugger* RewindDebugger) override;
	virtual FString GetName() { return TEXT("RewindDebuggerAnimation"); }
	
	void OnPIEResumed(bool bSimulating);
	void OnPIEStopped(bool bSimulating);
	void OnPIESingleStepped(bool bSimulating);

	UAnimInstance* GetDebugAnimInstance(uint64 ObjectId);

	static FRewindDebuggerAnimation* GetInstance() { return Instance; }
private:
	void ClearSpawnedComponents();
	void ApplyPoseToMesh(const class IAnimationProvider* AnimationProvider, const IGameplayProvider* GameplayProvider, const TraceServices::FFrame& Frame,
		const IAnimationProvider::SkeletalMeshPoseTimeline& TimelineData, USkeletalMeshComponent* MeshComponent, uint64 ObjectId, bool bQueueForReset, bool bApplyMesh);
	
	struct FMeshComponentResetData
	{
		TWeakObjectPtr<USkeletalMeshComponent> Component;
		FTransform RelativeTransform;
		int ForcedLod = -1;
		bool bIsVisible = true;
	};

	struct FSpawnedMeshComponentInfo
	{
		// mesh component object id
		uint64 id;
		// actor to hold the mesh component
		TWeakObjectPtr<AActor> Actor;
		// mesh
		TObjectPtr<USkeletalMeshComponent> Component;
	};
	
	struct FSpawnedAnimInstanceInfo
	{
		// AnimInstance id
		uint64 id;
		// Data used for anim BP debugging
		TWeakObjectPtr<UAnimInstance> AnimInstance;
	};
	
	FSpawnedMeshComponentInfo* SpawnMesh(uint64 ObjectId, const IGameplayProvider* GameplayProvider);
	UAnimInstance* SpawnAnimInstance(uint64 ObjectId, const IGameplayProvider* GameplayProvider);

	TMap<uint64, FSpawnedMeshComponentInfo> SpawnedMeshComponents;
	TMap<uint64, FSpawnedAnimInstanceInfo> SpawnedAnimInstances;

	TMap<uint64, FMeshComponentResetData> MeshComponentsToReset;
	double LastScrubTime = -1;

	RewindDebugger::FObjectId TargetActorMeshId;
	RewindDebugger::FObjectId TargetActorIdForMesh;
	TOptional<FVector> TargetActorPosition;

	static FRewindDebuggerAnimation* Instance;
};
