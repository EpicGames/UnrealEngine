// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerCamera.h"
#include "Camera/CameraActor.h"
#include "Camera/CameraComponent.h"
#include "Common/ProviderLock.h"
#include "Editor.h"
#include "IGameplayProvider.h"
#include "Insights/IUnrealInsightsModule.h"
#include "IRewindDebugger.h"
#include "LevelEditor.h"
#include "RewindDebuggerModule.h"
#include "Modules/ModuleManager.h"
#include "SLevelViewport.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "TraceServices/Model/Frames.h"

#define LOCTEXT_NAMESPACE "RewindDebuggerCamera"

FRewindDebuggerCamera::FRewindDebuggerCamera()
	: bLastPositionValid(false)
	, LastPosition()
{
}

FRewindDebuggerCamera::~FRewindDebuggerCamera()
{
	if (UToolMenus* Menus = UToolMenus::Get())
	{
		if (UToolMenu* Menu = Menus->FindMenu(FRewindDebuggerModule::MainToolBarName))
		{
			Menu->RemoveSection("CameraControls");
		}
	}
}

void FRewindDebuggerCamera::Initialize()
{
	UToolMenu* ToolBar = UToolMenus::Get()->ExtendMenu(FRewindDebuggerModule::MainToolBarName);
	FToolMenuSection& Section = ToolBar->AddSection("CameraControls", FText::GetEmpty(), FToolMenuInsert("CategoriesControls", EToolMenuInsertType::Before));

	Section.AddEntry(FToolMenuEntry::InitComboButton(
		"CameraMode",
		FUIAction(),
		FNewToolMenuDelegate::CreateRaw(this, &FRewindDebuggerCamera::MakeCameraModeMenu),
		TAttribute<FText>(),
		LOCTEXT("CameraModeTooltip", "Choose how the viewport camera behaves during playback"),
		FSlateIcon("RewindDebuggerStyle", "RewindDebugger.Camera")
	));
}

void FRewindDebuggerCamera::MakeCameraModeMenu(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->FindOrAddSection("CameraMode", LOCTEXT("CameraModeLabel", "Camera Mode"));

	Section.AddEntry(FToolMenuEntry::InitMenuEntry("CameraModeDisabled",
		LOCTEXT("Camera Mode Disabled", "Disabled"),
		LOCTEXT("CameraModeDisabledTooltip", "Camera moves freely, no automatic tracking"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FRewindDebuggerCamera::SetCameraMode, ERewindDebuggerCameraMode::Disabled),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([this]
				{
					return CameraMode() == ERewindDebuggerCameraMode::Disabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}
			)
		),
		EUserInterfaceActionType::Check
	));

	Section.AddEntry(FToolMenuEntry::InitMenuEntry("CameraModeFollow",
		LOCTEXT("Camera Mode Follow", "Follow Target Actor"),
		LOCTEXT("CameraModeFollowTooltip", "Camera offset stays fixed relative to the selected actor as it moves through the recording"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FRewindDebuggerCamera::SetCameraMode, ERewindDebuggerCameraMode::FollowTargetActor),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([this]
				{
					return CameraMode() == ERewindDebuggerCameraMode::FollowTargetActor ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}
			)
		),
		EUserInterfaceActionType::Check
	));

	Section.AddEntry(FToolMenuEntry::InitMenuEntry("CameraModeReplay",
		LOCTEXT("Camera Mode Recorded", "Replay Recorded Camera"),
		LOCTEXT("CameraModeReplayTooltip", "Locks the viewport to the recorded in-game camera, replaying its exact position and rotation"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FRewindDebuggerCamera::SetCameraMode, ERewindDebuggerCameraMode::Replay),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([this]
				{
					return CameraMode() == ERewindDebuggerCameraMode::Replay ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}
			)
		),
		EUserInterfaceActionType::Check
	));
}

ERewindDebuggerCameraMode FRewindDebuggerCamera::CameraMode() const
{
	return URewindDebuggerSettings::Get().CameraMode;
}

void FRewindDebuggerCamera::SetCameraMode(ERewindDebuggerCameraMode InMode)
{
	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	SLevelViewport* LevelViewport = LevelEditor.GetFirstActiveLevelViewport().Get();
	FLevelEditorViewportClient& LevelViewportClient = LevelViewport->GetLevelViewportClient();

	URewindDebuggerSettings& RewindDebuggerSettings = URewindDebuggerSettings::Get();
	
	if (RewindDebuggerSettings.CameraMode == ERewindDebuggerCameraMode::Replay && InMode != ERewindDebuggerCameraMode::Replay)
	{
		LevelViewportClient.SetActorLock(nullptr);
	}
	else if (RewindDebuggerSettings.CameraMode == ERewindDebuggerCameraMode::Replay)
	{
		if (CameraActor.IsValid())
		{
			LevelViewportClient.SetActorLock(CameraActor.Get());
		}
	}
	
	RewindDebuggerSettings.CameraMode = InMode;
	RewindDebuggerSettings.Modify();
	RewindDebuggerSettings.SaveConfig();
}

void FRewindDebuggerCamera::Update(float DeltaTime, IRewindDebugger* RewindDebugger)
{
	if (RewindDebugger->IsPIESimulating() || RewindDebugger->GetRecordingDuration() == 0.0)
	{
		bLastPositionValid = false;
		return;
	}
	
	if (const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession())
	{
		double CurrentTraceTime = RewindDebugger->CurrentTraceTime();

		static double LastCameraScrubTime = 0.0f;
		if (CurrentTraceTime != LastCameraScrubTime)
		{
			bool bCameraTraceDataFound = false;

			FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
			if (SLevelViewport* LevelViewport = LevelEditor.GetFirstActiveLevelViewport().Get())
			{
				FLevelEditorViewportClient& LevelViewportClient = LevelViewport->GetLevelViewportClient();

				FVector TargetActorPosition;
				const bool bTargetActorPositionValid = RewindDebugger->GetRootObjectPosition(TargetActorPosition);

				if (CameraMode() == ERewindDebuggerCameraMode::FollowTargetActor)
				{
					// Follow Actor mode: apply position changes from the target actor to the camera
					if (bTargetActorPositionValid)
					{
						if(bLastPositionValid)
						{
							LevelViewportClient.SetViewLocation(LevelViewportClient.GetViewLocation() + TargetActorPosition - LastPosition);
						}
					}
				}

				// always update the camera actor to the replay values even if it isn't locked
				if (const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider"))
				{
					TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
					GameplayProvider->ReadViewTimeline([&bCameraTraceDataFound, this, RewindDebugger, CurrentTraceTime, Session](const IGameplayProvider::ViewTimeline& TimelineData)
					{
						bool bFrameFound;
						TraceServices::FFrame Frame;
						{
							TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session);
							const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*Session);
							bFrameFound = FrameProvider.GetFrameFromTime(TraceFrameType_Game, CurrentTraceTime, Frame);
						}

						if (bFrameFound)
						{
							TimelineData.EnumerateEvents(Frame.StartTime, Frame.EndTime,
								[&bCameraTraceDataFound, this, RewindDebugger](double InStartTime, double InEndTime, uint32 InDepth, const FViewMessage& ViewMessage)
								{
									if (!CameraActor.IsValid())
									{
										FActorSpawnParameters SpawnParameters;
										SpawnParameters.ObjectFlags |= RF_Transient;
										if (UWorld* World = RewindDebugger->GetWorldToVisualize())
										{
											CameraActor = World->SpawnActor<ACameraActor>(ViewMessage.Position, ViewMessage.Rotation, SpawnParameters);
											CameraActor->SetActorLabel("RewindDebuggerCamera");
										}
									}

									if (CameraActor.IsValid())
									{
										UCameraComponent* Camera = CameraActor->GetCameraComponent();
										Camera->SetWorldLocationAndRotation(ViewMessage.Position, ViewMessage.Rotation);
										Camera->SetFieldOfView(ViewMessage.Fov);
										Camera->SetAspectRatio(ViewMessage.AspectRatio);
									}

									bCameraTraceDataFound = true;

									return TraceServices::EEventEnumerate::Stop;
								});
						}
					});
				}

				if (CameraMode() == ERewindDebuggerCameraMode::Replay)
				{
					if (CameraActor.IsValid())
					{
						LevelViewportClient.SetActorLock(CameraActor.Get());
					}
				}

				LastPosition = TargetActorPosition;
				bLastPositionValid = bTargetActorPositionValid;

				if (bCameraTraceDataFound) // don't update this if there was no trace data found, because when first pausing, it can take a few frames for latest data to get processed
				{
					// only update camera in playback or scrubbing when the time has changed (allow free movement when paused)
					LastCameraScrubTime = CurrentTraceTime;
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE