// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IRewindDebuggerExtension.h"
#include "RewindDebuggerSettings.h"
#include "UObject/WeakObjectPtr.h"

class ACameraActor;
class UToolMenu;

// Rewind debugger extension for camera support
//  replay of recorded camera data
//  follow selected actor

class FRewindDebuggerCamera : public IRewindDebuggerExtension
{
public:


	FRewindDebuggerCamera();
	virtual ~FRewindDebuggerCamera();
	void Initialize();

	virtual void Update(float DeltaTime, IRewindDebugger* RewindDebugger) override;
	virtual FString GetName() { return TEXT("RewindDebuggerCamera"); }

	ERewindDebuggerCameraMode CameraMode() const;
	void SetCameraMode(ERewindDebuggerCameraMode Mode);
	
private:

	void MakeCameraModeMenu(UToolMenu* InMenu);
	
	bool bLastPositionValid;
	FVector LastPosition;
	
	TWeakObjectPtr<ACameraActor> CameraActor; 
};
