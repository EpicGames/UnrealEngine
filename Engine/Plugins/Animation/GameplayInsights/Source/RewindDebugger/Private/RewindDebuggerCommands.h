// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/Commands.h"

#define LOCTEXT_NAMESPACE "FRewindDebuggerCommands"

class FRewindDebuggerCommands : public TCommands<FRewindDebuggerCommands>
{
public:

	/** Default constructor. */
	FRewindDebuggerCommands()
		: TCommands<FRewindDebuggerCommands>(TEXT("RewindDebugger"), LOCTEXT("RewindDebugger", "Rewind Debugger"), NAME_None, "RewindDebuggerStyle")
	{ }

	//~ TCommands interface
	virtual void RegisterCommands() override
	{
		UI_COMMAND(PauseOrPlay, "Pause or Play Recording", "Toggle recording playing", EUserInterfaceActionType::Button, FInputChord(EKeys::SpaceBar) );
		UI_COMMAND(StartRecording, "Start Recording", "Start recording gameplay data", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::R));
		UI_COMMAND(StopRecording, "Stop Recording", "Stop recording gameplay data", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::S));
		UI_COMMAND(FirstFrame, "First Frame", "Jump to first recorded frame", EUserInterfaceActionType::Button, FInputChord(EKeys::Up));
		UI_COMMAND(PreviousFrame, "Previous Frame", "Step one frame back", EUserInterfaceActionType::Button, FInputChord(EKeys::Left));
		UI_COMMAND(ReversePlay, "Reverse Play", "Playback recorded data in reverse", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::SpaceBar));
		UI_COMMAND(Pause, "Pause", "Pause playback", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Play, "Play", "Playback recorded data", EUserInterfaceActionType::Button, FInputChord(EKeys::Down));
		UI_COMMAND(NextFrame, "Next Frame", "Step one frame forward", EUserInterfaceActionType::Button, FInputChord(EKeys::Right));
		UI_COMMAND(LastFrame, "Last Frame", "Jump to last recorded frame", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Up));
		UI_COMMAND(AutoEject, "Auto Eject", "Automatically detach player control when PIE is paused", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(AutoRecord, "Auto Record", "Automatically start recording when PIE is started", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(OpenTrace, "Open Recording", "Open a Trace file containing a Rewind Debugger recording", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ClearAnalysis, "Clear Analysis", "Stop current analysis and clear related debugger data", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(AutoScroll, "AutoScroll", "Maintains the cursor on the last recorded event", EUserInterfaceActionType::ToggleButton, FInputChord());
	}


	TSharedPtr<FUICommandInfo> StartRecording;
	TSharedPtr<FUICommandInfo> StopRecording;
	TSharedPtr<FUICommandInfo> FirstFrame;
	TSharedPtr<FUICommandInfo> PreviousFrame;
	TSharedPtr<FUICommandInfo> ReversePlay;
	TSharedPtr<FUICommandInfo> Pause;
	TSharedPtr<FUICommandInfo> Play;
	TSharedPtr<FUICommandInfo> NextFrame;
	TSharedPtr<FUICommandInfo> LastFrame;
	TSharedPtr<FUICommandInfo> PauseOrPlay;
	TSharedPtr<FUICommandInfo> AutoEject;
	TSharedPtr<FUICommandInfo> AutoRecord;
	TSharedPtr<FUICommandInfo> OpenTrace;
	TSharedPtr<FUICommandInfo> ClearAnalysis;
	TSharedPtr<FUICommandInfo> AutoScroll;
};


#undef LOCTEXT_NAMESPACE
