// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerFallbackTrack.h"
#include "IRewindDebugger.h"
#include "IGameplayProvider.h"
#include "IAnimationProvider.h"
#include "IRewindDebuggerView.h"
#include "Styling/SlateIconFinder.h"
#include "ObjectTrace.h"

namespace RewindDebugger
{

FSlateIcon FRewindDebuggerFallbackTrack::GetIconInternal()
{
	return ViewCreator->GetIcon();
}

bool FRewindDebuggerFallbackTrack::UpdateInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	TSharedPtr<IRewindDebuggerView> PinnedView = View.Pin();
	if (PinnedView.IsValid())
	{
		PinnedView->SetTimeMarker(RewindDebugger->CurrentTraceTime());
	}

	return false;
}
	

TSharedPtr<SWidget> FRewindDebuggerFallbackTrack::GetDetailsViewInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	TSharedPtr<IRewindDebuggerView> RewindDebuggerView = ViewCreator->CreateDebugView(ObjectId, RewindDebugger->CurrentTraceTime(), *RewindDebugger->GetAnalysisSession());
	View = RewindDebuggerView;
	return RewindDebuggerView;
}

}
