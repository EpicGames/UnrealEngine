// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RewindDebuggerTrack.h"
#include "IRewindDebuggerViewCreator.h"

namespace RewindDebugger
{

class FRewindDebuggerFallbackTrack : public FRewindDebuggerTrack
{
public:

	FRewindDebuggerFallbackTrack(uint64 InObjectId, const IRewindDebuggerViewCreator* InViewCreator)
		: ViewCreator(InViewCreator), ObjectId(InObjectId)
	{
	}

private:
	virtual FSlateIcon GetIconInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual FName GetNameInternal() const override { return ViewCreator->GetName(); }
	virtual FText GetDisplayNameInternal() const override { return ViewCreator->GetTitle(); }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }
	virtual bool UpdateInternal() override;

	const IRewindDebuggerViewCreator* ViewCreator;
	TWeakPtr<IRewindDebuggerView> View;
	FSlateIcon Icon;
	TRange<double> ExistenceRange;
	uint64 ObjectId;
};

}