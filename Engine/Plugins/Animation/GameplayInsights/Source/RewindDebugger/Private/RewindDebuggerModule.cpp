// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerModule.h"
#include "AnimInstanceHelpers.h"
#include "BlueprintDoubleClickHandler.h"
#include "Engine/Selection.h"
#include "Features/IModularFeatures.h"
#include "Framework/Docking/LayoutExtender.h"
#include "IAnimationBlueprintEditorModule.h"
#include "Kismet2/DebuggerCommands.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "PropertyTraceMenu.h"
#include "RewindDebugger.h"
#include "RewindDebuggerCommands.h"
#include "RewindDebuggerEngineEditorBridge.h"
#include "RewindDebuggerRuntime/RewindDebuggerRuntime.h"
#include "RewindDebuggerStyle.h"
#include "SRewindDebugger.h"
#include "ToolMenus.h"
#include "TraceSessionsManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "RewindDebuggerModule"

const FName FRewindDebuggerModule::MainTabName("RewindDebugger2");
const FName FRewindDebuggerModule::DetailsTabName("RewindDebuggerDetails2");
const FName FRewindDebuggerModule::TrackContextMenuName("RewindDebugger.TrackContextMenu");
const FName FRewindDebuggerModule::MainToolBarName = FName("RewindDebugger.ToolBar");
const FName FRewindDebuggerModule::RightToolBarName = FName("RewindDebugger.RightToolBar");
const FName FRewindDebuggerModule::CategoriesMenuName = FName("RewindDebugger.CategoriesMenu");
const FName FRewindDebuggerModule::PreviewMenuName = FName("RewindDebugger.PreviewMenu");
const FName FRewindDebuggerModule::MainStatusBarName = FName("RewindDebugger.StatusBar");

#include "TraceServices/ModuleService.h"

FRewindDebuggerModule& FRewindDebuggerModule::Get()
{
	return FModuleManager::Get().LoadModuleChecked<FRewindDebuggerModule>(TEXT("RewindDebugger"));
}

FName FRewindDebuggerModule::GetMainTabName() const
{
	return MainTabName;
}

FName FRewindDebuggerModule::GetDetailsTabName() const
{
	return DetailsTabName;
}

FName FRewindDebuggerModule::GetMainToolbarName() const
{
	return MainToolBarName;
}

FName FRewindDebuggerModule::GetPreviewMenuName() const
{
	return PreviewMenuName;
}

TSharedRef<SDockTab> FRewindDebuggerModule::SpawnRewindDebuggerDetailsTab(const FSpawnTabArgs& SpawnTabArgs)
{
	if (FRewindDebugger::Instance() == nullptr)
	{
		FRewindDebugger::Initialize();
	}

	FRewindDebugger* RewindDebugger = FRewindDebugger::Instance();
	
	RewindDebugger->SetIsDetailsPanelOpen(true);
	
	TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(PanelTab);

	MajorTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda( [](TSharedRef<SDockTab>) { FRewindDebugger::Instance()->SetIsDetailsPanelOpen(false); }));

	RewindDebugger->UpdateDetailsPanel(MajorTab);

	return MajorTab;
}

TSharedRef<SDockTab> FRewindDebuggerModule::SpawnRewindDebuggerTab(const FSpawnTabArgs& SpawnTabArgs)
{
	if (FRewindDebugger::Instance() == nullptr)
	{
		FRewindDebugger::Initialize();
	}
	
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(PanelTab)
		.OnTabClosed_Lambda([this](TSharedRef<SDockTab>)
		{
			// clear reference to widget so it will be destroyed
			RewindDebuggerWidget = nullptr;
		});

	TSharedPtr<FUICommandList> CommandList = MakeShared<FUICommandList>();
	const FRewindDebuggerCommands& Commands = FRewindDebuggerCommands::Get();

	FRewindDebugger* DebuggerInstance = FRewindDebugger::Instance();

	CommandList->MapAction(Commands.Play,
							FExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::Play), 
							FCanExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::CanPlay),
							FIsActionChecked(),
							FIsActionButtonVisible());

	CommandList->MapAction(Commands.Pause,
							FExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::Pause), 
							FCanExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::CanPause),
							FIsActionChecked(),
							FIsActionButtonVisible());

	CommandList->MapAction(Commands.PauseOrPlay,
							FExecuteAction::CreateLambda([DebuggerInstance]()
							{
								if (DebuggerInstance->CanPause())
								{
									DebuggerInstance->Pause();
								}
								else if (DebuggerInstance->CanPlay())
								{
									DebuggerInstance->Play();
								}
							}), 
							FCanExecuteAction::CreateLambda([DebuggerInstance]()
							{
								return DebuggerInstance->CanPause() || DebuggerInstance->CanPlay();
							}),
							FIsActionChecked(),
							FIsActionButtonVisible());
	
	CommandList->MapAction(Commands.ReversePlay,
							FExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::PlayReverse), 
							FCanExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::CanPlayReverse),
							FIsActionChecked(),
							FIsActionButtonVisible());

	CommandList->MapAction(Commands.PreviousFrame,
							FExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::StepBackward), 
							FCanExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::CanScrub),
							FIsActionChecked(),
							FIsActionButtonVisible());


	CommandList->MapAction(Commands.FirstFrame,
							FExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::ScrubToStart), 
							FCanExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::CanScrub),
							FIsActionChecked(),
							FIsActionButtonVisible());

	CommandList->MapAction(Commands.LastFrame,
							FExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::ScrubToEnd), 
							FCanExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::CanScrub),
							FIsActionChecked(),
							FIsActionButtonVisible());

	CommandList->MapAction(Commands.NextFrame,
							FExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::StepForward), 
							FCanExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::CanScrub),
							FIsActionChecked(),
							FIsActionButtonVisible());

	CommandList->MapAction(Commands.AutoEject,
							FExecuteAction::CreateLambda([DebuggerInstance]() { DebuggerInstance->SetShouldAutoEject(!DebuggerInstance->ShouldAutoEject()); }),
							FCanExecuteAction(),
							FCanExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::ShouldAutoEject),
							FIsActionButtonVisible());

	CommandList->MapAction(Commands.AutoRecord,
							FExecuteAction::CreateLambda([DebuggerInstance]() { DebuggerInstance->SetShouldAutoRecordOnPIE(!DebuggerInstance->ShouldAutoRecordOnPIE()); }),
							FCanExecuteAction(),
							FCanExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::ShouldAutoRecordOnPIE),
							FIsActionButtonVisible());

	CommandList->MapAction(Commands.OpenTrace,
							FExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::OpenTrace),
							FCanExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::CanOpenTrace),
							FIsActionChecked(),
							FIsActionButtonVisible());

	CommandList->MapAction(Commands.ClearAnalysis,
							FExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::ClearAnalysis),
							FCanExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::CanClearAnalysis),
							FIsActionChecked(),
							FIsActionButtonVisible());

	CommandList->MapAction(Commands.AutoScroll,
							FExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::ToggleAutoScroll),
							FCanExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::CanEnableAutoScroll),
							FIsActionChecked::CreateRaw(DebuggerInstance, &FRewindDebugger::IsAutoScrollEnabled),
							FIsActionButtonVisible());

	// Register PIE Rewind Debugger Commands
	if (GEditor != nullptr)
	{
		check(FPlayWorldCommands::GlobalPlayWorldActions.IsValid());

		// Recording buttons have been replaced by the recording controls widget and the remote sessions manager.
		// The following commands are not added to the toolbar but still registered to allow users to use associated shortcuts.
		// To preserve legacy behavior they only control recording on the local session (i.e., PIE), regardless of the session
		// selected in the recording controls.
		FPlayWorldCommands::GlobalPlayWorldActions->MapAction(Commands.StartRecording, 
							FExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::StartRecording),
							FCanExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::CanStartRecording),
							FIsActionChecked(),
							FIsActionButtonVisible());

		FPlayWorldCommands::GlobalPlayWorldActions->MapAction(Commands.StopRecording,
							 FExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::StopRecordingLocalSession),
							 FCanExecuteAction::CreateRaw(DebuggerInstance, &FRewindDebugger::CanStopRecordingLocalSession),
							 FIsActionChecked(),
							 FIsActionButtonVisible());
	}

	RewindDebuggerWidget = SNew(SRewindDebugger, CommandList.ToSharedRef(), MajorTab, SpawnTabArgs.GetOwnerWindow())
								.DebuggedObjectName({ DebuggerInstance->GetRootObjectNameProperty(), URewindDebuggerSettings::Get().DebugTargetActor})
								.RecordingDuration(DebuggerInstance->GetRecordingDurationProperty())
								.Tracks(&DebuggerInstance->GetTracks())
								.TraceTime(DebuggerInstance->GetTraceTimeProperty())
								.OnScrubPositionChanged_Raw(DebuggerInstance,&FRewindDebugger::ScrubToTime)
								.OnViewRangeChanged_Raw(DebuggerInstance,&FRewindDebugger::SetCurrentViewRange)
								.OnTrackDoubleClicked_Raw(DebuggerInstance, &FRewindDebugger::TrackDoubleClicked)
								.OnTrackSelectionChanged_Raw(DebuggerInstance, &FRewindDebugger::TrackSelectionChanged)
								.BuildTrackContextMenu_Raw(DebuggerInstance, &FRewindDebugger::BuildTrackContextMenu)
								.TrackTypes_Lambda([]() { return FRewindDebugger::Instance()->GetTrackTypes(); })
								.ScrubTime_Lambda([]() { return FRewindDebugger::Instance()->GetScrubTime(); })
								.ForceRefreshTracks_Lambda([]
									{
										FRewindDebugger::Instance()->ClearTrackData();
										FRewindDebugger::Instance()->RefreshDebugTracks();
									});

	DebuggerInstance->SetTrackCursorDelegate(FRewindDebugger::FOnTrackCursor::CreateSP(RewindDebuggerWidget.Get(), &SRewindDebugger::TrackCursor));
	DebuggerInstance->SetCenterViewOnTimeDelegate(FRewindDebugger::FOnCenterViewOnTime::CreateSP(RewindDebuggerWidget.Get(), &SRewindDebugger::CenterView));
	DebuggerInstance->SetTrackListChangedDelegate(FRewindDebugger::FOnTrackListChanged::CreateSP(RewindDebuggerWidget.Get(), &SRewindDebugger::RefreshTracks));

	MajorTab->SetContent(RewindDebuggerWidget.ToSharedRef());

	return MajorTab;
}

static FAnimInstanceDoubleClickHandler AnimInstanceDoubleClickHandler;
static FBlueprintDoubleClickHandler BlueprintDoubleClickHandler;

void FRewindDebuggerModule::StartupModule()
{
	// @todo: this should only require UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS,
	// but we currently need the runtime instance to create the trace sessions manager
	// in order to generate the file name. This dependency needs to be removed if we want
	// some programs to only uses the analysis functionalities.
#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS && WITH_TRACE_BASED_DEBUGGERS
	TraceSessionsManager = MakeShared<UE::TraceBasedDebuggers::FTraceSessionsManager>(
		*RewindDebugger::FRewindDebuggerRuntime::Instance()
		, UE::RewindDebugger::FRewindDebuggerEngineEditorBridge::Get()
		, /*TraceModule*/nullptr
		, /*SaveDirectorySubPathInUserDir*/TEXT("RewindDebugger"));
#endif // UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS && WITH_TRACE_BASED_DEBUGGERS

	UToolMenus::Get()->RegisterMenu(TrackContextMenuName);
	UToolMenus::Get()->RegisterMenu(CategoriesMenuName);

	FRewindDebuggerStyle::Initialize();
	FRewindDebuggerCommands::Register();

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

	LevelEditorTabManagerChangedHandle = LevelEditorModule.OnTabManagerChanged().AddLambda([this]()
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
			TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

			LevelEditorTabManager->RegisterTabSpawner(
				MainTabName, FOnSpawnTab::CreateRaw(this, &FRewindDebuggerModule::SpawnRewindDebuggerTab))
				.SetDisplayName(LOCTEXT("TabTitle", "Rewind Debugger"))
				.SetIcon(FSlateIcon("RewindDebuggerStyle", "RewindDebugger.RewindIcon"))
				.SetTooltipText(LOCTEXT("TooltipText", "Opens Rewind Debugger."))
				.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory());

			LevelEditorTabManager->RegisterTabSpawner(
				DetailsTabName, FOnSpawnTab::CreateRaw(this, &FRewindDebuggerModule::SpawnRewindDebuggerDetailsTab))
				.SetDisplayName(LOCTEXT("DetailsTabTitle", "Rewind Debugger Details"))
				.SetIcon(FSlateIcon("RewindDebuggerStyle", "RewindDebugger.RewindDetailsIcon"))
				.SetTooltipText(LOCTEXT("DetailsWindowTooltipText", "Opens Rewind Debugger Details Window."))
				.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory());
			
		});

	LevelEditorLayoutExtensionHandle = LevelEditorModule.OnRegisterLayoutExtensions().AddLambda(
		[this](FLayoutExtender& Extender)
		{
			Extender.ExtendLayout(FName("LevelEditorSelectionDetails"), ELayoutExtensionPosition::After, FTabManager::FTab(DetailsTabName, ETabState::ClosedTab));
			Extender.ExtendLayout(FName("Sequencer"), ELayoutExtensionPosition::After, FTabManager::FTab(MainTabName, ETabState::ClosedTab));
		}
	);

	RewindDebuggerCameraExtension.Initialize();
	RewindDebuggerAnimationExtension.Initialize();
	IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, &RewindDebuggerCameraExtension);
	IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, &RewindDebuggerAnimationExtension);
	IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerDoubleClickHandler::ModularFeatureName, &AnimInstanceDoubleClickHandler);
	IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerDoubleClickHandler::ModularFeatureName, &BlueprintDoubleClickHandler);

	FPropertyTraceMenu::Register();
	FAnimInstanceMenu::Register();
	FRewindDebugger::RegisterTrackContextMenu();
	FRewindDebugger::RegisterToolBar();
}

void FRewindDebuggerModule::ShutdownModule()
{
	RewindDebuggerAnimationExtension.Shutdown();

	if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditorModule->OnTabManagerChanged().Remove(LevelEditorTabManagerChangedHandle);
		LevelEditorModule->OnRegisterLayoutExtensions().Remove(LevelEditorLayoutExtensionHandle);

		if (TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule->GetLevelEditorTabManager())
		{
			LevelEditorTabManager->UnregisterTabSpawner(MainTabName);
			LevelEditorTabManager->UnregisterTabSpawner(DetailsTabName);
		}
	}

	IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, &RewindDebuggerCameraExtension);
	IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, &RewindDebuggerAnimationExtension);
	IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerDoubleClickHandler::ModularFeatureName, &AnimInstanceDoubleClickHandler);
	IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerDoubleClickHandler::ModularFeatureName, &BlueprintDoubleClickHandler);

	FRewindDebuggerCommands::Unregister();
	FRewindDebuggerStyle::Shutdown();
	FRewindDebugger::Shutdown();

}

IMPLEMENT_MODULE(FRewindDebuggerModule, RewindDebugger);

#undef LOCTEXT_NAMESPACE
