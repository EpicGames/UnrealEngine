// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRewindDebuggerModule.h"
#include "RewindDebuggerCamera.h"
#include "RewindDebuggerAnimation.h"

namespace UE::TraceBasedDebuggers
{
class FTraceSessionsManager;
}

class SDockTab;
class SRewindDebugger;
class SRewindDebuggerDetails;

class FRewindDebuggerModule : public IRewindDebuggerModule
{
public:
	//~ IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	static FRewindDebuggerModule& Get();
	virtual FName GetMainTabName() const override;
	virtual FName GetMainToolbarName() const override;
	virtual FName GetPreviewMenuName() const override;
	virtual FName GetDetailsTabName() const override;
	virtual TSharedRef<SDockTab> SpawnRewindDebuggerTab(const FSpawnTabArgs& SpawnTabArgs) override;
	virtual TSharedRef<SDockTab> SpawnRewindDebuggerDetailsTab(const FSpawnTabArgs& SpawnTabArgs) override;

	static const FName MainToolBarName;
	static const FName RightToolBarName;
	static const FName CategoriesMenuName;
	static const FName PreviewMenuName;
	static const FName MainStatusBarName;
	static const FName MainTabName;
	static const FName DetailsTabName;
	static const FName TrackContextMenuName;

#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS
public:
	/** Returns the trace sessions manager instance used by RewindDebugger */
	TSharedPtr<UE::TraceBasedDebuggers::FTraceSessionsManager>& GetTraceSessionsManager()
	{
		return TraceSessionsManager;
	}

private:
	TSharedPtr<UE::TraceBasedDebuggers::FTraceSessionsManager> TraceSessionsManager;
#endif // UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS

private:
	TSharedPtr<SRewindDebugger> RewindDebuggerWidget;

	FRewindDebuggerCamera RewindDebuggerCameraExtension;
	FRewindDebuggerAnimation RewindDebuggerAnimationExtension;

	FDelegateHandle LevelEditorTabManagerChangedHandle;
	FDelegateHandle LevelEditorLayoutExtensionHandle;
};
