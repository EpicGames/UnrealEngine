// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerObjectTrack.h"
#include "Common/ProviderLock.h"
#include "IAnimationProvider.h"
#include "IGameplayProvider.h"
#include "IRewindDebugger.h"
#include "IRewindDebuggerDoubleClickHandler.h"
#include "ObjectTrace.h"
#include "RewindDebuggerFallbackTrack.h"
#include "RewindDebuggerTrackCreators.h"
#include "RewindDebuggerViewCreators.h"
#include "SEventTimelineView.h"
#include "Styling/SlateIconFinder.h"
#include "LevelSequence.h"
#include "LevelSequenceActor.h"
#include "RewindDebuggerSettings.h"

#define LOCTEXT_NAMESPACE "RewindDebuggerObjectTrack"

namespace RewindDebugger
{

// check if an object is or is a subclass of a type by name, based on Insights traced type info
static bool GetTypeHierarchyNames(const RewindDebugger::FObjectId InObjectId, const TraceServices::IAnalysisSession& InSession, TArray<const TCHAR*>& OutTypeNames)
{
	const IGameplayProvider* GameplayProvider = InSession.ReadProvider<IGameplayProvider>("GameplayProvider");

	TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
	const FObjectInfo& ObjectInfo = GameplayProvider->GetObjectInfo(InObjectId);
	uint64 ClassId = ObjectInfo.ClassId;

	while (ClassId != FObjectId::InvalidId)
	{
		const FClassInfo& ClassInfo = GameplayProvider->GetClassInfo(ClassId);
		OutTypeNames.Add(ClassInfo.Name);
		ClassId = ClassInfo.SuperId;
	}

	return !OutTypeNames.IsEmpty();
}

FRewindDebuggerObjectTrack::FRewindDebuggerObjectTrack(const FObjectId& InObjectId, const FString& InObjectName, bool bInAddController)
	: ObjectName(InObjectName)
	, ObjectId(InObjectId)
	, bAddController(bInAddController)
	, bDisplayNameValid(false)
	, bIconSearched(false)
{
	ExistenceRange = MakeShared<SEventTimelineView::FTimelineEventData>();
	
	const TraceServices::IAnalysisSession* Session = IRewindDebugger::Instance()->GetAnalysisSession();

	if (TArray<const TCHAR*> TypeNames; GetTypeHierarchyNames(InObjectId, *Session, TypeNames))
	{
		FRewindDebuggerTrackCreators::EnumerateCreators([&Creators = ChildTrackCreators, &TypeNames](const IRewindDebuggerTrackCreator* Creator)
			{

				if (TypeNames.FindByPredicate([TargetTypeName = Creator->GetTargetTypeName()](const TCHAR* TypeName) { return TargetTypeName == TypeName; }))
				{
					Creators.Push({ Creator, /*Track*/nullptr });
				}
			});
	}

	// sort by creators by priority + name
	ChildTrackCreators.Sort([](const FTrackCreatorAndTrack& A, const FTrackCreatorAndTrack& B)
		{
			const int32 SortOrderPriorityA = A.Creator->GetSortOrderPriority();
			const int32 SortOrderPriorityB = B.Creator->GetSortOrderPriority();

			if (SortOrderPriorityA != SortOrderPriorityB)
			{
				return SortOrderPriorityA > SortOrderPriorityB;
			}

			return A.Creator->GetName().ToString() < B.Creator->GetName().ToString();
		});
}

TSharedPtr<SWidget> FRewindDebuggerObjectTrack::GetDetailsViewInternal()
{
	if (PrimaryChildTrack.IsValid())
	{
		if (TSharedPtr<SWidget> DetailsViewOverride = PrimaryChildTrack->GetDetailsView())
		{
			return DetailsViewOverride;
		}
	}

	return TSharedPtr<SWidget>();
}

TSharedPtr<SWidget> FRewindDebuggerObjectTrack::GetTimelineViewInternal()
{
	if (PrimaryChildTrack.IsValid())
	{
		if (TSharedPtr<SWidget> TimelineViewOverride = PrimaryChildTrack->GetTimelineView())
		{
			return TimelineViewOverride;
		}
	}

	return SNew(SEventTimelineView)
		.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
		.EventData_Raw(this, &FRewindDebuggerObjectTrack::GetExistenceRange);
}

bool FRewindDebuggerObjectTrack::HandleDoubleClickInternal()
{
	// Allow primary child track to handle double-click
	if (PrimaryChildTrack.IsValid())
	{
		if (PrimaryChildTrack->HandleDoubleClick())
		{
			return true;
		}
	}

	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	static const FName HandlerFeatureName = IRewindDebuggerDoubleClickHandler::ModularFeatureName;

	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();

	if (const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession())
	{
		const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider");

		// Cache the class hierarchy names under the lock
		TArray<FName> ClassHierarchyNames;
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			const FObjectInfo& ObjectInfo = GameplayProvider->GetObjectInfo(ObjectId);
			uint64 ClassId = ObjectInfo.ClassId;

			while (ClassId != 0)
			{
				const FClassInfo& ClassInfo = GameplayProvider->GetClassInfo(ClassId);
				ClassHierarchyNames.Add(ClassInfo.Name);
				ClassId = ClassInfo.SuperId;
			}
		}

		// Iterate handlers outside the lock since HandleDoubleClick may perform heavy operations
		bool bHandled = false;
		const int32 NumExtensions = ModularFeatures.GetModularFeatureImplementationCount(HandlerFeatureName);

		for (int32 HierarchyIndex = 0; HierarchyIndex < ClassHierarchyNames.Num() && !bHandled; ++HierarchyIndex)
		{
			for (int32 ExtensionIndex = 0; ExtensionIndex < NumExtensions; ++ExtensionIndex)
			{
				IRewindDebuggerDoubleClickHandler* Handler = static_cast<IRewindDebuggerDoubleClickHandler*>(ModularFeatures.GetModularFeatureImplementation(HandlerFeatureName, ExtensionIndex));
				if (Handler->GetTargetTypeName() == ClassHierarchyNames[HierarchyIndex])
				{
					if (Handler->HandleDoubleClick(RewindDebugger))
					{
						bHandled = true;
						break;
					}
				}
			}
		}
	}

	return true;
}

FText FRewindDebuggerObjectTrack::GetStepCommandTooltipInternal(const EStepMode StepMode) const
{
	// Allow primary child track to override step tooltip
	if (PrimaryChildTrack.IsValid())
	{
		return PrimaryChildTrack->GetStepCommandTooltip(StepMode);
	}

	return {};
}

TOptional<double> FRewindDebuggerObjectTrack::GetStepFrameTimeInternal(const EStepMode StepMode, const FScrubTimeInformation& CurrentScrubTime) const
{
	// Allow primary child track to override step frame time
	if (PrimaryChildTrack.IsValid())
	{
		return PrimaryChildTrack->GetStepFrameTime(StepMode, CurrentScrubTime);
	}

	return {};
}

TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> FRewindDebuggerObjectTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	for (const FTrackCreatorAndTrack& Creator : ChildTrackCreators)
	{
		if (Creator.Track.IsValid())
		{
			OutTracks.Push(Creator.Track);
		}
	}

	return Children;
}

FName FRewindDebuggerObjectTrack::GetNameInternal() const
{
	if (PrimaryChildTrack.IsValid())
	{
		const FName NameOverride = PrimaryChildTrack->GetName();
		if (!NameOverride.IsNone())
		{
			return NameOverride;
		}
	}

	return NAME_None;
}

FSlateIcon FRewindDebuggerObjectTrack::GetIconInternal()
{
	if (PrimaryChildTrack.IsValid())
	{
		const FSlateIcon IconOverride = PrimaryChildTrack->GetIcon();
		if (IconOverride.IsSet())
		{
			return IconOverride;
		}
	}

	return Icon;
};

FText FRewindDebuggerObjectTrack::GetDisplayNameInternal() const
{
	if (PrimaryChildTrack.IsValid())
	{
		const FText DisplayNameOverride = PrimaryChildTrack->GetDisplayName();
		if (!DisplayNameOverride.IsEmpty())
		{
			return DisplayNameOverride;
		}
	}

	if (!bDisplayNameValid)
	{
		if (ObjectId.IsSet())
		{
			IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
			const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession();
			const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider");
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

			const FObjectInfo& ObjectInfo = GameplayProvider->GetObjectInfo(ObjectId);

			DisplayName = FText::FromString(ObjectInfo.Name);

			if (const FWorldInfo* WorldInfo = GameplayProvider->FindWorldInfoFromObject(ObjectId.GetMainId()))
			{
				if (WorldInfo->NetMode == FWorldInfo::ENetMode::DedicatedServer)
				{
					DisplayName = FText::Format(NSLOCTEXT("RewindDebuggerTrack", " (Server)", "{0} (Server)"), FText::FromString(ObjectInfo.Name));
				}
			}
		}
		else
		{
			DisplayName = FText::FromString(ObjectName);
		}

		bDisplayNameValid = true;
	}

	return DisplayName;
}

uint64 FRewindDebuggerObjectTrack::GetObjectIdInternal() const
{
	return ObjectId.GetMainId();
}

bool FRewindDebuggerObjectTrack::HasDebugDataInternal() const
{
	if (PrimaryChildTrack.IsValid())
	{
		return PrimaryChildTrack->HasDebugData();
	}
	return false;
}

bool FRewindDebuggerObjectTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebuggerObjectTrack::UpdateInternal);
	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();

	bool bChanged = false;

	const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession();
	const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider");
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

		const TRange<double> Existence = GameplayProvider->GetObjectRecordingLifetime(ObjectId);

		ExistenceRange->EventWindows.SetNum(0, EAllowShrinking::No);
		if (Existence.HasLowerBound() && Existence.HasUpperBound())
		{
			double EndTime = Existence.GetUpperBoundValue();
			if (EndTime == std::numeric_limits<double>::infinity())
			{
				EndTime = GameplayProvider->GetRecordingDuration();
			}
			ExistenceRange->EventWindows.Add({ Existence.GetLowerBoundValue(), EndTime, LOCTEXT("Object Existence","Object Existence"), LOCTEXT("Object Existence","Object Existence"), FLinearColor(0.1f,0.11f,0.1f) });
		}

		if (!bIconSearched)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebuggerObjectTrack::FindIcon);
			if (const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(ObjectId))
			{
				Icon = GameplayProvider->FindIconForClass(ObjectInfo->ClassId);
			}

			if (!Icon.IsSet())
			{
				Icon = FSlateIconFinder::FindIconForClass(UObject::StaticClass());
			}
			bIconSearched = true;
			bChanged = true;
		}
	}

	TArray<FObjectId, TInlineAllocator<32>> FoundObjects;

	FoundObjects.Add(ObjectId); // prevent debug views from being removed

	// add debug views as children and make sure primary child track is up-to-date
	PrimaryChildTrack.Reset();

	for (FTrackCreatorAndTrack& CreatorAndTrack : ChildTrackCreators)
	{
		const bool bHasDebugInfo = CreatorAndTrack.Creator->HasDebugInfo(ObjectId);

		if (CreatorAndTrack.Track.IsValid())
		{
			if (!bHasDebugInfo)
			{
				bChanged = true;
				CreatorAndTrack.Track.Reset();
			}
		}
		else
		{
			if (bHasDebugInfo)
			{
				bChanged = true;
				CreatorAndTrack.Track = CreatorAndTrack.Creator->CreateTrack(ObjectId);
			}
		}

		if (CreatorAndTrack.Track.IsValid()
			&& CreatorAndTrack.Creator->IsCreatingPrimaryChildTrack())
		{
			ensureMsgf(!PrimaryChildTrack.IsValid(), TEXT("Multiple primary tracks is not supported yet."));
			PrimaryChildTrack = CreatorAndTrack.Track;
			// Hide the track since its data will be exposed by the current Object track
			PrimaryChildTrack->SetHiddenFlag(ETrackHiddenFlags::HiddenByCode);
		}
	}

	// Fallback code path to add views with no track implementation
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebuggerObjectTrack::UpdateInternal_AddViews);

		if (TArray<const TCHAR*> TypeNames; GetTypeHierarchyNames(ObjectId, *Session, TypeNames))
		{
			FRewindDebuggerViewCreators::EnumerateCreators([ObjectId = ObjectId.GetMainId(), &Children = Children, &bChanged, &TypeNames](const IRewindDebuggerViewCreator* Creator)
				{
					const int32 FoundIndex = Children.FindLastByPredicate([Creator](const TSharedPtr<FRewindDebuggerTrack>& Track) { return Track->GetName() == Creator->GetName(); });

					const bool bHasDebugInfo = Creator->HasDebugInfo(ObjectId)
						&& TypeNames.FindByPredicate([TargetTypeName = Creator->GetTargetTypeName()](const TCHAR* TypeName) { return TargetTypeName == TypeName; });

					if (FoundIndex >= 0)
					{
						if (!bHasDebugInfo)
						{
							bChanged = true;
							Children.RemoveAt(FoundIndex);
						}
					}
					else
					{
						if (bHasDebugInfo)
						{
							bChanged = true;
							const TSharedPtr<FRewindDebuggerTrack> Track = MakeShared<FRewindDebuggerFallbackTrack>(ObjectId, Creator);
							Children.Add(Track);
						}
					}
				});
		}
	}

	// add child objects
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebuggerObjectTrack::UpdateInternal_AddChildComponents);

		TRange<double> ViewRange = RewindDebugger->GetCurrentViewRange();

		auto AddChildObject = [this, &FoundObjects, &bChanged, &ViewRange, GameplayProvider](const FObjectId& SubObjectId)
		{
			const TRange<double> Lifetime = GameplayProvider->GetObjectRecordingLifetime(SubObjectId);
			const TRange<double> Overlap = TRange<double>::Intersection(Lifetime, ViewRange);
			// only display the track if the lifetime of the object and the view range overlap
			if (!Overlap.IsEmpty())
			{
				const int32 FoundIndex = Children.FindLastByPredicate([SubObjectId](const TSharedPtr<FRewindDebuggerTrack>& Track)
					{
						return Track->GetAssociatedObjectId() == SubObjectId;
					});

				if (FoundIndex == INDEX_NONE)
				{
					const FObjectInfo& ObjectInfo = GameplayProvider->GetObjectInfo(SubObjectId);
					Children.Add(MakeShared<FRewindDebuggerObjectTrack>(SubObjectId, ObjectInfo.Name));
					bChanged = true;
				}

				FoundObjects.Add(FObjectId{SubObjectId});
			}
		};

		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
		GameplayProvider->EnumerateSubobjects(ObjectId, [this, AddChildObject](const FObjectId& SubObjectId)
		{
			AddChildObject(SubObjectId);
		});

#if OBJECT_TRACE_ENABLED
		// for now we'll hard code the level sequence special case.
		// the level sequence is not part of the same world - and it is not using an outer
		// relationship to the level sequence actor. We introduce a fake parenting relationship
		// here to allow inspection of the level sequence under the level sequence actor.
		if (const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(ObjectId))
		{
			if (const FClassInfo* LevelSequenceActorClassInfo = GameplayProvider->FindClassInfo(*ALevelSequenceActor::StaticClass()->GetPathName()))
			{
				const uint64 LevelSequenceActorClassId = LevelSequenceActorClassInfo->Id;
				bool bIsLevelSequenceActor = false;
	
				uint64 ClassId = ObjectInfo->ClassId;
				while (ClassId != FObjectId::InvalidId)
				{
					if (ClassId == LevelSequenceActorClassId)
					{
						bIsLevelSequenceActor = true;
						break;
					}
					const FClassInfo& ClassInfo = GameplayProvider->GetClassInfo(ClassId);
					ClassId = ClassInfo.SuperId;
				}

				if (bIsLevelSequenceActor)
				{
					const ALevelSequenceActor* LevelSequenceActor = Cast<ALevelSequenceActor>(FObjectTrace::GetObjectFromId(ObjectId.GetMainId()));
					if (LevelSequenceActor)
					{
						const FObjectId LevelSequenceObjectId(FObjectTrace::GetObjectId(LevelSequenceActor->LevelSequenceAsset));
						AddChildObject(LevelSequenceObjectId);
					}
				}
			}
		}
#endif
	}

	// add controller and it's component hierarchy if one is attached
	if (bAddController)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebuggerObjectTrack::FindController);
		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
		// Should probably update this to use a time range and return all possessing controllers from the visible time range.  For now just returns the one at the current time.
		if (uint64 ControllerId = GameplayProvider->FindPossessingController(ObjectId.GetMainId(), RewindDebugger->CurrentTraceTime()))
		{
			const FObjectInfo& ObjectInfo = GameplayProvider->GetObjectInfo(ControllerId);
			const int32 FoundIndex = Children.FindLastByPredicate([ObjectInfo](const TSharedPtr<FRewindDebuggerTrack>& Track)
				{
					return Track->GetAssociatedObjectId() == ObjectInfo.GetId();
				});

			if (FoundIndex < 0)
			{
				bChanged = true;
				Children.Add(MakeShared<FRewindDebuggerObjectTrack>(ObjectInfo.GetId(), ObjectInfo.Name));
			}

			FoundObjects.Add(FObjectId{ControllerId});
		}
	}

	// remove any components previously in the list that were not found in this time range.
	for (int Index = Children.Num() - 1; Index >= 0; Index--)
	{
		if (!FoundObjects.Contains(Children[Index]->GetAssociatedObjectId()))
		{
			bChanged = true;
			Children.RemoveAt(Index);
		}
	}

	if (bChanged)
	{
		const URewindDebuggerSettings* UserSettings = GetDefault<URewindDebuggerSettings>();
		if (UserSettings->ObjectTrackSortMode == ERewindDebuggerObjectSortMode::Name)
		{
			// sort child object tracks by name
			// note that stable track ordering requires non-empty display names
			Children.Sort([](const TSharedPtr<FRewindDebuggerTrack>& A, const TSharedPtr<FRewindDebuggerTrack>& B)
				{
					return A->GetDisplayName().ToString() < B->GetDisplayName().ToString();
				});
		}
		else
		{
			// sort child object tracks by Id which will generally make older objects first because ids are allocated as increasing integers
			Children.Sort([](const TSharedPtr<FRewindDebuggerTrack>& A, const TSharedPtr<FRewindDebuggerTrack>& B)
				{
					return A->GetUObjectId() < B->GetUObjectId();
				});
		}
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebuggerObjectTrack::UpdateChilden);
		for (const TSharedPtr<FRewindDebuggerTrack>& Child : Children)
		{
			if (Child->Update())
			{
				UE_LOGF(LogRewindDebugger, Verbose, "List changed by: '%ls'", *Child->GetDisplayName().ToString());
				bChanged = true;
			}
		}
	}
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebuggerObjectTrack::UpdateTrackChilden);
		for (FTrackCreatorAndTrack& Creator : ChildTrackCreators)
		{
			if (Creator.Track.IsValid())
			{
				if (Creator.Track->Update())
				{
					UE_LOGF(LogRewindDebugger, Verbose, "List changed by: '%ls'", *Creator.Track->GetDisplayName().ToString());
					bChanged = true;
				}
			}
		}
	}

	return bChanged;
}

} // namespace RewindDebugger

#undef LOCTEXT_NAMESPACE