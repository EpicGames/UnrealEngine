// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RewindDebuggerTrack.h"
#include "SEventTimelineView.h"


namespace RewindDebugger
{

class IRewindDebuggerTrackCreator;

struct FTrackCreatorAndTrack
{
	const IRewindDebuggerTrackCreator* Creator;
	TSharedPtr<FRewindDebuggerTrack> Track;
};

class FRewindDebuggerObjectTrack : public FRewindDebuggerTrack
{
public:

	FRewindDebuggerObjectTrack(const FObjectId& InObjectId, const FString& InObjectName, bool bInAddController = false);

	TSharedPtr<SEventTimelineView::FTimelineEventData> GetExistenceRange() const
	{
		return ExistenceRange;
	}

private:
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
	virtual bool UpdateInternal() override;
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;
	virtual FName GetNameInternal() const override;
	virtual FSlateIcon GetIconInternal() override;
	virtual FText GetDisplayNameInternal() const override;
	virtual uint64 GetObjectIdInternal() const override;
	virtual bool HasDebugDataInternal() const override;
	virtual bool HandleDoubleClickInternal() override;
	virtual FText GetStepCommandTooltipInternal(EStepMode) const override;
	virtual TOptional<double> GetStepFrameTimeInternal(EStepMode, const FScrubTimeInformation&) const override;

	mutable FText DisplayName;
	FString ObjectName;
	FSlateIcon Icon;
	TSharedPtr<SEventTimelineView::FTimelineEventData> ExistenceRange;
	FObjectId ObjectId;
	TArray<FTrackCreatorAndTrack> ChildTrackCreators;
	TArray<TSharedPtr<FRewindDebuggerTrack>> Children;

	/** This child track will be used by the current object track to replace its own appearance and timeline values */
	TSharedPtr<FRewindDebuggerTrack> PrimaryChildTrack;

	bool bAddController;
	mutable bool bDisplayNameValid;
	bool bIconSearched;
};

}