// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerRecordingControls.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "RewindDebugger.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "RewindDebuggerEngineEditorBridge.h"
#include "RewindDebuggerModule.h"
#include "RewindDebuggerRemoteSessionsHandler.h"
#include "RewindDebuggerStyle.h"
#include "SessionInfo.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "RewindDebuggerRecordingControls"

namespace UE::RewindDebugger
{

const FName FDebuggerRecordingControls::RecordingOptionsMenuName = "RewindDebugger.RecordingOptions";

FDebuggerRecordingControls::FDebuggerRecordingControls(const FName& InStatusBarId
	, const TSharedRef<TraceBasedDebuggers::FRemoteSessionsManager>& InRemoteSessionManager
	, const FGuid& InDebuggerId
	, const FLogCategoryBase& InLogCategory)
	: FRecordingControls(InStatusBarId, InRemoteSessionManager, InDebuggerId, InLogCategory)
{
	StartRecordingBrush = FRewindDebuggerStyle::Get().GetBrush("RewindDebugger.StartRecording");
	StopRecordingBrush = FRewindDebuggerStyle::Get().GetBrush("RewindDebugger.StopRecording");
	StatusBarRecordingBrush = FRewindDebuggerStyle::Get().GetBrush("RewindDebugger.StartRecording.StatusBar");
	RecordingSessionIcon = FSlateIcon("RewindDebuggerStyle", FName("RewindDebugger.StartRecording"), FName("RewindDebugger.StartRecording"));

#if WITH_TRACE_BASED_DEBUGGERS
	TraceRelayTransportInstanceWeakPtr = FRewindDebuggerEngineEditorBridge::Get().GetTraceRelayTransportInstance().ToWeakPtr();
#endif

	// Register the recording options menu
	if (!UToolMenus::Get()->IsMenuRegistered(RecordingOptionsMenuName))
	{
		UToolMenu* OptionsMenu = UToolMenus::Get()->RegisterMenu(RecordingOptionsMenuName, FRecordingControls::BaseRecordingOptionsMenuName);
		FToolMenuSection& ActionsSection = OptionsMenu->FindOrAddSection("FolderActions", LOCTEXT("FolderOptionsLabel", "Folder Actions"));
		ActionsSection.AddEntry(FToolMenuEntry::InitMenuEntry(
			"OpenRecordingsFolder",
			LOCTEXT("OpenRecordingsFolderLabel", "Open Recordings Folder"),
			LOCTEXT("OpenRecordingsFolderTooltip", "Open trace analysis recordings folder"),
			FSlateIcon("RewindDebuggerStyle", "RewindDebugger.OpenRecordingDirectory"),
			FUIAction(FExecuteAction::CreateLambda([]()
			{
				if (const TSharedPtr<UE::TraceBasedDebuggers::FTraceSessionsManager> TraceSessionsManager = FRewindDebuggerModule::Get().GetTraceSessionsManager())
				{
					FPlatformProcess::ExploreFolder(*TraceSessionsManager->GetSaveDirPath());
				}
			}))
		));
		ActionsSection.AddEntry(FToolMenuEntry::InitMenuEntry(
			"DeleteAllRecordings",
			LOCTEXT("DeleteAllRecordingsLabel", "Delete Old Recordings"),
			LOCTEXT("DeleteAllRecordingsTooltip", "Delete all old trace analysis files"),
			FSlateIcon("RewindDebuggerStyle", "RewindDebugger.DeleteOldTraces"),
			FUIAction(FExecuteAction::CreateLambda([]()
			{
				if (const TSharedPtr<UE::TraceBasedDebuggers::FTraceSessionsManager> TraceSessionsManager = FRewindDebuggerModule::Get().GetTraceSessionsManager())
				{
					const FString Dir = TraceSessionsManager->GetSaveDirPath();
					TArray<FString> Files;
					IFileManager::Get().FindFiles(Files, *Dir, TEXT("utrace"));
					for (const FString& File : Files)
					{
						IFileManager::Get().Delete(*FString::Printf(TEXT("%s/%s"), *Dir, *File));
					}
				}
			}))
		));
	}
}

bool FDebuggerRecordingControls::IsRecordingLocalSession() const
{
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FSessionInfo> SessionInfo = GetLocalSessionInfo())
	{
		return SessionInfo->IsRecording(DebuggerGuid);
	}

	return false;
}

TSharedPtr<TraceBasedDebuggers::FSessionInfo> FDebuggerRecordingControls::GetLocalSessionInfo() const
{
	using namespace UE::TraceBasedDebuggers;
	TSharedPtr<FSessionInfo> LocalSessionInfo;
	if (const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = RemoteSessionManagerWeakPtr.Pin())
	{
		RemoteSessionManager->EnumerateActiveSessions([&LocalSessionInfo](const TSharedRef<FSessionInfo>& InSessionInfoRef)
			{
				bool bKeepIterating = true;
				if (InSessionInfoRef->SessionName == FRemoteSessionsManager::LocalEditorSessionName)
				{
					LocalSessionInfo = InSessionInfoRef;
					bKeepIterating = false;
				}
				return bKeepIterating;
			});
	}

	return LocalSessionInfo;
}

TraceBasedDebuggers::FRecordingControlsConfig FDebuggerRecordingControls::GetControlsConfig() const
{
	return {
		.OptionsMenuName = RecordingOptionsMenuName,
		.bHasRecordingLabel = false,
		.SectionInsertPosition = FToolMenuInsert("PlaybackControls", EToolMenuInsertType::After),
	};
}

void FDebuggerRecordingControls::AddToMenuInternal(FToolMenuSection& InSection)
{
	FToolMenuEntry Separator = FToolMenuEntry::InitSeparator(NAME_None);
	Separator.InsertPosition = FToolMenuInsert("AvailableSessions", EToolMenuInsertType::Before);
	InSection.AddEntry(Separator);
}

TSharedRef<SWidget> FDebuggerRecordingControls::GenerateTargetSessionSelector()
{
	return SNew(SComboButton)
		.MenuPlacement(MenuPlacement_AboveAnchor)
		.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar").ComboButtonStyle)
		.OnGetMenuContent(FOnGetContent::CreateSPLambda(this, [this]()
			{
				return GenerateTargetSessionDropdown();
			}))
		.HasDownArrow(true)
		.ToolTipText(MakeAttributeSPLambda(this, [this]() -> FText
			{
				if (const TSharedPtr<TraceBasedDebuggers::FSessionInfo> SessionInfo = GetCurrentSessionInfo())
				{
					return FText::Format(LOCTEXT("SessionSelectorCurrentTarget", "Target: {0}"), FText::AsCultureInvariant(SessionInfo->SessionName));
				}
				return LOCTEXT("SessionSelectorNoTarget", "No target session selected\nOpen to select a session");
			}))
		.ButtonContent()
		[
			SNew(SImage)
			.Image(FRewindDebuggerStyle::Get().GetBrush("RewindDebugger.ConnectToSession"))
			.ColorAndOpacity(MakeAttributeSPLambda(this, [this]() -> FSlateColor
				{
					if (const TSharedPtr<TraceBasedDebuggers::FSessionInfo> SessionInfo = GetCurrentSessionInfo())
					{
						// Accent color only for non-editor targets (standalone, game, device)
						if (SessionInfo->BuildTargetType != EBuildTargetType::Editor)
						{
							return FStyleColors::AccentBlue;
						}
					}
					
					return FSlateColor::UseStyle();
				}))
		];
}

FString FDebuggerRecordingControls::GetSaveDirPath() const
{
	if (const TSharedPtr<TraceBasedDebuggers::FTraceSessionsManager> TraceSessionsManager = FRewindDebuggerModule::Get().GetTraceSessionsManager())
	{
		return TraceSessionsManager->GetSaveDirPath();
	}
	return TEXT("");
}

bool FDebuggerRecordingControls::CanStartRecordingLocalSession() const
{
	using namespace UE::TraceBasedDebuggers;
	const TSharedPtr<FSessionInfo> LocalSessionInfo = GetLocalSessionInfo();
	return LocalSessionInfo
		&& !LocalSessionInfo->IsRecording(DebuggerGuid)
		&& IsRecordingAvailableForSessionInternal(LocalSessionInfo.ToSharedRef());
}

void FDebuggerRecordingControls::StartRecordingLocalSession()
{
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FSessionInfo> LocalSessionInfo = GetLocalSessionInfo())
	{
		if (!LocalSessionInfo->IsRecording(DebuggerGuid))
		{
			ToggleRecordingState(LocalSessionInfo.ToSharedRef());
		}

		SelectTargetSession(LocalSessionInfo->InstanceId);
	}
}

void FDebuggerRecordingControls::StopRecordingLocalSession()
{
	using namespace TraceBasedDebuggers;
	const TSharedPtr<FSessionInfo> LocalSessionInfo = GetLocalSessionInfo();
	if (LocalSessionInfo
		&& LocalSessionInfo->IsRecording(DebuggerGuid))
	{
		ToggleRecordingState(LocalSessionInfo.ToSharedRef());
	}
}

UE::TraceBasedDebuggers::ETraceTransportMode FDebuggerRecordingControls::GetTransportModeOverrideForTargetTypeInternal(const EBuildTargetType InBuildTarget) const
{
	return URewindDebuggerSettings::Get().GetTransportModeForTargetType(InBuildTarget);
}

bool FDebuggerRecordingControls::ConnectToLiveSession_RelayInternal(const FGuid& InSessionId)
{
#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS
	return FRewindDebugger::Instance()->ConnectToLiveSession_Relay(InSessionId);
#else
	return false;
#endif
}

bool FDebuggerRecordingControls::ConnectToLiveSession_DirectInternal(const FGuid& InSessionId)
{
#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS
	return FRewindDebugger::Instance()->ConnectToLiveSession_Direct(InSessionId);
#else
	return false;
#endif
}

bool FDebuggerRecordingControls::IsRecordingAvailableForSessionInternal(const TSharedRef<TraceBasedDebuggers::FSessionInfo>& InSessionInfo) const
{
	return (InSessionInfo->BuildTargetType == EBuildTargetType::Editor
			&& (FRewindDebugger::Instance()->IsPIEStarted()))
		|| InSessionInfo->BuildTargetType != EBuildTargetType::Editor;
}

bool FDebuggerRecordingControls::ShouldFilterOutSessionInternal(const TSharedRef<TraceBasedDebuggers::FSessionInfo>& InSessionInfo) const
{
	// Do not allow users to see and start recordings in other Editor processes (locally or on the network)
	return (InSessionInfo->InstanceId != TraceBasedDebuggers::FRemoteSessionsManager::LocalEditorSessionID
		&& InSessionInfo->BuildTargetType == EBuildTargetType::Editor);
}

TNotNull<TUniquePtr<TraceBasedDebuggers::FStartRecordingCommandMessage>> FDebuggerRecordingControls::BuildStartRecordingParamsInternal()
{
	return MakeUnique<FStartRecordingCommand>();
}

void FDebuggerRecordingControls::SendStartRecordingCommandInternal(const FMessageAddress& InAddress, TUniquePtr<TraceBasedDebuggers::FStartRecordingCommandMessage> InStartRecordingCommand)
{
	if (const TSharedPtr<TraceBasedDebuggers::FRemoteSessionsManager> SessionsManager = RemoteSessionManagerWeakPtr.Pin())
	{
		const FStartRecordingCommand* StartRecordingCommand = static_cast<FStartRecordingCommand*>(InStartRecordingCommand.Get());
		SessionsManager->SendCommand(InAddress, *StartRecordingCommand);
	}
}

void FDebuggerRecordingControls::SendStopRecordingCommandInternal(const FMessageAddress& InAddress)
{
	if (const TSharedPtr<TraceBasedDebuggers::FRemoteSessionsManager> SessionsManager = RemoteSessionManagerWeakPtr.Pin())
	{
		SessionsManager->SendCommand(InAddress, FStopRecordingCommand{});
	}
}

void FDebuggerRecordingControls::OnToggleRecordingStateInternal(const TSharedRef<TraceBasedDebuggers::FSessionInfo>& InSessionInfo)
{
	ToggleSingleSessionRecordingState(InSessionInfo);
}

void FDebuggerRecordingControls::CloseSessionByRemoteSessionIDInternal(const FGuid& InSessionId)
{
#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS
	FRewindDebugger::Instance()->ClearAnalysisSessionLinkedToRemoteSessionID(InSessionId);
#endif
}

const TraceBasedDebuggers::FTraceSessionDescriptor& FDebuggerRecordingControls::GetCurrentSessionDescriptorInternal()
{
#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS
	return FRewindDebugger::Instance()->GetCurrentSessionDescriptor();
#endif
}

} // UE::RewindDebugger

#undef LOCTEXT_NAMESPACE