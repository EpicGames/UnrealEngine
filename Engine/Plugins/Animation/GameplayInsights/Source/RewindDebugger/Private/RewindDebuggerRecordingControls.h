// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/RecordingControls.h"

namespace UE::RewindDebugger
{

struct FDebuggerRecordingControls : TraceBasedDebuggers::FRecordingControls
{
	FDebuggerRecordingControls(const FName& InStatusBarId
		, const TSharedRef<TraceBasedDebuggers::FRemoteSessionsManager>& InRemoteSessionManager
		, const FGuid& InDebuggerId
		, const FLogCategoryBase& InLogCategory);

	using FRecordingControls::IsRecording;
	bool IsRecordingLocalSession() const;

	bool CanStartRecordingLocalSession() const;

	/** Starts recording in the local session and makes it the selected session. */
	void StartRecordingLocalSession();

	void StopRecordingLocalSession();

protected:
	TSharedPtr<TraceBasedDebuggers::FSessionInfo> GetLocalSessionInfo() const;

	virtual TraceBasedDebuggers::FRecordingControlsConfig GetControlsConfig() const override;
	virtual TSharedRef<SWidget> GenerateTargetSessionSelector() override;
	virtual void AddToMenuInternal(FToolMenuSection& InSection) override;
	virtual FString GetSaveDirPath() const override;

	virtual UE::TraceBasedDebuggers::ETraceTransportMode GetTransportModeOverrideForTargetTypeInternal(EBuildTargetType InBuildTarget) const override;
	virtual bool ConnectToLiveSession_RelayInternal(const FGuid& InSessionId) override;
	virtual bool ConnectToLiveSession_DirectInternal(const FGuid& InSessionId) override;
	virtual bool IsRecordingAvailableForSessionInternal(const TSharedRef<TraceBasedDebuggers::FSessionInfo>& InSessionInfo) const override;
	virtual bool ShouldFilterOutSessionInternal(const TSharedRef<TraceBasedDebuggers::FSessionInfo>& InSessionInfo) const override;

	virtual TNotNull<TUniquePtr<TraceBasedDebuggers::FStartRecordingCommandMessage>> BuildStartRecordingParamsInternal() override;
	virtual void SendStartRecordingCommandInternal(const FMessageAddress& InAddress, TUniquePtr<TraceBasedDebuggers::FStartRecordingCommandMessage> InStartRecordingCommand) override;
	virtual void SendStopRecordingCommandInternal(const FMessageAddress& InAddress) override;

	virtual void OnToggleRecordingStateInternal(const TSharedRef<TraceBasedDebuggers::FSessionInfo>& InSessionInfo) override;
	virtual void CloseSessionByRemoteSessionIDInternal(const FGuid& InSessionId) override;
	virtual const TraceBasedDebuggers::FTraceSessionDescriptor& GetCurrentSessionDescriptorInternal() override;
	
	static const FName RecordingOptionsMenuName;
};

}
