// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerSettings.h"

#include "GameFramework/Actor.h"
#include "IRewindDebugger.h"
#include "Misc/CoreDelegates.h"
#include "TraceDataRelayTransport.h"
#include "UObject/SoftObjectPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RewindDebuggerSettings)

#define LOCTEXT_NAMESPACE "RewindDebuggerSettings"

URewindDebuggerProjectSettings::URewindDebuggerProjectSettings()
{
	SelectorAllowedTypes.Add(FSoftClassPath(AActor::StaticClass()));
}

FName URewindDebuggerProjectSettings::GetCategoryName() const
{
	return FName(TEXT("Plugins"));
}

URewindDebuggerSettings::URewindDebuggerSettings()
	: CameraMode(ERewindDebuggerCameraMode::Replay)
	, bShouldAutoEject(false)
	, bShouldAutoRecordOnPIE(false)
	, GameServerDataTransportMode(UE::TraceBasedDebuggers::ETraceTransportMode::Direct)
	, GameClientDataTransportMode(UE::TraceBasedDebuggers::ETraceTransportMode::Direct)
{
	OnPreExitHandle = FCoreDelegates::OnPreExit.AddLambda([]()
	{
		GetMutableDefault<URewindDebuggerSettings>()->SaveConfig();
	});
}

URewindDebuggerSettings::~URewindDebuggerSettings()
{
	FCoreDelegates::OnPreExit.Remove(OnPreExitHandle);
}

#if WITH_EDITOR

FText URewindDebuggerProjectSettings::GetSectionDescription() const
{
	return LOCTEXT("RewindDebuggerProjectSettingsDescription", "Configure project options for the Rewind Debugger.");
}

FText URewindDebuggerSettings::GetSectionText() const
{
	return LOCTEXT("RewindDebuggerSettingsName", "Rewind Debugger");
}

FText URewindDebuggerSettings::GetSectionDescription() const
{
	return LOCTEXT("RewindDebuggerSettingsDescription", "Configure user options for the Rewind Debugger.");
}

#endif

FName URewindDebuggerSettings::GetCategoryName() const
{
	return FName(TEXT("Plugins"));
}

UE::TraceBasedDebuggers::ETraceTransportMode URewindDebuggerSettings::GetTransportModeForTargetType(const EBuildTargetType InTargetType) const
{
	switch (InTargetType)
	{
	case EBuildTargetType::Server:
		return GameServerDataTransportMode;
	case EBuildTargetType::Game:
	case EBuildTargetType::Client:
		return GameClientDataTransportMode;
	case EBuildTargetType::Editor:
	case EBuildTargetType::Unknown:
	case EBuildTargetType::Program:
	default:
		return UE::TraceBasedDebuggers::ETraceTransportMode::Direct;
	}
}

void URewindDebuggerSettings::SetTransportModeForTargetType(const EBuildTargetType InTargetType, const UE::TraceBasedDebuggers::ETraceTransportMode InTransportMode)
{
	switch (InTargetType)
	{
	case EBuildTargetType::Server:
		GameServerDataTransportMode = InTransportMode;
		break;
	case EBuildTargetType::Game:
	case EBuildTargetType::Client:
		GameClientDataTransportMode = InTransportMode;
		break;
	case EBuildTargetType::Editor:
	case EBuildTargetType::Unknown:
	case EBuildTargetType::Program:
	default:
		UE_LOGF(LogRewindDebugger, Error, "%s Changing transport mode in the selected target type is not supported. Target type [%ls]", __func__, LexToString(InTargetType));
	}
}

URewindDebuggerSettings& URewindDebuggerSettings::Get()
{
	URewindDebuggerSettings* MutableCDO = GetMutableDefault<URewindDebuggerSettings>();
	check(MutableCDO != nullptr)
	return *MutableCDO;
}

#undef LOCTEXT_NAMESPACE