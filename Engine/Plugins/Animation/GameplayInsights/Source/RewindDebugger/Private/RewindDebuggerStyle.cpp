// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/ToolBarStyle.h"

TSharedPtr< FRewindDebuggerStyle > FRewindDebuggerStyle::StyleInstance = nullptr;


FRewindDebuggerStyle::FRewindDebuggerStyle() :
    FSlateStyleSet("RewindDebuggerStyle")
{
	const FVector2D Icon24x24(24.0f, 24.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon12x12(12.0f, 12.0f);

	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Animation/GameplayInsights/Content"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT(""));

	// Playback controls
	Set("RewindDebugger.StartRecording", new CORE_IMAGE_BRUSH_SVG("Editor/Slate/Sequencer/PlaybackControls/PlayControlsRecord", Icon20x20));
	Set("RewindDebugger.StartRecording.StatusBar", new CORE_IMAGE_BRUSH_SVG("Editor/Slate/Sequencer/PlaybackControls/PlayControlsRecord", Icon12x12));
	Set("RewindDebugger.StopRecording", new CORE_IMAGE_BRUSH_SVG("Editor/Slate/Sequencer/PlaybackControls/PlayControlsStop", Icon20x20));
	Set("RewindDebugger.FirstFrame", new CORE_IMAGE_BRUSH_SVG("Editor/Slate/Sequencer/PlaybackControls/PlayControlsToFront", Icon20x20));
	Set("RewindDebugger.PreviousFrame", new CORE_IMAGE_BRUSH_SVG("Editor/Slate/Sequencer/PlaybackControls/PlayControlsToPrevious", Icon20x20));
	Set("RewindDebugger.ReversePlay", new CORE_IMAGE_BRUSH_SVG("Editor/Slate/Sequencer/PlaybackControls/PlayControlsPlayReverse", Icon20x20));
	Set("RewindDebugger.Pause", new CORE_IMAGE_BRUSH_SVG("Editor/Slate/Sequencer/PlaybackControls/PlayControlsPause", Icon20x20));
	Set("RewindDebugger.Play", new CORE_IMAGE_BRUSH_SVG("Editor/Slate/Sequencer/PlaybackControls/PlayControlsPlayForward", Icon20x20));
	Set("RewindDebugger.NextFrame", new CORE_IMAGE_BRUSH_SVG("Editor/Slate/Sequencer/PlaybackControls/PlayControlsToNext", Icon20x20));
	Set("RewindDebugger.LastFrame", new CORE_IMAGE_BRUSH_SVG("Editor/Slate/Sequencer/PlaybackControls/PlayControlsToEnd", Icon20x20));
	Set("RewindDebugger.AutoScroll", new CORE_IMAGE_BRUSH_SVG("Slate/Starship/Insights/AutoScrollRight_20", Icon20x20));

	// Automatic actions
	Set("RewindDebugger.AutoEject", new IMAGE_BRUSH_SVG("autoeject", Icon20x20));
	Set("RewindDebugger.AutoRecord", new IMAGE_BRUSH_SVG("autorecord", Icon20x20));

	// Trace / Analysis
	Set("RewindDebugger.ConnectToSession", new IMAGE_BRUSH_SVG("Session_20", {20.f, 20.f}));
	Set("RewindDebugger.ClearAnalysis", new CORE_IMAGE_BRUSH_SVG("Slate/Starship/Common/delete-outline", Icon16x16));
	Set("RewindDebugger.DeleteOldTraces", new CORE_IMAGE_BRUSH_SVG("Slate/Starship/Common/delete-outline", Icon16x16));
	Set("RewindDebugger.OpenRecordingDirectory", new CORE_IMAGE_BRUSH_SVG("Editor/Slate/Starship/Common/BrowseContent", Icon16x16));

	// Actor Picker
	Set("RewindDebugger.SelectActor", new CORE_IMAGE_BRUSH("Editor/Slate/Icons/eyedropper_16px", Icon16x16));

	// Tab icon
	Set("RewindDebugger.RewindIcon", new IMAGE_BRUSH("Rewind_24x", Icon16x16));
	Set("RewindDebugger.RewindDetailsIcon", new IMAGE_BRUSH("RewindDetails_24x", Icon16x16));

	// Camera mode
	Set("RewindDebugger.Camera", new CORE_IMAGE_BRUSH_SVG("Editor/Slate/Starship/AssetIcons/CameraActor_16", Icon16x16));

	// Categories
	Set("RewindDebugger.Filter", new CORE_IMAGE_BRUSH_SVG("Slate/Starship/Common/filter", Icon16x16));
	Set("RewindDebugger.Filter.Recording", new IMAGE_BRUSH_SVG("record_filter", Icon12x12));
	Set("RewindDebugger.Preview", new CORE_IMAGE_BRUSH_SVG("Slate/Starship/Common/visible", Icon16x16));

	FTableRowStyle TableRowStyle = FTableRowStyle(FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"));
	Set("RewindDebugger.TableRow", TableRowStyle);

	// Toolbar style
	{
		FToolBarStyle CompactToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar");

		// Reduce button padding
		FButtonStyle CompactButton = CompactToolbarStyle.ButtonStyle;
		CompactButton.SetNormalPadding(FMargin(2.f, 4.f, 2.f, 4.f)); // Changed from (8.f, 4.f, 8.f, 4.f)
		CompactButton.SetPressedPadding(FMargin(2.f, 5.f, 2.f, 3.f)); // Changed from (8.f, 5.f, 8.f, 3.f)
		CompactToolbarStyle.SetButtonStyle(CompactButton);

		// Reduce toggle button padding
		FCheckBoxStyle CompactToggle = CompactToolbarStyle.ToggleButton;
		CompactToggle.SetPadding(FMargin(2.f, 4.f, 2.f, 4.f)); // Changed from (8.f, 4.f, 8.f, 4.f)
		CompactToolbarStyle.SetToggleButtonStyle(CompactToggle);

		// Match settings button padding (used when a button is paired with a SimpleComboBox options block)
		FButtonStyle CompactSettingsButton = CompactToolbarStyle.SettingsButtonStyle;
		CompactSettingsButton.SetNormalPadding(FMargin(2.f, 4.f, 2.f, 4.f));
		CompactSettingsButton.SetPressedPadding(FMargin(2.f, 5.f, 2.f, 3.f));
		CompactToolbarStyle.SetSettingsButtonStyle(CompactSettingsButton);

		Set("RewindDebugger.Toolbar", CompactToolbarStyle);

		// Primary action variant, same compact style but with labels visible
		FToolBarStyle PrimaryStyle = CompactToolbarStyle;
		PrimaryStyle.SetShowLabels(true);
		PrimaryStyle.SetLabelPadding(FMargin(2.f, 0.f, 0.f, 0.f));
		Set("RewindDebugger.Toolbar.Primary", PrimaryStyle);
	}
}

void FRewindDebuggerStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = MakeShared<FRewindDebuggerStyle>();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FRewindDebuggerStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

const ISlateStyle& FRewindDebuggerStyle::Get()
{
	return *StyleInstance;
}
