// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

/**
* Implements the visual style of RewindDebugger
*/
class FRewindDebuggerStyle : FSlateStyleSet
{
public:
    FRewindDebuggerStyle();
	static void Initialize();
	static void Shutdown();

	static const ISlateStyle& Get();

private:

	static TSharedPtr< FRewindDebuggerStyle > StyleInstance;
};

