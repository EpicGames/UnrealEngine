// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerTrackCreators.h"
#include "Features/IModularFeatures.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "IGameplayProvider.h"

namespace RewindDebugger
{

void FRewindDebuggerTrackCreators::EnumerateCreators(TFunctionRef<void(const IRewindDebuggerTrackCreator*)> Callback)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	static const FName CreatorFeatureName = IRewindDebuggerTrackCreator::ModularFeatureName;

	const int32 NumExtensions = ModularFeatures.GetModularFeatureImplementationCount(CreatorFeatureName);
	for (int32 ExtensionIndex = 0; ExtensionIndex < NumExtensions; ++ExtensionIndex)
	{
		IRewindDebuggerTrackCreator* TrackCreator = static_cast<IRewindDebuggerTrackCreator*>(ModularFeatures.GetModularFeatureImplementation(CreatorFeatureName, ExtensionIndex));
		Callback(TrackCreator);
	}
}

const IRewindDebuggerTrackCreator* FRewindDebuggerTrackCreators::GetCreator(FName CreatorName)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	static const FName CreatorFeatureName = IRewindDebuggerTrackCreator::ModularFeatureName;

	const int32 NumExtensions = ModularFeatures.GetModularFeatureImplementationCount(CreatorFeatureName);
	for (int32 ExtensionIndex = 0; ExtensionIndex < NumExtensions; ++ExtensionIndex)
	{
		IRewindDebuggerTrackCreator* TrackCreator = static_cast<IRewindDebuggerTrackCreator*>(ModularFeatures.GetModularFeatureImplementation(CreatorFeatureName, ExtensionIndex));
		if (TrackCreator->GetName() == CreatorName)
		{
			return TrackCreator;
		}
	}

	return nullptr;

}

}
