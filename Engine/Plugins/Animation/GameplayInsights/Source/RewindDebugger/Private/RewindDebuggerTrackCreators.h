// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RewindDebuggerTrack.h"
#include "IRewindDebuggerTrackCreator.h"

namespace RewindDebugger
{
	class FRewindDebuggerTrackCreators
	{
	public:
		static void EnumerateCreators(TFunctionRef<void(const IRewindDebuggerTrackCreator*)> Callback);
		static const IRewindDebuggerTrackCreator* GetCreator(FName CreatorName);
	};
}