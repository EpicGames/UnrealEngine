// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerViewCreators.h"
#include "Features/IModularFeatures.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "IGameplayProvider.h"

void FRewindDebuggerViewCreators::EnumerateCreators(TFunctionRef<void(const IRewindDebuggerViewCreator*)> Callback)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	static const FName CreatorFeatureName = IRewindDebuggerViewCreator::ModularFeatureName;

	const int32 NumExtensions = ModularFeatures.GetModularFeatureImplementationCount(CreatorFeatureName);
	for (int32 ExtensionIndex = 0; ExtensionIndex < NumExtensions; ++ExtensionIndex)
	{
		IRewindDebuggerViewCreator* ViewCreator = static_cast<IRewindDebuggerViewCreator*>(ModularFeatures.GetModularFeatureImplementation(CreatorFeatureName, ExtensionIndex));
        Callback(ViewCreator);
    }
}

const IRewindDebuggerViewCreator* FRewindDebuggerViewCreators::GetCreator(FName CreatorName)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	static const FName CreatorFeatureName = IRewindDebuggerViewCreator::ModularFeatureName;

	const int32 NumExtensions = ModularFeatures.GetModularFeatureImplementationCount(CreatorFeatureName);
	for (int32 ExtensionIndex = 0; ExtensionIndex < NumExtensions; ++ExtensionIndex)
	{
		IRewindDebuggerViewCreator* ViewCreator = static_cast<IRewindDebuggerViewCreator*>(ModularFeatures.GetModularFeatureImplementation(CreatorFeatureName, ExtensionIndex));
		if (ViewCreator->GetName() == CreatorName)
		{
			return ViewCreator;
		}
    }

    return nullptr;
}
