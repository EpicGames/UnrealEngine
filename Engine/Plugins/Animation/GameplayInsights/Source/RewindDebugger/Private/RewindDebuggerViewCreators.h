// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IRewindDebuggerView.h"
#include "IRewindDebuggerViewCreator.h"

namespace TraceServices
{
	class IRewindDebuggerViewCreator;
}

class FRewindDebuggerViewCreators
{
public:
	static void EnumerateCreators(TFunctionRef<void(const IRewindDebuggerViewCreator*)> Callback);
	static const IRewindDebuggerViewCreator* GetCreator(FName CreatorName);
};
