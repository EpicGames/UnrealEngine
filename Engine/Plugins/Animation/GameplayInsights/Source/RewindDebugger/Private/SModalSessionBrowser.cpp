// Copyright Epic Games, Inc. All Rights Reserved.

#include "SModalSessionBrowser.h"

#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Logging/LogMacros.h"
#include "SModalSessionNameListPicker.h"
#include "Trace/StoreClient.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "ModalSessionBrowser"

DEFINE_LOG_CATEGORY_STATIC(LogSessionBrowser, Log, All);

/** Reduced Version of the IP4 structure from the Network Module
 * used just to convert to String an IP saved as host byte order.
 */
struct FCVDIPv4Address
{
	union
	{
		/** The IP address value as A.B.C.D components. */
		struct
		{
#if PLATFORM_LITTLE_ENDIAN
#ifdef _MSC_VER
			uint8 D, C, B, A;
#else
			uint8 D GCC_ALIGN(4);
			uint8 C, B, A;
#endif
#else
			uint8 A, B, C, D;
#endif
		};

		/** The IP address value in host byte order. */
		uint32 Value;
	};
	
	/**
 * Creates and initializes a new IPv4 address with the specified components.
 *
 * The created IP address has the value A.B.C.D.
 *
 * @param InA The first component.
 * @param InB The second component.
 * @param InC The third component.
 * @param InD The fourth component.
 */
	FCVDIPv4Address(uint8 InA, uint8 InB, uint8 InC, uint8 InD)
#if PLATFORM_LITTLE_ENDIAN
		: D(InD)
		, C(InC)
		, B(InB)
		, A(InA)
#else
		: A(InA)
		, B(InB)
		, C(InC)
		, D(InD)
#endif // PLATFORM_LITTLE_ENDIAN
	{ }

	/**
	 * Creates and initializes a new IPv4 address with the specified value.
	 *
	 * @param InValue The address value (in host byte order).
	 */
	FCVDIPv4Address(uint32 InValue)
		: Value(InValue)
	{ }

	FString ToString() const { return FString::Printf(TEXT("%i.%i.%i.%i"), A, B, C, D); }
};

SModalSessionBrowser::~SModalSessionBrowser()
{
	if (FSlateApplication::IsInitialized())
	{
		if (ModalTickHandle.IsValid())
		{
			FSlateApplication::Get().GetOnModalLoopTickEvent().Remove(ModalTickHandle);
		}
	}
}

void SModalSessionBrowser::Construct(const FArguments& InArgs)
{
	CurrentTraceStoreAddress = TEXT("127.0.0.1");
	
	SWindow::Construct(SWindow::FArguments()
	.Title(LOCTEXT("SModalSessionBrowser_Title", "Live Session Browser"))
	.SupportsMinimize(false)
	.SupportsMaximize(false)
	.UserResizeBorder(0)
	.ClientSize(FVector2D(450, 120))
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(5)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(0.0f, 0.0f, 0.0f, 5.0f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TraceStoreAddress", "Trace Store Address"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 12))
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SAssignNew(TraceStoreAddressWidget, SEditableTextBox)
						.Text(FText::AsCultureInvariant(CurrentTraceStoreAddress))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 12))
						.OnTextCommitted_Raw(this, &SModalSessionBrowser::OnTraceStoreAddressUpdated)
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SelectedLiveSession", "Selected Live Session"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 12))
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SAssignNew(NamePickerWidget, SModalSessionNameListPicker)
						.OnNameSleceted_Raw(this, &SModalSessionBrowser::HandleSessionNameSelected)
					]	
				]
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(5)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
			.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
			+SUniformGridPanel::Slot(0, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("ConnectToSession", "Connect to Session"))
				.IsEnabled_Raw(this, &SModalSessionBrowser::CanOpenSession)
				.OnClicked(this, &SModalSessionBrowser::OnButtonClick, EAppReturnType::Ok)
			]
			+SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("Cancel", "Cancel"))
				.OnClicked(this, &SModalSessionBrowser::OnButtonClick, EAppReturnType::Cancel)
			]
		]
	]);

	UpdateCurrentSessionInfoMap();

	if (FSlateApplication::IsInitialized())
	{
		ModalTickHandle = FSlateApplication::Get().GetOnModalLoopTickEvent().AddRaw(this, &SModalSessionBrowser::ModalTick);
	}
}

EAppReturnType::Type SModalSessionBrowser::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

SModalSessionBrowser::FTraceSessionInfo SModalSessionBrowser::GetSelectedTraceInfo()
{
	if (ensure(CurrentTraceSessionSelected.IsValid()))
	{
		if (const FTraceSessionInfo* SessionInfoPtr = CurrentSessionInfosMap.Find(CurrentTraceSessionSelected))
		{
			return *SessionInfoPtr;
		}
		else
		{
			UE_LOGF(LogSessionBrowser, Error, "Selected session [%ls] is no longer available", *CurrentTraceSessionSelected.ToString())
		}
	}
	else
	{
		UE_LOGF(LogSessionBrowser, Error, "Selected session [%ls] is invalid", *CurrentTraceSessionSelected.ToString())
	}

	return FTraceSessionInfo();
}

void SModalSessionBrowser::ModalTick(float InDeltaTime)
{
	AccumulatedTimeBetweenTicks += InDeltaTime;

	// Update the sessions at least once per desired interval
	constexpr float PollingInterval = 1.0f;
	if (AccumulatedTimeBetweenTicks > PollingInterval)
	{
		UpdateCurrentSessionInfoMap();
		AccumulatedTimeBetweenTicks = 0.0f;
	}
}

bool SModalSessionBrowser::CanOpenSession() const
{
	return !CurrentTraceSessionSelected.IsNone();
}

void SModalSessionBrowser::UpdateCurrentSessionInfoMap()
{
	using namespace UE::Trace;
	const TUniquePtr<FStoreClient> StoreClient = FStoreClient::Create(*CurrentTraceStoreAddress);

	if (!StoreClient)
	{
		UE_LOGF(LogSessionBrowser, Error, "Failed to connect to trace store at [%ls]", *CurrentTraceStoreAddress)
		return;
	}

	const uint32 SessionCount = StoreClient->GetSessionCount();
	CurrentSessionInfosMap.Empty(SessionCount);

	for (uint32 SessionIndex = 0; SessionIndex < SessionCount; SessionIndex++)
	{
		if (const FStoreClient::FSessionInfo* SessionInfo = StoreClient->GetSessionInfo(SessionIndex))
		{
			//TODO: We need a way to know if it is an IPV6
			FCVDIPv4Address IP(SessionInfo->GetIpAddress());
			
			FStringFormatOrderedArguments Args {IP.ToString(), FString::FromInt(SessionInfo->GetId()) };
			FString SessionDisplayNameString = FString::Format(TEXT("IP: {0} + {1}"), Args);

			FTraceSessionInfo CVDSessionInfo { SessionInfo->GetTraceId(), SessionInfo->GetIpAddress(), SessionInfo->GetControlPort(), true };

			CurrentSessionInfosMap.Add(FName(SessionDisplayNameString), CVDSessionInfo);
		}	
	}

	TArray<TSharedPtr<FName>> NewNameList;
	NewNameList.Reserve(CurrentSessionInfosMap.Num());

	Algo::Transform(CurrentSessionInfosMap, NewNameList, [](const TPair<FName, FTraceSessionInfo>& TransformData){ return MakeShared<FName>(TransformData.Key);});
	
	NamePickerWidget->UpdateNameList(MoveTemp(NewNameList));
}

FReply SModalSessionBrowser::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;
	RequestDestroyWindow();
	return FReply::Handled();
}

void SModalSessionBrowser::HandleSessionNameSelected(TSharedPtr<FName> SelectedName)
{
	CurrentTraceSessionSelected = SelectedName.IsValid() ? *SelectedName : FName();
}

void SModalSessionBrowser::OnTraceStoreAddressUpdated(const FText& InText, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter || CommitType == ETextCommit::Default)
	{
		CurrentTraceStoreAddress = InText.ToString();
	}

	UpdateCurrentSessionInfoMap();
}

#undef LOCTEXT_NAMESPACE
