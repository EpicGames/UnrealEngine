// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SWindow.h"

class FName;
class FText;
class FLiveSessionTracker;
class SModalSessionNameListPicker;
class SEditableTextBox;

namespace EAppReturnType { enum Type : int; }

/** Structure containing info about a Trace Session */

/**
 * Modal window used to find, show and select active Trace Sessions to be used
 * with Chaos Visual Debugger
 */
class SModalSessionBrowser : public SWindow
{
public:
	struct FTraceSessionInfo
	{
		uint32 TraceID = 0;
		uint32 IPAddress = 0;
		uint32 ControlPort = 0;
		bool bIsValid = false;
	};

	SLATE_BEGIN_ARGS(SModalSessionBrowser)
		{
		}
	SLATE_END_ARGS()

	virtual ~SModalSessionBrowser() override;

	void Construct(const FArguments& InArgs);

	/** Displays the dialog in a blocking fashion */
	EAppReturnType::Type ShowModal();

	/** Response that triggered the closing of this modal */
	EAppReturnType::Type GetResponse() const { return UserResponse; }

	/** Returns information about the active trace session selected */
	FTraceSessionInfo GetSelectedTraceInfo();

	/** Returns the address of the trace store selected while looking for active Trace Sessions */
	FString GetSelectedTraceStoreAddress() const { return CurrentTraceStoreAddress; };

	void ModalTick(float InDeltaTime);

protected:

	bool CanOpenSession() const;
	
	void UpdateCurrentSessionInfoMap();

	FReply OnButtonClick(EAppReturnType::Type ButtonID);
	
	void HandleSessionNameSelected(TSharedPtr<FName> SelectedName);
	
	void OnTraceStoreAddressUpdated(const FText& InText, ETextCommit::Type CommitType);

	TSharedPtr<SModalSessionNameListPicker> NamePickerWidget;
	
	TSharedPtr<SEditableTextBox> TraceStoreAddressWidget;
	
	TMap<FName, FTraceSessionInfo> CurrentSessionInfosMap;

	EAppReturnType::Type UserResponse = EAppReturnType::Cancel;

	FString CurrentTraceStoreAddress;

	FName CurrentTraceSessionSelected;

	FDelegateHandle ModalTickHandle;
	float AccumulatedTimeBetweenTicks = 0.0f;
};
