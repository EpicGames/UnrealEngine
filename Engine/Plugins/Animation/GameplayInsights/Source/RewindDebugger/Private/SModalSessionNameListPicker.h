// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "SListViewSelectorDropdownMenu.h"
#include "Widgets/SCompoundWidget.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

class SComboButton;
class STextBlock;

DECLARE_DELEGATE_OneParam(FModalSessionNameListSelected, TSharedPtr<FName>)

/** Widget that takes a list of String Views and creates a drop down menu for it for the Chaos Visual Debugger Tool.
 */
class SModalSessionNameListPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SModalSessionNameListPicker){}
	SLATE_EVENT(FModalSessionNameListSelected, OnNameSleceted)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);

	/**
	 * Replaces the names in the list view
	 * @param NewNameList Array of Name to represent.
	 */
	void UpdateNameList(TArray<TSharedPtr<FName>>&& NewNameList);
	
	void SelectName(const TSharedPtr<FName>& NameToSelect, ESelectInfo::Type SelectionInfo = ESelectInfo::Direct);
	
protected:

	bool HasElements() const;

	FText GetCurrentDropDownButtonLabel() const;
	
	void OnClickItem(TSharedPtr<FName> Item);

	void OnListOpened();

	TSharedRef<ITableRow> MakeNameItemForList(TSharedPtr<FName> Name, const TSharedRef<STableViewBase>& OwnerTable);

	TSharedPtr<SComboButton> PickerComboButton;

	TSharedPtr<STextBlock> DropDownButtonLabelTextBox;
	
	TSharedPtr<SListView<TSharedPtr<FName>>> NameListWidget;

	TSharedPtr<SListViewSelectorDropdownMenu<TSharedPtr<FName>>> NamesListDropdown;

	TArray<TSharedPtr<FName>> CachedNameList;

	TSharedPtr<FName> CurrentSelectedName;

	FModalSessionNameListSelected NameSelectedDelegate;
};
