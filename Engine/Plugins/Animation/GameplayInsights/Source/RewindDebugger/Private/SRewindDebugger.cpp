// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRewindDebugger.h"
#include "ActorPickerMode.h"
#include "Common/ProviderLock.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IGameplayProvider.h"
#include "Insights/IUnrealInsightsModule.h"
#include "IRewindDebuggerTrackCreator.h"
#include "Kismet2/DebuggerCommands.h"
#include "LocalizationDescriptor.h"
#include "Modules/ModuleManager.h"
#include "ObjectTrace.h"
#include "RewindDebuggerCommands.h"
#include "RewindDebuggerEngineEditorBridge.h"
#include "RewindDebuggerModule.h"
#include "RewindDebuggerRuntime/RewindDebuggerRuntime.h"
#include "RewindDebuggerSettings.h"
#include "RewindDebuggerStyle.h"
#include "Selection.h"
#include "SSimpleTimeSlider.h"
#include "StatusBarSubsystem.h"
#include "StructUtils/InstancedStruct.h"
#include "Styling/SlateIconFinder.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Widgets/SConnectionStatusBar.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "SRewindDebugger"

SRewindDebugger::SRewindDebugger()
	: ViewRange(0, 10)
	, Commands(FRewindDebuggerCommands::Get())
	, Settings(URewindDebuggerSettings::Get())
{
}

SRewindDebugger::~SRewindDebugger()
{
#if WITH_TRACE_BASED_DEBUGGERS
	// Sessions discovery no longer needed after closing the main debugger window
	using namespace UE::RewindDebugger;
	if (TSharedPtr<UE::TraceBasedDebuggers::FRemoteSessionsManager> RemoteSessionManager = FRewindDebuggerEngineEditorBridge::Get().GetSessionsManager())
	{
		RemoteSessionManager->StopSessionDiscovery(DebuggerGuid);
	}
#endif // WITH_TRACE_BASED_DEBUGGERS
}

void SRewindDebugger::TrackCursor(const bool bReverse)
{
	const float ScrubTime = ScrubTimeAttribute.Get();
	TRange<double> CurrentViewRange = ViewRange;
	const float ViewSize = CurrentViewRange.GetUpperBoundValue() - CurrentViewRange.GetLowerBoundValue();

	static constexpr double LeadingEdgeSize = 0.05;
	static constexpr double TrailingEdgeThreshold = 0.01;

	if (bReverse)
	{
		// playing in reverse (cursor moving left)
		if (ScrubTime < CurrentViewRange.GetLowerBoundValue() + ViewSize * LeadingEdgeSize)
		{
			CurrentViewRange.SetLowerBound(ScrubTime - ViewSize * LeadingEdgeSize);
			CurrentViewRange.SetUpperBound(CurrentViewRange.GetLowerBoundValue() + ViewSize);
		}
		if (ScrubTime > CurrentViewRange.GetUpperBoundValue() + ViewSize * TrailingEdgeThreshold)
		{
			CurrentViewRange.SetUpperBound(ScrubTime);
			CurrentViewRange.SetLowerBound(CurrentViewRange.GetUpperBoundValue() - ViewSize);
		}
	}
	else
	{
		// playing normally or recording (cursor moving right)
		if (ScrubTime > CurrentViewRange.GetUpperBoundValue() - ViewSize * LeadingEdgeSize)
		{
			CurrentViewRange.SetUpperBound(ScrubTime + ViewSize * LeadingEdgeSize);
			CurrentViewRange.SetLowerBound(CurrentViewRange.GetUpperBoundValue() - ViewSize);
		}
		if (ScrubTime < CurrentViewRange.GetLowerBoundValue() - ViewSize * TrailingEdgeThreshold)
		{
			CurrentViewRange.SetLowerBound(ScrubTime);
			CurrentViewRange.SetUpperBound(CurrentViewRange.GetLowerBoundValue() + ViewSize);
		}
	}

	SetViewRange(CurrentViewRange);
}

void SRewindDebugger::CenterView(const double Time)
{
	const double ViewSize = ViewRange.GetUpperBoundValue() - ViewRange.GetLowerBoundValue();
	const double HalfViewSize = ViewSize * 0.5;
	SetViewRange(TRange<double>(Time - HalfViewSize, Time + HalfViewSize));
}

void SRewindDebugger::SetViewRange(TRange<double> NewRange)
{
	ViewRange = NewRange;
	OnViewRangeChanged.ExecuteIfBound(NewRange);
}

void SRewindDebugger::ToggleHideTrackType(const FName& TrackType)
{
	const int32 Index = Settings.HiddenTrackTypes.Find(TrackType);

	if (Index >= 0)
	{
		Settings.HiddenTrackTypes.RemoveAtSwap(Index);
	}
	else
	{
		Settings.HiddenTrackTypes.Add(TrackType);
	}
	RefreshTracks();
}

bool SRewindDebugger::ShouldHideTrackType(const FName& TrackType) const
{
	return Settings.HiddenTrackTypes.Contains(TrackType);
}

void SRewindDebugger::ToggleDisplayEmptyTracks()
{
	Settings.bShowEmptyObjectTracks = !Settings.bShowEmptyObjectTracks;
	RefreshTracks();
}

bool SRewindDebugger::ShouldDisplayEmptyTracks() const
{
	return Settings.bShowEmptyObjectTracks;
}

void SRewindDebugger::SortObjectsByName()
{
	Settings.ObjectTrackSortMode = ERewindDebuggerObjectSortMode::Name;
	ForceRefreshTracks.ExecuteIfBound();
}

bool SRewindDebugger::ShouldSortObjectsByName() const
{
	return Settings.ObjectTrackSortMode == ERewindDebuggerObjectSortMode::Name;
}

void SRewindDebugger::SortObjectsByTime()
{
	Settings.ObjectTrackSortMode = ERewindDebuggerObjectSortMode::Time;
	ForceRefreshTracks.ExecuteIfBound();
}

bool SRewindDebugger::ShouldSortObjectsByTime() const
{
	return Settings.ObjectTrackSortMode == ERewindDebuggerObjectSortMode::Time;
}

FReply SRewindDebugger::OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	const FInputChord KeyEventAsInputChord = FInputChord(InKeyEvent.GetKey(), EModifierKey::FromBools(InKeyEvent.IsControlDown(), InKeyEvent.IsAltDown(), InKeyEvent.IsShiftDown(), InKeyEvent.IsCommandDown()));
	FReply Reply = FReply::Unhandled();

	// Handle Rewind Debugger VCR Commands
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		Reply = FReply::Handled();
	}

	// Prevent bubbling up shortcut for "ReversePlay"
	if (Commands.ReversePlay->HasDefaultChord(KeyEventAsInputChord) && !IsPIESimulating.Get())
	{
		Reply = FReply::Handled();
	}

	return Reply;
}

void SRewindDebugger::SetDebuggedObject(const UObject* Object)
{
	// Spawned actors have a "RewindDebugger: " prefix on their label
	const AActor* Actor = Cast<AActor>(Object);
	if (Actor != nullptr && Actor->GetActorLabel().StartsWith("RewindDebugger: "))
	{
		FString ActorLabel = Actor->GetActorLabel();
		ActorLabel.RemoveFromStart("RewindDebugger: ");
		DebuggedObjectName.Set(ActorLabel);
	}
	else
	{
		DebuggedObjectName.Set(Object->GetName());
	}
}

TSharedRef<SWidget> SRewindDebugger::MakeObjectSelectionMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddSearchWidget();

	// Menu entry for each actor selected in the scene
	MenuBuilder.BeginSection("From Selection Section", LOCTEXT("FromSelection", "From Scene Selection"));

	TArray<AActor*> SelectedActors;
	GEditor->GetSelectedActors()->GetSelectedObjects(SelectedActors);
	if (SelectedActors.Num() > 0)
	{
		for (AActor* SelectedActor : SelectedActors)
		{
			FText SelectedLabel = FText::FromString(SelectedActor->GetActorLabel());
			FSlateIcon ActorIcon = FSlateIconFinder::FindIconForClass(SelectedActors[0]->GetClass());

			MenuBuilder.AddMenuEntry(SelectedLabel, FText(), ActorIcon, FExecuteAction::CreateLambda([this, SelectedActor] {
				FSlateApplication::Get().DismissAllMenus();
				SetDebuggedObject(SelectedActor);
				}));
		}
	}
	else
	{
		MenuBuilder.AddMenuEntry(LOCTEXT("No scene selection", "No scene selection"),
			LOCTEXT("SceneSelectionToolTip", "If you select an object in the scene, then it will be listed here"),
			FSlateIcon(),
			FUIAction(FExecuteAction(),
				FCanExecuteAction::CreateLambda([]() {return false; })));
	}
	MenuBuilder.EndSection();

	// Menu entry for each object with recorded data in the scene
	MenuBuilder.BeginSection("From Recording Section", LOCTEXT("FromRecording", "From Recording:"));

	if (const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance())
	{
		if (const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession())
		{
			const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider");
			if (GameplayProvider)
			{
				// Gather allowed type names from settings outside the provider lock
				TArray<FString> AllowedTypeNames;
	
				const URewindDebuggerProjectSettings* ProjectSettings = GetDefault<URewindDebuggerProjectSettings>();
				Algo::Transform(ProjectSettings->SelectorAllowedTypes, AllowedTypeNames, [](const FSoftClassPath& Path) { return Path.ToString(); });
				Algo::Transform(ProjectSettings->SelectorAllowedStructTypes, AllowedTypeNames, [](const FInstancedStruct& Type) { return GetPathNameSafe(Type.GetScriptStruct()); });
	
				const URewindDebuggerSettings* UserSettings = GetDefault<URewindDebuggerSettings>();
				Algo::Transform(UserSettings->SelectorAllowedTypes, AllowedTypeNames, [](const FSoftClassPath& Path) { return Path.ToString(); });
				Algo::Transform(UserSettings->SelectorAllowedStructTypes, AllowedTypeNames, [](const FInstancedStruct& Type) { return GetPathNameSafe(Type.GetScriptStruct()); });
	
				// Cache object data from provider under a tight lock scope
				struct FCachedObjectEntry
				{
					FString Name;
					FSlateIcon Icon;
				};
				TArray<FCachedObjectEntry> CachedObjects;
	
				{
					TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
	
					TArray<const uint64> AllowedTypeIds;
					AllowedTypeIds.Reserve(AllowedTypeNames.Num());
					for (const FString& AllowedTypeName : AllowedTypeNames)
					{
						if (const FClassInfo* ClassInfo = GameplayProvider->FindClassInfo(*AllowedTypeName))
						{
							AllowedTypeIds.AddUnique(ClassInfo->Id);
						}
					}
	
					auto IsClassOrChildOfAllowed = [&AllowedTypeIds, GameplayProvider](const uint64 ClassId)
						{
							if (AllowedTypeIds.Num() > 0)
							{
								for (const uint64 AllowedTypeId : AllowedTypeIds)
								{
									if (GameplayProvider->IsSubClassOf(ClassId, AllowedTypeId))
									{
										return true;
									}
								}
								return false;
							}
	
							// Allow everything if allowed class list is not used
							return true;
						};
	
					GameplayProvider->EnumerateObjects([&CachedObjects, GameplayProvider, &IsClassOrChildOfAllowed](const FObjectInfo& ObjectInfo)
						{
							if (!IsClassOrChildOfAllowed(ObjectInfo.ClassId))
							{
								return;
							}
	
							CachedObjects.Add({ ObjectInfo.Name, GameplayProvider->FindIconForClass(ObjectInfo.ClassId) });
						});
				}
	
				// Build menu entries outside the lock
				for (const FCachedObjectEntry& Entry : CachedObjects)
				{
					const FText SelectedLabel = FText::FromString(Entry.Name);
	
					MenuBuilder.AddMenuEntry(SelectedLabel, FText(), Entry.Icon, FExecuteAction::CreateLambda([this, Name = Entry.Name]()
						{
							FSlateApplication::Get().DismissAllMenus();
							DebuggedObjectName.Set(Name);
						}));
				}
			}
		}
		else
		{
			MenuBuilder.AddMenuEntry(LOCTEXT("No recording loaded", "No recording loaded"),
				LOCTEXT("NoRecordingToolTip", "Start or load a recording, and the recorded actors will be listed here"),
				FSlateIcon(),
				FUIAction(FExecuteAction(),
					FCanExecuteAction::CreateLambda([]() {return false; })));
		}

	}

	return MenuBuilder.MakeWidget();
}

void SRewindDebugger::Construct(const FArguments& InArgs, TSharedRef<FUICommandList> InCommandList, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	ForceRefreshTracks = InArgs._ForceRefreshTracks;
	OnScrubPositionChanged = InArgs._OnScrubPositionChanged;
	OnViewRangeChanged = InArgs._OnViewRangeChanged;
	OnTrackSelectionChanged = InArgs._OnTrackSelectionChanged;
	BuildTrackContextMenu = InArgs._BuildTrackContextMenu;
	TrackTypesAttribute = InArgs._TrackTypes;
	ScrubTimeAttribute = InArgs._ScrubTime;
	Tracks = InArgs._Tracks;
	TraceTime.Initialize(InArgs._TraceTime);
	RecordingDuration.Initialize(InArgs._RecordingDuration);
	DebuggedObjectName.Initialize(InArgs._DebuggedObjectName);
	IsPIESimulating = InArgs._IsPIESimulating;
	CommandList = InCommandList;

	TrackFilterBox = SNew(SSearchBox).HintText(LOCTEXT("Filter Tracks", "Filter Tracks")).OnTextChanged_Lambda([this](const FText&)
		{
			RefreshTracks();
		});

	const TSharedPtr<SScrollBar> ScrollBar = SNew(SScrollBar);
	const FToolMenuContext ToolMenuContext(CommandList);

	TrackTreeView = SNew(SRewindDebuggerTrackTree)
		.ExternalScrollBar(ScrollBar)
		.OnExpansionChanged_Lambda([this]()
			{
				if (!bInExpansionChanged)
				{
					bInExpansionChanged = true;
					TimelinesView->RestoreExpansion();
					bInExpansionChanged = false;
				}
			})
		.OnScrolled_Lambda([this](double ScrollOffset) { TimelinesView->ScrollTo(ScrollOffset); })
		.Tracks(InArgs._Tracks)
		.OnMouseButtonDoubleClick(InArgs._OnTrackDoubleClicked)
		.OnContextMenuOpening(this, &SRewindDebugger::OnContextMenuOpening)
		.OnSelectionChanged_Lambda(
			[this](TSharedPtr<RewindDebugger::FRewindDebuggerTrack> SelectedItem, ESelectInfo::Type SelectInfo)
			{
				if (!bInSelectionChanged)
				{
					bInSelectionChanged = true;
					TimelinesView->SetSelection(SelectedItem);
					OnSelectedTrackChanged(SelectedItem, SelectInfo);
					bInSelectionChanged = false;
				}
			});

	TimelinesView = SNew(SRewindDebuggerTimelines)
		.ExternalScrollbar(ScrollBar)
		.OnExpansionChanged_Lambda(
			[this]()
			{
				if (!bInExpansionChanged)
				{
					bInExpansionChanged = true;
					TrackTreeView->RestoreExpansion();
					bInExpansionChanged = false;
				}
			})
		.OnScrolled_Lambda([this](double ScrollOffset) { TrackTreeView->ScrollTo(ScrollOffset); })
		.Tracks(InArgs._Tracks)
		.ViewRange_Lambda([this]() {return ViewRange; })
		.ClampRange_Lambda(
			[this]()
			{
				return TRange<double>(0.0f, RecordingDuration.Get());
			})
		.OnViewRangeChanged(this, &SRewindDebugger::SetViewRange)
		.ScrubPosition(ScrubTimeAttribute)
		.OnScrubPositionChanged_Lambda(
			[this](double NewScrubTime, bool bIsScrubbing)
			{
				OnScrubPositionChanged.ExecuteIfBound(NewScrubTime, bIsScrubbing);
			})
		.OnSelectionChanged_Lambda(
			[this](TSharedPtr<RewindDebugger::FRewindDebuggerTrack> SelectedItem, ESelectInfo::Type SelectInfo)
			{
				if (!bInSelectionChanged)
				{
					bInSelectionChanged = true;
					TrackTreeView->SetSelection(SelectedItem);
					OnSelectedTrackChanged(SelectedItem, SelectInfo);
					bInSelectionChanged = false;
				}
			});

	TSharedPtr<SWidget> StatusBarWidget = SNullWidget::NullWidget;

#if WITH_TRACE_BASED_DEBUGGERS
	using namespace UE::RewindDebugger;
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = FRewindDebuggerEngineEditorBridge::Get().GetSessionsManager())
	{
		// Start remote sessions discovery
		RemoteSessionManager->StartSessionDiscovery(DebuggerGuid);

		// Add status bar for connection statistics updates
		if (UStatusBarSubsystem* StatusBarSubsystem = GEditor ? GEditor->GetEditorSubsystem<UStatusBarSubsystem>() : nullptr)
		{
			using namespace UE::Editor::Toolbars;
			constexpr ECreateStatusBarOptions StatusBarOptions =
				ECreateStatusBarOptions::HideContentBrowser
				| ECreateStatusBarOptions::HideOutputLog
				| ECreateStatusBarOptions::HideSourceControl;

			StatusBarWidget = StatusBarSubsystem->MakeStatusBarWidget(FRewindDebuggerModule::MainStatusBarName, ConstructUnderMajorTab, StatusBarOptions);

			const FName StatusBarToolbarID = FName(FRewindDebuggerModule::MainStatusBarName.ToString() + TEXT(".Toolbar"));
			if (UToolMenu* StatusToolBarMenu = UToolMenus::Get()->ExtendMenu(StatusBarToolbarID))
			{
				FToolMenuSection& ConnectionStatusSection = StatusToolBarMenu->AddSection("ConnectionStatus");

				constexpr bool bNoIndent = true;
				constexpr bool bSearchable = false;
				ConnectionStatusSection.AddEntry(FToolMenuEntry::InitWidget("ConnectionStatusBar"
					, SNew(SConnectionStatusBar, UE::RewindDebugger::DebuggerGuid, RemoteSessionManager.ToSharedRef())
					, FText::GetEmpty()
					, bNoIndent
					, bSearchable));
			}
		}
	}
#endif // WITH_TRACE_BASED_DEBUGGERS

	const TSharedRef<SWidget> ToolBar = UToolMenus::Get()->GenerateWidget(FRewindDebuggerModule::MainToolBarName, ToolMenuContext);
	const TSharedRef<SWidget> RightToolBar = UToolMenus::Get()->GenerateWidget(FRewindDebuggerModule::RightToolBarName, ToolMenuContext);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot().AutoHeight()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot().MaxHeight(48)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					ToolBar
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.FillContentWidth(1.f)
				[
					RightToolBar
				]
			]
		]
				+ SVerticalBox::Slot().FillHeight(1.0)
				[
					SNew(SSplitter)
						+ SSplitter::Slot().MinSize(280).Value(0)
						[
							SNew(SVerticalBox)
								+ SVerticalBox::Slot().AutoHeight()
								[
									SNew(SHorizontalBox)
										+ SHorizontalBox::Slot().FillWidth(1.0f)
										[
											// SNew(SRewindDebuggerTargetSelector)
											SNew(SComboButton)
												.OnGetMenuContent(this, &SRewindDebugger::MakeObjectSelectionMenu)
												.ButtonContent()
												[
													SNew(SHorizontalBox)
														+ SHorizontalBox::Slot().AutoWidth().Padding(3)
														[
															SNew(SImage)
																.Image_Lambda([this]
																	{
																		if (Tracks != nullptr && Tracks->Num() > 0)
																		{
																			return (*Tracks)[0]->GetIcon().GetIcon();
																		}

																		return FSlateIconFinder::FindIconForClass(AActor::StaticClass()).GetIcon();
																	}
																)
														]
														+ SHorizontalBox::Slot().Padding(3)
														[
															SNew(STextBlock)
																.Text_Lambda([this]()
																	{
																		if (Tracks == nullptr || Tracks->Num() == 0)
																		{
																			return LOCTEXT("Select Actor", "Debug Target Actor");
																		}

																		FText ReadableName = (*Tracks)[0]->GetDisplayName();
#if OBJECT_TRACE_ENABLED
																		const RewindDebugger::FObjectId ObjectId = (*Tracks)[0]->GetAssociatedObjectId();

																		if (UObject* Object = FObjectTrace::GetObjectFromId(ObjectId.GetMainId()))
																		{
																			if (const AActor* Actor = Cast<AActor>(Object))
																			{
																				ReadableName = FText::FromString(Actor->GetActorLabel());
																			}
																		}
#endif // OBJECT_TRACE_ENABLED
																		return ReadableName;
																	})
														]
												]
										]
										+ SHorizontalBox::Slot().AutoWidth().HAlign(HAlign_Right)
										[
											SNew(SButton)
												.ButtonStyle(FAppStyle::Get(), "SimpleButton")
												.OnClicked(this, &SRewindDebugger::OnSelectActorClicked)
												.ToolTipText(LOCTEXT("SelectActorTooltip", "Select Target Actor in Scene"))
												[
													SNew(SImage)
														.Image(FRewindDebuggerStyle::Get().GetBrush("RewindDebugger.SelectActor"))
												]
										]
										+ SHorizontalBox::Slot().AutoWidth()
										[
											SNew(SComboButton)
												.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
												.OnGetMenuContent(this, &SRewindDebugger::MakeFilterMenu)
												.ButtonContent()
												[
													SNew(SImage)
														.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
												]
										]
								]
								+ SVerticalBox::Slot().FillHeight(1.0f)
								[
									TrackTreeView.ToSharedRef()
								]
						]
						+ SSplitter::Slot()
						[
							SNew(SVerticalBox)
								+ SVerticalBox::Slot().AutoHeight()
								[
									SNew(SSimpleTimeSlider)
										.DesiredSize({ 100,24 })
										.ClampRangeHighlightSize(0.15f)
										.ClampRangeHighlightColor(FLinearColor::Red.CopyWithNewOpacity(0.5f))
										.ScrubPosition(ScrubTimeAttribute)
										.ViewRange_Lambda([this]() { return ViewRange; })
										.OnViewRangeChanged(this, &SRewindDebugger::SetViewRange)
										.ClampRange_Lambda(
											[this]()
											{
												return TRange<double>(0.0f, RecordingDuration.Get());
											})
										.OnScrubPositionChanged_Lambda(
											[this](double NewScrubTime, bool bIsScrubbing)
											{
												if (bIsScrubbing)
												{
													OnScrubPositionChanged.ExecuteIfBound(NewScrubTime, bIsScrubbing);
												}
											})
								]
								+ SVerticalBox::Slot().FillHeight(1.0f)
								[
									SNew(SOverlay)
										+ SOverlay::Slot()
										[
											TimelinesView.ToSharedRef()
										]
										+ SOverlay::Slot().HAlign(HAlign_Right)
										[
											ScrollBar.ToSharedRef()
										]
								]
						]
				]
				+ SVerticalBox::Slot()
				.Padding(0.0f, 2.0f, 0.0f, 0.0f)
				.AutoHeight()
				[
					StatusBarWidget.ToSharedRef()
				]
		];
}

bool FilterTrack(const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& Track, const FString& FilterString, bool bRemoveNoData, const TArray<FName>& FilteredTrackTypes, bool bParentFilterPassed = false)
{
	using namespace RewindDebugger;
	if (FilteredTrackTypes.Contains(Track->GetName()))
	{
		Track->SetHiddenFlag(ETrackHiddenFlags::HiddenByUI);
		return false;
	}

	// No need to test for UI filtering if track is hidden for any other reason
	if (Track->HasAnyFlags(ETrackHiddenFlags::AnyFlag & ~ETrackHiddenFlags::HiddenByUI))
	{
		return false;
	}

	const bool bStringFilterEmpty = FilterString.IsEmpty();
	const bool bStringFilterPassed = bParentFilterPassed || bStringFilterEmpty || Track->GetDisplayName().ToString().Contains(FilterString);

	const bool bThisFilterPassed = (!bStringFilterEmpty && bStringFilterPassed);

	bool bAnyChildVisible = false;
	Track->IterateSubTracks([&bAnyChildVisible, bThisFilterPassed, FilterString, bRemoveNoData, FilteredTrackTypes](const TSharedPtr<FRewindDebuggerTrack>& ChildTrack)
		{
			const bool bChildIsVisible = FilterTrack(ChildTrack, FilterString, bRemoveNoData, FilteredTrackTypes, bThisFilterPassed);
			bAnyChildVisible |= bChildIsVisible;
		});

	const bool bVisible = bAnyChildVisible || ((!bRemoveNoData || Track->HasDebugData()) && bStringFilterPassed);
	if (bVisible)
	{
		Track->UnsetHiddenFlag(ETrackHiddenFlags::HiddenByUI);
	}
	else
	{
		Track->SetHiddenFlag(ETrackHiddenFlags::HiddenByUI);
	}

	return Track->IsVisible();
}

void SRewindDebugger::RefreshTracks()
{
	if (Tracks)
	{
		for (TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& DebugComponent : *Tracks)
		{
			FilterTrack(DebugComponent, TrackFilterBox->GetText().ToString(), !ShouldDisplayEmptyTracks(), Settings.HiddenTrackTypes);
		}
	}

	TrackTreeView->Refresh();
	TimelinesView->Refresh();
}


TSharedRef<SWidget> SRewindDebugger::MakeFilterMenu()
{
	FMenuBuilder Builder(true, nullptr);
	Builder.AddWidget(TrackFilterBox.ToSharedRef(), FText(), true, false);

	Builder.BeginSection("TrackTypes", LOCTEXT("Track Types", "Track Types"));

	const TArrayView<RewindDebugger::FRewindDebuggerTrackType> TrackTypes = TrackTypesAttribute.Get();

	for (const RewindDebugger::FRewindDebuggerTrackType& TrackType : TrackTypes)
	{
		Builder.AddMenuEntry(
			TrackType.DisplayName,
			FText::Format(LOCTEXT("FilterTrackToolTip", "Show tracks of type {0}"), { TrackType.DisplayName }),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this, TrackType]() { ToggleHideTrackType(TrackType.Name); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, TrackType]() { return !ShouldHideTrackType(TrackType.Name); })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	Builder.AddSeparator();

	Builder.AddMenuEntry(
		LOCTEXT("DisplayEmptyTracks", "Show Empty Object Tracks"),
		LOCTEXT("DisplayEmptyTracksToolTip", "Show Object/Component tracks which have no sub tracks with any debug data"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &SRewindDebugger::ToggleDisplayEmptyTracks),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw(this, &SRewindDebugger::ShouldDisplayEmptyTracks)),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	Builder.AddSeparator();
	
	Builder.AddMenuEntry(
		LOCTEXT("Sort Objects By Name", "Sort Objects By Name"),
		LOCTEXT("Sort Objects By Name Tooltip", "Object Tracks will be sorted alphabetically"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &SRewindDebugger::SortObjectsByName),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw(this, &SRewindDebugger::ShouldSortObjectsByName)),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
		Builder.AddMenuEntry(
    		LOCTEXT("Sort Objects By Id", "Sort Objects By Time"),
    		LOCTEXT("Sort Objects By Id Tooltip", "Object Tracks will be sorted by Id, making older objects display first"),
    		FSlateIcon(),
    		FUIAction(
    			FExecuteAction::CreateRaw(this, &SRewindDebugger::SortObjectsByTime),
    			FCanExecuteAction(),
    			FIsActionChecked::CreateRaw(this, &SRewindDebugger::ShouldSortObjectsByTime)),
    		NAME_None,
    		EUserInterfaceActionType::ToggleButton
    	);

	return Builder.MakeWidget();
}

void SRewindDebugger::OnSelectedTrackChanged(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> SelectedItem, ESelectInfo::Type SelectInfo)
{
	if (SelectedComponent)
	{
		SelectedComponent->SetIsSelected(false);
	}

	SelectedComponent = SelectedItem;

	if (SelectedComponent)
	{
		SelectedComponent->SetIsSelected(true);
	}

	OnTrackSelectionChanged.ExecuteIfBound(SelectedItem);
}

TSharedPtr<SWidget> SRewindDebugger::OnContextMenuOpening()
{
	return BuildTrackContextMenu.Execute();
}

FReply SRewindDebugger::OnSelectActorClicked()
{
	const FActorPickerModeModule& ActorPickerMode = FModuleManager::Get().GetModuleChecked<FActorPickerModeModule>("ActorPickerMode");

	const bool bShouldForceEject = GEditor->PlayWorld && !GEditor->bIsSimulatingInEditor;
	if (bShouldForceEject)
	{
		// Eject PIE
		GEditor->RequestToggleBetweenPIEandSIE();
	}

	ActorPickerMode.BeginActorPickingMode(
		FOnGetAllowedClasses(),
		FOnShouldFilterActor(),
		FOnActorSelected::CreateLambda([this, bShouldForceEject](const AActor* InActor)
			{
				SetDebuggedObject(InActor);
				if (bShouldForceEject && GEditor->bIsSimulatingInEditor)
				{
					// If we force ejected PIE, revert this after actor selection.
					GEditor->RequestToggleBetweenPIEandSIE();
				}
			}));

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
