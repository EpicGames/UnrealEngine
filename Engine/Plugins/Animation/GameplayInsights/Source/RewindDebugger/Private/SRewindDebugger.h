// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BindableProperty.h"
#include "Framework/Commands/UICommandList.h"
#include "IRewindDebuggerTrackCreator.h"
#include "RewindDebuggerCommands.h"
#include "RewindDebuggerSettings.h"
#include "SRewindDebuggerTimelines.h"
#include "SRewindDebuggerTrackTree.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/SCompoundWidget.h"

class FRewindDebuggerModule;
class SCheckBox;
class SDockTab;
class SSearchBox;

class SRewindDebugger : public SCompoundWidget
{
	typedef TBindablePropertyInitializer<FString, BindingType_Out> DebugTargetInitializer;

public:
	DECLARE_DELEGATE_TwoParams(FOnScrubPositionChanged, double, bool)
	DECLARE_DELEGATE_OneParam(FOnViewRangeChanged, const TRange<double>&)
	DECLARE_DELEGATE_OneParam(FOnDebugTargetChanged, TSharedPtr<FString>)
	DECLARE_DELEGATE_OneParam(FOnTrackDoubleClicked, TSharedPtr<RewindDebugger::FRewindDebuggerTrack>)
	DECLARE_DELEGATE_OneParam(FOnTrackSelectionChanged, TSharedPtr<RewindDebugger::FRewindDebuggerTrack>)
	DECLARE_DELEGATE_RetVal(TSharedPtr<SWidget>, FBuildTrackContextMenu)
	DECLARE_DELEGATE(FForceRefreshTracks)

	SLATE_BEGIN_ARGS(SRewindDebugger) {}
		SLATE_ARGUMENT(TArray< TSharedPtr< RewindDebugger::FRewindDebuggerTrack > >*, Tracks);
		SLATE_ARGUMENT(DebugTargetInitializer, DebuggedObjectName);
		SLATE_ARGUMENT(TBindablePropertyInitializer<double>, TraceTime);
		SLATE_ARGUMENT(TBindablePropertyInitializer<double>, RecordingDuration);
		SLATE_ATTRIBUTE(TArrayView<RewindDebugger::FRewindDebuggerTrackType>, TrackTypes);
		SLATE_ATTRIBUTE(double, ScrubTime);
		SLATE_ATTRIBUTE(bool, IsPIESimulating);
		SLATE_EVENT(FOnScrubPositionChanged, OnScrubPositionChanged);
		SLATE_EVENT(FOnViewRangeChanged, OnViewRangeChanged);
		SLATE_EVENT(FBuildTrackContextMenu, BuildTrackContextMenu);
		SLATE_EVENT(FOnTrackDoubleClicked, OnTrackDoubleClicked);
		SLATE_EVENT(FOnTrackSelectionChanged, OnTrackSelectionChanged);
		SLATE_EVENT(FForceRefreshTracks, ForceRefreshTracks);
	SLATE_END_ARGS()

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param CommandList The command list to use.
	 * @param ConstructUnderMajorTab The major tab which will contain the session front-end.
	 * @param ConstructUnderWindow The window in which this widget is being constructed.
	 */
	void Construct(const FArguments& InArgs, TSharedRef<FUICommandList> CommandList, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);

	SRewindDebugger();
	virtual ~SRewindDebugger();
	void TrackCursor(bool bReverse);
	void CenterView(double Time);
	void RefreshTracks();

private:
	void SetViewRange(TRange<double> NewRange);

	void ToggleHideTrackType(const FName& TrackType);
	bool ShouldHideTrackType(const FName& TrackType) const;
	void ToggleDisplayEmptyTracks();
	bool ShouldDisplayEmptyTracks() const;
	
	void SortObjectsByName();
	bool ShouldSortObjectsByName() const;
	void SortObjectsByTime();
	bool ShouldSortObjectsByTime() const;

	virtual FReply OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return true; }

	/** Time Slider */
	TAttribute<double> ScrubTimeAttribute;
	TAttribute<bool> IsPIESimulating;
	TAttribute<bool> TrackScrubbingAttribute;
	TAttribute<TArrayView<RewindDebugger::FRewindDebuggerTrackType>> TrackTypesAttribute;
	FOnScrubPositionChanged OnScrubPositionChanged;
	FOnViewRangeChanged OnViewRangeChanged;
	TRange<double> ViewRange;
	TBindableProperty<double> TraceTime;
	TBindableProperty<double> RecordingDuration;

	FForceRefreshTracks ForceRefreshTracks;
	TSharedPtr<FUICommandList> CommandList;
	const FRewindDebuggerCommands& Commands;

	/** debug object selector */
	TSharedRef<SWidget> MakeObjectSelectionMenu();
	void SetDebuggedObject(const UObject* Object);

	FReply OnSelectActorClicked();

	TBindableProperty<FString, BindingType_Out> DebuggedObjectName;

	TSharedRef<SWidget> MakeFilterMenu();


	/** Track tree view */
	TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>* Tracks = nullptr;
	TSharedPtr<RewindDebugger::FRewindDebuggerTrack> SelectedComponent;
	void OnSelectedTrackChanged(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> SelectedItem, ESelectInfo::Type SelectInfo);
	FBuildTrackContextMenu BuildTrackContextMenu;
	FOnTrackSelectionChanged OnTrackSelectionChanged;

	TSharedPtr<SRewindDebuggerTrackTree> TrackTreeView;
	TSharedPtr<SRewindDebuggerTimelines> TimelinesView;
	TSharedPtr<SWidget> OnContextMenuOpening();

	TSharedPtr<SSearchBox> TrackFilterBox;

	bool bInExpansionChanged = false;
	bool bInSelectionChanged = false;
	bool bDisplayEmptyTracks = false;

	URewindDebuggerSettings& Settings;
};
