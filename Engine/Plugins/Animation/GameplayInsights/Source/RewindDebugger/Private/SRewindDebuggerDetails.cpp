// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRewindDebuggerDetails.h"

#define LOCTEXT_NAMESPACE "SRewindDebuggerDetails"

void SRewindDebuggerDetails::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
}

#undef LOCTEXT_NAMESPACE
