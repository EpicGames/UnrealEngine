// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BindableProperty.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SDockTab;

class SRewindDebuggerDetails : public SCompoundWidget
{
	typedef TBindablePropertyInitializer<FString, BindingType_Out> FDebugTargetInitializer;

public:
	SLATE_BEGIN_ARGS(SRewindDebuggerDetails) { }
	SLATE_END_ARGS()

	/**
	 * Constructs the application.
	 *
	 * @param InArgs The Slate argument list.
	 * @param ConstructUnderMajorTab The major tab which will contain the session front-end.
	 * @param ConstructUnderWindow The window in which this widget is being constructed.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);
};
