// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRewindDebuggerTimelines.h"
#include "ObjectTrace.h"
#include "RewindDebugger.h"
#include "RewindDebuggerStyle.h"
#include "SSimpleTimeSlider.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "SAnimationInsights"


class SRewindDebuggerTimelineTableRow : public STableRow<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>
{
public:

	SLATE_BEGIN_ARGS(SRewindDebuggerTimelineTableRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		STableRow::Construct(STableRow::FArguments(), InOwnerTableView);
		Style = &FRewindDebuggerStyle::Get().GetWidgetStyle<FTableRowStyle>("RewindDebugger.TableRow");

		SetExpanderArrowVisibility(EVisibility::Collapsed);

		SetHover(TAttribute<bool>::CreateLambda([this]()
			{
				if (auto TableView = OwnerTablePtr.Pin())
				{
					if (const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>* Track = GetItemForThis(TableView.ToSharedRef()))
					{
						return (*Track)->GetIsHovered();
					}
				}
				return false;
			}));
	}

	virtual int32 GetIndentLevel() const override
	{
		// don't indent timeline tracks
		return 0;
	}

	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (auto TableView = OwnerTablePtr.Pin())
		{
			if (const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>* Track = GetItemForThis(TableView.ToSharedRef()))
			{
				(*Track)->SetIsTrackHovered(true);
			}
		}
		STableRow::OnMouseEnter(MyGeometry, MouseEvent);
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		STableRow::OnMouseLeave(MouseEvent);

		if (auto TableView = OwnerTablePtr.Pin())
		{
			if (const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>* Track = GetItemForThis(TableView.ToSharedRef()))
			{
				(*Track)->SetIsTrackHovered(false);
			}
		}
	}
};

void SRewindDebuggerTimelines::SetSelection(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> SelectedItem) const
{
	TreeView->SetSelection(SelectedItem);
}

void SRewindDebuggerTimelines::ScrollTo(double ScrollOffset) const
{
	TreeView->SetScrollOffset(ScrollOffset);
}


TSharedRef<ITableRow> SRewindDebuggerTimelines::TreeViewGenerateRow(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedPtr<SRewindDebuggerTimelineTableRow> Track = SNew(SRewindDebuggerTimelineTableRow, OwnerTable);
	TSharedPtr<SWidget> TimelineView = InItem->GetTimelineView();

	if (!TimelineView.IsValid())
	{
		TimelineView = SNew(SSpacer).Size(FVector2D(100, 20));
	}

	Track->SetContent(TimelineView.ToSharedRef());

	return Track.ToSharedRef();
}

void TimelineViewGetChildren(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InItem, TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>& OutChildren)
{
	InItem->IterateSubTracks([&OutChildren](TSharedPtr<RewindDebugger::FRewindDebuggerTrack> Track)
		{
			if (Track->IsVisible())
			{
				OutChildren.Add(Track);
			}
		});
}

void SRewindDebuggerTimelines::TimelineViewExpansionChanged(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InItem, bool bShouldBeExpanded) const
{
	InItem->SetIsExpanded(bShouldBeExpanded);
	OnExpansionChanged.ExecuteIfBound();
}

void SRewindDebuggerTimelines::Construct(const FArguments& InArgs)
{
	SSimpleTimeSlider::Construct(SSimpleTimeSlider::FArguments()
		.ViewRange(InArgs._ViewRange)
		.OnViewRangeChanged(InArgs._OnViewRangeChanged)
		.ClampRange(InArgs._ClampRange)
		.ScrubPosition(InArgs._ScrubPosition)
		.OnScrubPositionChanged(InArgs._OnScrubPositionChanged));

	Tracks = InArgs._Tracks;

	OnExpansionChanged = InArgs._OnExpansionChanged;

	TreeView = SNew(STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>)
		.OnSelectionChanged(InArgs._OnSelectionChanged)
		.TreeItemsSource(Tracks)
		.OnGenerateRow(this, &SRewindDebuggerTimelines::TreeViewGenerateRow)
		.OnGetChildren_Static(&TimelineViewGetChildren)
		.OnExpansionChanged(this, &SRewindDebuggerTimelines::TimelineViewExpansionChanged)
		.SelectionMode(ESelectionMode::Single)
		.AllowOverscroll(EAllowOverscroll::No)
		.ExternalScrollbar(InArgs._ExternalScrollbar)
		.OnTreeViewScrolled(InArgs._OnScrolled);

	ChildSlot
		[
			TreeView.ToSharedRef()
		];
}

int32 SRewindDebuggerTimelines::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	TRange<double> LocalViewRange = ViewRange.Get();
	FScrubRangeToScreen RangeToScreen(LocalViewRange, AllottedGeometry.GetLocalSize());

	// Paint Child Widgets (Tracks) 
	LayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, ShouldBeEnabled(bParentEnabled));

	// Paint Major Time Marks
	FDrawTickArgs TickArgs;
	TickArgs.AllottedGeometry = AllottedGeometry;
	TickArgs.bOnlyDrawMajorTicks = true;
	TickArgs.TickColor = FLinearColor(0.2f, 0.2f, 0.2f, 0.2f);
	TickArgs.ClippingRect = MyCullingRect;
	TickArgs.DrawEffects = ESlateDrawEffect::None;
	TickArgs.StartLayer = ++LayerId;
	TickArgs.TickOffset = 0;
	TickArgs.MajorTickHeight = AllottedGeometry.Size.Y;
	DrawTicks(OutDrawElements, RangeToScreen, TickArgs);

	// Paint Cursor
	const float XPos = RangeToScreen.InputToLocalX(ScrubPosition.Get());

	TArray<FVector2D> Points{ {XPos, 0}, { XPos, AllottedGeometry.Size.Y} };

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		++LayerId,
		AllottedGeometry.ToPaintGeometry(),
		Points,
		ESlateDrawEffect::None,
		FLinearColor::White,
		false
	);

	return LayerId;
}

static void RestoreTimelineExpansion(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> Track, TSharedPtr<STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>>& TreeView)
{
	TreeView->SetItemExpansion(Track, Track->GetIsExpanded());
	if (Track->IsVisible())
	{
		Track->IterateSubTracks([&TreeView](TSharedPtr<RewindDebugger::FRewindDebuggerTrack> SubTrack)
			{
				RestoreTimelineExpansion(SubTrack, TreeView);
			});
	}
}

void SRewindDebuggerTimelines::RestoreExpansion()
{
	for (auto& Track : *Tracks)
	{
		::RestoreTimelineExpansion(Track, TreeView);
	}
}

void SRewindDebuggerTimelines::Refresh()
{
	TreeView->RebuildList();

	if (Tracks)
	{
		// make sure any newly added TreeView nodes are created expanded
		RestoreExpansion();
	}
}

#undef LOCTEXT_NAMESPACE
