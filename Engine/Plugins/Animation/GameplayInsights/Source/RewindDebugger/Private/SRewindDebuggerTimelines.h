// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RewindDebuggerTrack.h"
#include "SSimpleTimeSlider.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"


class SRewindDebuggerTimelines : public SSimpleTimeSlider
{
	using FOnSelectionChanged = STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>::FOnSelectionChanged;
public:
	SLATE_BEGIN_ARGS(SRewindDebuggerTimelines) {}

		SLATE_ARGUMENT(TArray< TSharedPtr< RewindDebugger::FRewindDebuggerTrack > >*, Tracks);
		SLATE_ARGUMENT(TSharedPtr<SScrollBar>, ExternalScrollbar)
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		SLATE_EVENT(FOnTableViewScrolled, OnScrolled)
		SLATE_EVENT(FSimpleDelegate, OnExpansionChanged)


		/** The scrub position */
		SLATE_ATTRIBUTE(double, ScrubPosition);

		/** View time range */
		SLATE_ATTRIBUTE(TRange<double>, ViewRange);

		/** Clamp time range */
		SLATE_ATTRIBUTE(TRange<double>, ClampRange);

		/** Called when the scrub position changes */
		SLATE_EVENT(FOnScrubPositionChanged, OnScrubPositionChanged);

		/** Called right before the scrubber begins to move */
		SLATE_EVENT(FSimpleDelegate, OnBeginScrubberMovement);

		/** Called right after the scrubber handle is released by the user */
		SLATE_EVENT(FSimpleDelegate, OnEndScrubberMovement);

		/** Called when the view range changes */
		SLATE_EVENT(FOnRangeChanged, OnViewRangeChanged);

	SLATE_END_ARGS()

	/**
	* Constructs the application.
	*
	* @param InArgs The Slate argument list.
	*/
	void Construct(const FArguments& InArgs);

	void Refresh();
	void RestoreExpansion();

	void SetSelection(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> SelectedItem) const;
	void ScrollTo(double ScrollOffset) const;

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

private:
	TSharedRef<ITableRow> TreeViewGenerateRow(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	void TimelineViewExpansionChanged(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InItem, bool bShouldBeExpanded) const;

	TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>* Tracks = nullptr;
	TSharedPtr<STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>> TreeView;

	FSimpleDelegate OnExpansionChanged;
};
