// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRewindDebuggerTrackTree.h"
#include "ObjectTrace.h"
#include "RewindDebugger.h"
#include "RewindDebuggerStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"

#define LOCTEXT_NAMESPACE "SAnimationInsights"

class SRewindDebuggerComponentTreeTableRow final : public STableRow<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>
{
public:

	SLATE_BEGIN_ARGS(SRewindDebuggerComponentTreeTableRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		STableRow<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>::Construct(STableRow<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>::FArguments(), InOwnerTableView);
		Style = &FRewindDebuggerStyle::Get().GetWidgetStyle<FTableRowStyle>("RewindDebugger.TableRow");

		SetHover(TAttribute<bool>::CreateLambda([this]()
			{
				if (TSharedPtr<ITypedTableView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>> TableView = OwnerTablePtr.Pin())
				{
					if (const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>* Track = GetItemForThis(TableView.ToSharedRef()))
					{
						return (*Track)->GetIsHovered();
					}
				}

				return false;
			}));
	}

	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (TSharedPtr<ITypedTableView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>> TableView = OwnerTablePtr.Pin())
		{
			if (const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>* Track = GetItemForThis(TableView.ToSharedRef()))
			{
				(*Track)->SetIsTreeHovered(true);
			}
		}
		STableRow<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>::OnMouseEnter(MyGeometry, MouseEvent);
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		STableRow<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>::OnMouseLeave(MouseEvent);

		if (TSharedPtr<ITypedTableView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>> TableView = OwnerTablePtr.Pin())
		{
			if (const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>* Track = GetItemForThis(TableView.ToSharedRef()))
			{
				(*Track)->SetIsTreeHovered(false);
			}
		}
	}
};


TSharedRef<ITableRow> SRewindDebuggerTrackTree::TreeViewGenerateRow(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InItem, const TSharedRef<STableViewBase>& OwnerTable) const
{
	const FSlateIcon ObjectIcon = InItem->GetIcon();

	const TSharedRef<SLayeredImage> LayeredIcons = SNew(SLayeredImage)
		.DesiredSizeOverride(FVector2D(16, 16))
		.Image(ObjectIcon.GetIcon());

	if (ObjectIcon.GetOverlayIcon())
	{
		LayeredIcons->AddLayer(ObjectIcon.GetOverlayIcon());
	}

	TSharedRef<SRewindDebuggerComponentTreeTableRow> Row = SNew(SRewindDebuggerComponentTreeTableRow, OwnerTable);


	Row->SetContent(
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot().AutoWidth().Padding(2)
		[
			LayeredIcons
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(1.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
				.Text(InItem->GetDisplayName())
				.Font_Lambda([this, InItem]() { return InItem->GetIsHovered() ? FCoreStyle::GetDefaultFontStyle("Bold", 10) : FCoreStyle::GetDefaultFontStyle("Regular", 10); })
		]
	);

	return Row;
}

void TreeViewGetChildren(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InItem, TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>& OutChildren)
{
	InItem->IterateSubTracks([&OutChildren](const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& Track)
		{
			if (Track->IsVisible())
			{
				OutChildren.Add(Track);
			}
		});
}

void SRewindDebuggerTrackTree::TreeViewExpansionChanged(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InItem, const bool bShouldBeExpanded) const
{
	InItem->SetIsExpanded(bShouldBeExpanded);
	OnExpansionChanged.ExecuteIfBound();
}

void SRewindDebuggerTrackTree::SetSelection(const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& SelectedItem) const
{
	TreeView->SetSelection(SelectedItem);
}

void SRewindDebuggerTrackTree::ScrollTo(const double ScrollOffset) const
{
	TreeView->SetScrollOffset(ScrollOffset);
}

void SRewindDebuggerTrackTree::Construct(const FArguments& InArgs)
{
	Tracks = InArgs._Tracks;

	TreeView = SNew(STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>)
		.TreeItemsSource(Tracks)
		.OnGenerateRow(this, &SRewindDebuggerTrackTree::TreeViewGenerateRow)
		.OnGetChildren_Static(&TreeViewGetChildren)
		.OnExpansionChanged(this, &SRewindDebuggerTrackTree::TreeViewExpansionChanged)
		.SelectionMode(ESelectionMode::Single)
		.OnSelectionChanged(InArgs._OnSelectionChanged)
		.OnMouseButtonDoubleClick(InArgs._OnMouseButtonDoubleClick)
		.ExternalScrollbar(InArgs._ExternalScrollBar)
		.AllowOverscroll(EAllowOverscroll::No)
		.OnTreeViewScrolled(InArgs._OnScrolled)
		.ScrollbarDragFocusCause(EFocusCause::SetDirectly)
		.OnContextMenuOpening(InArgs._OnContextMenuOpening);

	ChildSlot
		[
			TreeView.ToSharedRef()
		];

	OnExpansionChanged = InArgs._OnExpansionChanged;
}

static void RestoreExpansion(const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& Track, TSharedPtr<STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>>& TreeView)
{
	TreeView->SetItemExpansion(Track, Track->GetIsExpanded());
	Track->IterateSubTracks([&TreeView](const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& SubTrack)
		{
			RestoreExpansion(SubTrack, TreeView);
		});
}

void SRewindDebuggerTrackTree::RestoreExpansion()
{
	for (TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& Track : *Tracks)
	{
		::RestoreExpansion(Track, TreeView);
	}
}

void SRewindDebuggerTrackTree::Refresh()
{
	TreeView->RebuildList();

	if (Tracks)
	{
		// make sure any newly added TreeView nodes are created expanded
		RestoreExpansion();
	}
}

#undef LOCTEXT_NAMESPACE
