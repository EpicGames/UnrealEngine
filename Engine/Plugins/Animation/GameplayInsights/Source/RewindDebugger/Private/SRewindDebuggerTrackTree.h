// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IRewindDebugger.h"
#include "RewindDebuggerTrack.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class SRewindDebuggerTrackTree : public SCompoundWidget
{
	using FOnSelectionChanged = STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>::FOnSelectionChanged;
	using FOnMouseButtonDoubleClick = STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>::FOnMouseButtonDoubleClick;

public:
	SLATE_BEGIN_ARGS(SRewindDebuggerTrackTree) {}
		SLATE_ARGUMENT(TArray< TSharedPtr< RewindDebugger::FRewindDebuggerTrack > >*, Tracks);
		SLATE_ARGUMENT(TSharedPtr< SScrollBar >, ExternalScrollBar);
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		SLATE_EVENT(FOnMouseButtonDoubleClick, OnMouseButtonDoubleClick)
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
		SLATE_EVENT(FSimpleDelegate, OnExpansionChanged)
		SLATE_EVENT(FOnTableViewScrolled, OnScrolled)
	SLATE_END_ARGS()

	/**
	 * Constructs the application.
	 *
	 * @param InArgs The Slate argument list.
	 */
	void Construct(const FArguments& InArgs);

	void Refresh();
	void RestoreExpansion();

	void SetSelection(const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& SelectedItem) const;
	void ScrollTo(double ScrollOffset) const;

private:
	TSharedRef<ITableRow> TreeViewGenerateRow(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InItem, const TSharedRef<STableViewBase>& OwnerTable) const;
	void TreeViewExpansionChanged(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InItem, bool bShouldBeExpanded) const;

	TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>* Tracks = nullptr;
	TSharedPtr<STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>> TreeView;

	FSimpleDelegate OnExpansionChanged;
};
