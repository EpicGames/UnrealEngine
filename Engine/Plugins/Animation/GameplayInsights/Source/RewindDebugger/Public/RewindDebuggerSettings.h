// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "Engine/DeveloperSettings.h"
#include "RewindDebuggerSettings.generated.h"

struct FInstancedStruct;
struct FSoftClassPath;
class URewindDebuggerExtensionSettings;

#define UE_API REWINDDEBUGGER_API

namespace UE::TraceBasedDebuggers
{
enum class ETraceTransportMode : uint8;
}

UENUM()
enum class ERewindDebuggerCameraMode : uint8
{
	Replay UMETA(Tooltip="Replay Recorded Camera"),
	FollowTargetActor UMETA(Tooltip="Follow Target Actor"),
	Disabled UMETA(Tooltip="Disable Camera On Playback"),
};

UENUM()
enum class ERewindDebuggerObjectSortMode : uint8
{
	Time UMETA(Tooltip="Sort Object tracks By Id, making older objects display first"),
	Name UMETA(Tooltip="Sort Object Tracks alphabetically by Name"),
};

/**
 * Implements project settings for the Rewind Debugger.
 */
UCLASS(config = RewindDebugger, defaultconfig, EditInlineNew, DisplayName = "Rewind Debugger", collapseCategories, MinimalAPI)
class URewindDebuggerProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	URewindDebuggerProjectSettings();

#if WITH_EDITOR
	virtual FText GetSectionDescription() const override;
#endif

	virtual FName GetCategoryName() const override;

	/**
	 * Class types allowed in the target selection dropdown for the project.
	 * Empty list means that all types are allowed (if SelectorAllowedStructTypes is also empty).
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Target selection")
	TArray<FSoftClassPath> SelectorAllowedTypes;

	/**
	 * Struct types allowed in the target selection dropdown for the project.
	 * Empty list means that all types are allowed (if SelectorAllowedTypes is also empty).
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Target selection")
	TArray<FInstancedStruct> SelectorAllowedStructTypes;
};

/**
 * Implements user settings for the Rewind Debugger.
 */
UCLASS(config = EditorPerProjectUserSettings, meta=(DisplayName="Rewind Debugger (user preferences)"))
class URewindDebuggerSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	URewindDebuggerSettings();
	~URewindDebuggerSettings();

#if WITH_EDITOR
	virtual FText GetSectionText() const override;
	virtual FText GetSectionDescription() const override;
#endif

	virtual FName GetCategoryName() const override;

	/** Returns the configured transport mode for the provided target type
	 * @param InTargetType Build target type to evaluate
	 */
	UE_API UE::TraceBasedDebuggers::ETraceTransportMode GetTransportModeForTargetType(EBuildTargetType InTargetType) const;

	/** Sets a specific transport mode for the provided target type
	 * @param InTargetType Build target type to evaluate
	 * @param InTransportMode New transport mode to use for the provided target
	 */
	UE_API void SetTransportModeForTargetType(EBuildTargetType InTargetType, UE::TraceBasedDebuggers::ETraceTransportMode InTransportMode);

	/** Rewind Debugger Playback Camera Mode */
	UPROPERTY(EditAnywhere, Config, Category = Camera)
	ERewindDebuggerCameraMode CameraMode;

	/** Rewind Debugger Object Track Sort Mode */
	UPROPERTY(EditAnywhere, Config, Category = Other)
	ERewindDebuggerObjectSortMode ObjectTrackSortMode = ERewindDebuggerObjectSortMode::Time;

	/** If enabled, automatically detach player control when PIE is paused */
	UPROPERTY(EditAnywhere, Config, Category = Other)
	bool bShouldAutoEject;

	/** If enabled, start recording information at the start of PIE */
	UPROPERTY(EditAnywhere, Config, Category = Other)
	bool bShouldAutoRecordOnPIE;

	/** Playback speed multiplier */
	UPROPERTY(EditAnywhere, Config, Category = Other)
	float PlaybackRate = 1.0;

	/** If enabled, show empty tracks on Rewind Debugger Timeline*/
	UPROPERTY(EditAnywhere, Config, Category = Filters)
	bool bShowEmptyObjectTracks;

	/** The track types listed here will be hidden from the track tree view */
	UPROPERTY(EditAnywhere, Config, Category = Filters)
	TArray<FName> HiddenTrackTypes;

	/** Currently selected target actor's name */
	UPROPERTY(Config)
	FString DebugTargetActor;

	/**
	 * Additional class types allowed in the target selection dropdown
	 * @note These can only add more allowed types to the types allowed by the project settings, not replace/override them
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Target selection")
	TArray<FSoftClassPath> SelectorAllowedTypes;

	/**
	 * Additional struct types allowed in the target selection dropdown
	 * @note These can only add more allowed types to the types allowed by the project settings, not replace/override them
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Target selection")
	TArray<FInstancedStruct> SelectorAllowedStructTypes;

	/** Get Mutable CDO of URewindDebuggerSettings */
	UE_API static URewindDebuggerSettings& Get();

private:
	/** Default Data Transport for Game Servers - Used to change the transport mode for the trace data. This is not saved in the config by design */
	UPROPERTY(EditAnywhere, Category = Connection)
	UE::TraceBasedDebuggers::ETraceTransportMode GameServerDataTransportMode;

	/** Default Data Transport for Game Clients - Used to change the transport mode for the trace data. This is not saved in the config by design */
	UPROPERTY(EditAnywhere, Category = Connection)
	UE::TraceBasedDebuggers::ETraceTransportMode GameClientDataTransportMode;

	FDelegateHandle OnPreExitHandle;
};

#undef UE_API