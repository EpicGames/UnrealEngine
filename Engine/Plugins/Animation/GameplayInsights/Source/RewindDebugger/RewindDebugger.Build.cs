// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class RewindDebugger : ModuleRules
	{
		public RewindDebugger(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ActorPickerMode",
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"GameplayInsights",
				"InputCore",
				"LevelSequence", // required for hard coded level sequence special case in ObjectTrack
				"RewindDebuggerInterface",
				"RewindDebuggerRuntime",
				"Slate",
				"SlateCore",
				"Sockets",
				"StatusBar",
				"ToolMenus",
				"ToolWidgets",
				"TraceAnalysis",
				"TraceBasedDebuggers",
				"TraceBasedDebuggersAnalysis",
				"TraceInsights",
				"TraceLog",
				"TraceServices",
				"UnrealEd",
			});
		}
	}
}

