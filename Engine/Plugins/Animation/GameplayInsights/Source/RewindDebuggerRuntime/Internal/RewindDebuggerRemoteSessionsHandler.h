// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteSessionsManager.h"
#include "RewindDebuggerRemoteSessionsHandler.generated.h"

#define UE_API REWINDDEBUGGERRUNTIME_API

namespace UE::RewindDebugger
{
USTRUCT()
struct FStartRecordingCommand : public UE::TraceBasedDebuggers::FStartRecordingCommandMessage
{
GENERATED_BODY()
};


USTRUCT()
struct FStopRecordingCommand : public UE::TraceBasedDebuggers::FStopRecordingCommandMessage
{
GENERATED_BODY()
};

#if WITH_TRACE_BASED_DEBUGGERS
/**
 * Remote sessions handler to handle RewindDebugger specific message types.
 */
struct FSessionsHandler : TraceBasedDebuggers::IRemoteSessionsHandler
{
	UE_API virtual void OnCreatingMessageEndpoint(const TSharedPtr<TraceBasedDebuggers::FRemoteSessionsManager>&, FMessageEndpointBuilder&) override;
	UE_API virtual void OnSubscribingMessageTypes(const TSharedPtr<TraceBasedDebuggers::FRemoteSessionsManager>&, FMessageEndpoint&) override;
	UE_API virtual void OnBuildingFullSessionInfoResponseMessage(TraceBasedDebuggers::FFullSessionInfoResponseMessage&) override;
	UE_API virtual void OnHandlingFullSessionInfoResponseMessage(const TraceBasedDebuggers::FFullSessionInfoResponseMessage&, const TSharedPtr<TraceBasedDebuggers::FSessionInfo>&) override;
};
#endif // WITH_TRACE_BASED_DEBUGGERS
} // UE::RewindDebugger

#undef UE_API