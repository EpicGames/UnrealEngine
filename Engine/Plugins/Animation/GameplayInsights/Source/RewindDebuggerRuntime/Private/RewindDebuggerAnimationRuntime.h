// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimTrace.h"
#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"


namespace RewindDebugger
{

	class FRewindDebuggerAnimationRuntime : public IRewindDebuggerRuntimeExtension 
	{
	public:
		virtual void RecordingStarted() override
		{
			UE::Trace::ToggleChannel(TEXT("Animation"), true);
		}
		
		virtual void RecordingStopped() override
		{
			UE::Trace::ToggleChannel(TEXT("Animation"), false);
		}

		virtual void Clear() override
		{
#if ANIM_TRACE_ENABLED
			FAnimTrace::Reset();
#endif
		}
	};
}
