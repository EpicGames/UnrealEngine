// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerEngineEditorBridge.h"

#if WITH_TRACE_BASED_DEBUGGERS

#include "RewindDebuggerRemoteSessionsHandler.h"
#include "TraceBasedDebuggerRuntime.h"
#include "TraceDataRelayTransport.h"
#include "Engine/Engine.h"
#include "Misc/App.h"
#include "UObject/Package.h"

namespace UE::RewindDebugger
{
using namespace ::RewindDebugger;

bool FRewindDebuggerEngineEditorBridge::bIsInstantiated = false;

FRewindDebuggerEngineEditorBridge::FRewindDebuggerEngineEditorBridge(): FEngineEditorBridge(LogRewindDebuggerRuntime)
{
	RegisterRemoteSessionsHandler(MakeShared<FSessionsHandler>());
}

FRewindDebuggerEngineEditorBridge& FRewindDebuggerEngineEditorBridge::Get()
{
	static FRewindDebuggerEngineEditorBridge EngineEditorBridge;

	// Keep track of the instantiation of the bridge.
	// Doing so allows external modules responsible for calling Teardown
	// to avoid calling Get (and instantiate the bridge) if the bridge was not instantiated.
	bIsInstantiated = true;
	return EngineEditorBridge;
}

bool FRewindDebuggerEngineEditorBridge::IsInstantiated()
{
	return bIsInstantiated;
}

void FRewindDebuggerEngineEditorBridge::OnInitializeInternal()
{
	using namespace UE::TraceBasedDebuggers;

#if WITH_EDITOR
	PIEStartedHandle = FWorldDelegates::OnPIEStarted.AddRaw(this, &FRewindDebuggerEngineEditorBridge::HandlePIEStarted);
#endif

	if (FRewindDebuggerRuntime* Runtime = FRewindDebuggerRuntime::Instance())
	{
		RecordingStartedHandle = Runtime->RecordingStarted.AddRaw(this, &FRewindDebuggerEngineEditorBridge::HandleRecordingStarted);
		RecordingStartFailedHandle = Runtime->RecordingStartFailed.AddRaw(this, &FRewindDebuggerEngineEditorBridge::HandleRecordingStartFailed);
		RecordingStoppedHandle = Runtime->RecordingStopped.AddRaw(this, &FRewindDebuggerEngineEditorBridge::HandleRecordingStopped);

		Runtime->OnTraceConnectionDetailsUpdated().AddRaw(this, &FRewindDebuggerEngineEditorBridge::HandleTraceConnectionDetailsUpdated);

		// If we were already recording, we need to make sure we run the initialization step to set up the session broadcast ticker
		// and the collision channel serialization
		if (Runtime->IsRecording())
		{
			HandleRecordingStarted();
		}
	}
}

void FRewindDebuggerEngineEditorBridge::OnTearDownInternal()
{
#if WITH_EDITOR
	FWorldDelegates::OnPIEStarted.Remove(PIEStartedHandle);
#endif

	FTSTicker::RemoveTicker(DeferredShowMessageOnScreenHandle);

	if (FRewindDebuggerRuntime* Runtime = FRewindDebuggerRuntime::Instance())
	{
		Runtime->RecordingStarted.Remove(RecordingStartedHandle);
		Runtime->RecordingStartFailed.Remove(RecordingStartFailedHandle);
		Runtime->RecordingStopped.Remove(RecordingStoppedHandle);
		Runtime->OnTraceConnectionDetailsUpdated().RemoveAll(this);

		// Make sure of removing the message from the screen in case the recording didn't quite stop yet
		if (Runtime->IsRecording())
		{
			HandleRecordingStopped();
		}
	}
}

bool FRewindDebuggerEngineEditorBridge::AddOnScreenRecordingMessage(float DummyDeltaTime)
{
	constexpr bool bContinueLooping = false;
	if (!GEngine)
	{
		return bContinueLooping;
	}

	if (!IsInGameThread())
	{
		if (!DeferredShowMessageOnScreenHandle.IsValid())
		{
			DeferredShowMessageOnScreenHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FRewindDebuggerEngineEditorBridge::AddOnScreenRecordingMessage));
		}

		return bContinueLooping;
	}

	DeferredShowMessageOnScreenHandle.Reset();

	const FText RecordingStartedMessage = NSLOCTEXT("RewindDebugger", "OnScreenRecordingStartedMessage", "Rewind Debugger recording in progress...");

	if (RecordingMessageKey == 0)
	{
		RecordingMessageKey = GetTypeHash(RecordingStartedMessage.ToString());
	}

	// Add a long duration value, we will remove the message manually when the recording stops
	// @todo: consider using FSlateNotificationManager if properly supported on all platforms
	constexpr float MessageDurationSeconds = 3600.0f;
	GEngine->AddOnScreenDebugMessage(RecordingMessageKey, MessageDurationSeconds, FColor::Red, RecordingStartedMessage.ToString());

	return bContinueLooping;
}

void FRewindDebuggerEngineEditorBridge::RemoveOnScreenRecordingMessage()
{
	ensure(IsInGameThread());

	if (!GEngine)
	{
		return;
	}

	if (DeferredShowMessageOnScreenHandle.IsValid())
	{
		FTSTicker::RemoveTicker(DeferredShowMessageOnScreenHandle);
		DeferredShowMessageOnScreenHandle.Reset();
	}
	else if (RecordingMessageKey != 0)
	{
		GEngine->RemoveOnScreenDebugMessage(RecordingMessageKey);
	}
}

void FRewindDebuggerEngineEditorBridge::OnRecordingStartedInternal()
{
	AddOnScreenRecordingMessage();
}

void FRewindDebuggerEngineEditorBridge::OnRecordingStoppedInternal()
{
	RemoveOnScreenRecordingMessage();
}

void FRewindDebuggerEngineEditorBridge::BuildRecordingStatusInternal(TraceBasedDebuggers::FRecordingStatusMessage& OutStatusMessage) const
{
	if (const FRewindDebuggerRuntime* Runtime = FRewindDebuggerRuntime::Instance())
	{
		OutStatusMessage.DebuggerId = Runtime->GetDebuggerId();
		OutStatusMessage.RequesterId = Runtime->GetRecordingRequesterId();
		OutStatusMessage.ElapsedTime = Runtime->GetAccumulatedRecordingTime();
		OutStatusMessage.BufferedDataBytesSize = Runtime->GetBufferedDataBytesSize();
	}
}

void FRewindDebuggerEngineEditorBridge::HandleTraceConnectionDetailsUpdated() const
{
	TraceBasedDebuggers::FTraceConnectionDetailsMessage ConnectionDetailsMessage;
	ConnectionDetailsMessage.InstanceId = FApp::GetInstanceId();

	if (const FRewindDebuggerRuntime* Runtime = FRewindDebuggerRuntime::Instance())
	{
		ConnectionDetailsMessage.TraceDetails = Runtime->GetCurrentTraceConnectionDetails();
	}

	GetSessionsManager()->PublishMessage(ConnectionDetailsMessage);
}

void FRewindDebuggerEngineEditorBridge::HandleRecordingStartFailed(const FText& InFailureReason) const
{
#if !WITH_EDITOR
	if (GEngine)
	{
		// In non-editor builds we don't have an error pop-up, therefore we want to show the error message on screen
		const FText ErrorMessage = FText::FormatOrdered(NSLOCTEXT("RewindDebugger", "StartRecordingFailedOnScreenMessage", "Failed to start debug trace recording. {0}"), InFailureReason);

		constexpr float MessageDurationSeconds = 4.0f;
		GEngine->AddOnScreenDebugMessage(RecordingMessageKey, MessageDurationSeconds, FColor::Red, ErrorMessage.ToString());
	}
#endif
}

void FRewindDebuggerEngineEditorBridge::HandlePIEStarted(UGameInstance* GameInstance)
{
	if (const FRewindDebuggerRuntime* Runtime = FRewindDebuggerRuntime::Instance())
	{
		if (Runtime->IsRecording())
		{
			AddOnScreenRecordingMessage();
		}
	}
}

} // UE::RewindDebugger

#else

namespace UE::RewindDebugger
{

FRewindDebuggerEngineEditorBridge& FRewindDebuggerEngineEditorBridge::Get()
{
	static FRewindDebuggerEngineEditorBridge EngineEditorBridge;
	return EngineEditorBridge;
}

} // UE::RewindDebugger

#endif // WITH_TRACE_BASED_DEBUGGERS
