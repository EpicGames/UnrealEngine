// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TRACE_BASED_DEBUGGERS

#include "RewindDebuggerRemoteSessionsHandler.h"
#include "Features/IModularFeatures.h"
#include "MessageEndpointBuilder.h"
#include "RewindDebuggerRuntime/RewindDebuggerRuntime.h"
#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"
#include "RewindDebuggerRuntimeModule.h"
#include "SessionInfo.h"

namespace UE::RewindDebugger
{

void FSessionsHandler::OnCreatingMessageEndpoint(
	const TSharedPtr<TraceBasedDebuggers::FRemoteSessionsManager>& InSessionsManager
	, FMessageEndpointBuilder& InEndPointBuilder)
{
	InEndPointBuilder
		.Handling<FStartRecordingCommand>([](const FStartRecordingCommand& InMessage, const TSharedRef<IMessageContext>& InContext)
			{
				UE_AUTORTFM_ONCOMMIT(InMessage, InContext)
				{
					if (::RewindDebugger::FRewindDebuggerRuntime* Runtime = ::RewindDebugger::FRewindDebuggerRuntime::Instance())
					{
						Runtime->StartRecording(InMessage);
					}
				};
			})
		.Handling<FStopRecordingCommand>([](const FStopRecordingCommand&, const TSharedRef<IMessageContext>&)
			{
				UE_AUTORTFM_ONCOMMIT()
				{
					if (::RewindDebugger::FRewindDebuggerRuntime* Runtime = ::RewindDebugger::FRewindDebuggerRuntime::Instance())
					{
						Runtime->StopRecording();
					}
				};
			});

	// Allow extensions to register their message handlers
	::RewindDebugger::IterateExtensions([&InSessionsManager, &InEndPointBuilder](IRewindDebuggerRuntimeExtension* Extension)
		{
			Extension->RegisterMessageHandlers(InSessionsManager, InEndPointBuilder);
		}
	);

}

void FSessionsHandler::OnSubscribingMessageTypes(
	const TSharedPtr<TraceBasedDebuggers::FRemoteSessionsManager>& InSessionsManager
	, FMessageEndpoint& InMessageEndpoint)
{
	if (InSessionsManager)
	{
		InSessionsManager->RegisterExternalSupportedMessageType<FStartRecordingCommand>();
		InSessionsManager->RegisterExternalSupportedMessageType<FStopRecordingCommand>();
	}

	InMessageEndpoint.Subscribe<FStartRecordingCommand>();
	InMessageEndpoint.Subscribe<FStopRecordingCommand>();

	// Allow extensions to register their message types
	::RewindDebugger::IterateExtensions([&InSessionsManager, &InMessageEndpoint](IRewindDebuggerRuntimeExtension* Extension)
		{
			Extension->RegisterMessageTypes(InSessionsManager, InMessageEndpoint);
		}
	);

}

void FSessionsHandler::OnBuildingFullSessionInfoResponseMessage(
	TraceBasedDebuggers::FFullSessionInfoResponseMessage& InMessage)
{
	if (const ::RewindDebugger::FRewindDebuggerRuntime* Runtime = ::RewindDebugger::FRewindDebuggerRuntime::Instance())
	{
		InMessage.RecordingRequesterId = Runtime->GetRecordingRequesterId();
		InMessage.DebuggerId = DebuggerGuid;
	}

	// Session information currently doesn't require any RewindDebugger specific data, but if it's eventually
	// needed then this is where it needs to be added to the message.
	// e.g.,
	//    FRewindDebuggerSessionInfoResponseData MessageData;
	//    InMessage.DebuggerSpecificData = FInstancedStruct::Make(MoveTemp(MessageData));
}

void FSessionsHandler::OnHandlingFullSessionInfoResponseMessage(
	const TraceBasedDebuggers::FFullSessionInfoResponseMessage& InMessage
	, const TSharedPtr<TraceBasedDebuggers::FSessionInfo>& InSessionInfo)
{
	// Session information currently doesn't require any RewindDebugger specific data, but if it's eventually
	// needed then this is where it needs to be extracted from the message and stored in the session info.
	// e.g.,
	//    if (const FRewindDebuggerSessionInfoResponseData* MessageData = InMessage.GetDebuggerData<FRewindDebuggerSessionInfoResponseData>())
	//    {
	//        FRewindDebuggerSessionData NewSessionData;
	//        ... setup data ...
	//        InSessionInfo->SetDebuggerData<FRewindDebuggerSessionData>(MoveTemp(NewSessionData));
	//    }
}

} // UE::RewindDebugger

#endif // WITH_TRACE_BASED_DEBUGGERS