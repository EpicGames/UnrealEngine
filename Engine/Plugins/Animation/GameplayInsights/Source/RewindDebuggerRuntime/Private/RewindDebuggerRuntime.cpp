// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TRACE_BASED_DEBUGGERS

#include "RewindDebuggerRuntime/RewindDebuggerRuntime.h"

#include "EngineUtils.h"
#include "Features/IModularFeatures.h"
#include "GameFramework/Pawn.h"
#include "ObjectTrace.h"
#include "RemoteSessionsManager.h"
#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"
#include "UObject/UObjectIterator.h"

DEFINE_LOG_CATEGORY(LogRewindDebuggerRuntime)

namespace RewindDebugger
{

	FRewindDebuggerRuntime* FRewindDebuggerRuntime::InternalInstance = nullptr;

	FRewindDebuggerRuntime::FRewindDebuggerRuntime()
		: FRuntimeModule(LogRewindDebuggerRuntime, UE::RewindDebugger::DebuggerGuid)
	{
		DebuggerName = TEXT("RewindDebugger");
		AutoRecordCommand = TEXT("StartRewindDebuggerRecording");
		HostAddressCommand = TEXT("HostAddress");
	}

	void FRewindDebuggerRuntime::Initialize()
	{
		InternalInstance = new FRewindDebuggerRuntime();
	}
	
	void FRewindDebuggerRuntime::Shutdown()
	{
		delete InternalInstance;
		InternalInstance = nullptr;
	}

	void FRewindDebuggerRuntime::StartRecording()
	{
		using namespace UE::TraceBasedDebuggers;
		FRuntimeModule::StartRecording(FStartRecordingCommandMessage
			{
				.InstanceId = FApp::GetInstanceId(),
				.Target = TEXT("127.0.0.1"),
				.RecordingMode = ERecordingMode::Live,
				.TransportMode = ETraceTransportMode::TraceServer
			});
	}

	void FRewindDebuggerRuntime::OnEnableRequiredTraceChannelsInternal()
	{
		UE::Trace::ToggleChannel(TEXT("Object"), true);
		UE::Trace::ToggleChannel(TEXT("ObjectProperties"), true);
		UE::Trace::ToggleChannel(TEXT("Frame"), true);
	}

	void FRewindDebuggerRuntime::OnRecordingStartingInternal()
	{
#if OBJECT_TRACE_ENABLED
		FObjectTrace::Reset();
#endif

		ClearRecording.Broadcast();

		// update extensions
		IterateExtensions([](IRewindDebuggerRuntimeExtension* Extension)
			{
				Extension->Clear();
			}
		);

		// Reset world elapsed time at the beginning of the "start recording" action before channels 
		// gets enabled and events traced (i.e., HandleTraceConnectionEstablished)
#if OBJECT_TRACE_ENABLED
		for (TObjectIterator<UWorld> World; World; ++World)
		{
			FObjectTrace::ResetWorldElapsedTime(*World);
		}
#endif // OBJECT_TRACE_ENABLED
	}

	void FRewindDebuggerRuntime::OnRecordingStartedInternal()
	{
		// TObjectIterator requires execution on the game thread
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [this]
			{
				// trace each play-in-editor world, and all the actors in it.
				for (TObjectIterator<UWorld> World; World; ++World)
				{
					TRACE_WORLD(*World);

					for (TActorIterator<AController> Iterator(*World); Iterator; ++Iterator)
					{
						if (APawn* Pawn = Iterator->GetPawn())
						{
							TRACE_PAWN_POSSESS(static_cast<UObject*>(*Iterator), static_cast<UObject*>(Pawn));
						}
					}
				}
			});

		// update extensions so they can enable their channels now that the trace connection is established
		IterateExtensions([](IRewindDebuggerRuntimeExtension* Extension)
			{
				Extension->RecordingStarted();
			});

		RecordingStarted.Broadcast();
	}

	void FRewindDebuggerRuntime::OnRecordingStartFailedInternal(const FText& FailureReason)
	{
		RecordingStartFailed.Broadcast(FailureReason);
	}

	void FRewindDebuggerRuntime::OnRecordingStoppedInternal()
	{
		// update extensions
		IterateExtensions([](IRewindDebuggerRuntimeExtension* Extension)
			{
				Extension->RecordingStopped();
			}
		);

		RecordingStopped.Broadcast();
	}
}

#endif // WITH_TRACE_BASED_DEBUGGERS