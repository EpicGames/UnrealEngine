// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerRuntimeModule.h"
#include "Modules/ModuleManager.h"

#if WITH_TRACE_BASED_DEBUGGERS
#include "EngineEditorBridge.h"
#include "Features/IModularFeatures.h"
#include "HAL/IConsoleManager.h"
#include "ObjectTrace.h"
#include "RewindDebuggerAnimationRuntime.h"
#include "RewindDebuggerEngineEditorBridge.h"
#include "RewindDebuggerRuntime/RewindDebuggerRuntime.h"

class FRewindDebuggerRuntimeModule : public IModuleInterface
{
public:
	//~ IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
private:
	TArray<IConsoleObject*> ConsoleObjects;

	RewindDebugger::FRewindDebuggerAnimationRuntime AnimationExtension;
};

void FRewindDebuggerRuntimeModule::StartupModule()
{
	if (RewindDebugger::FRewindDebuggerRuntime::Instance() == nullptr)
	{
		RewindDebugger::FRewindDebuggerRuntime::Initialize();
	}

	ConsoleObjects.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("RewindDebugger.StartRecording"),
		TEXT("Starts making a rewind debugger recording."),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				UE_AUTORTFM_ONCOMMIT(Args)
				{
					if (RewindDebugger::FRewindDebuggerRuntime* Runtime = RewindDebugger::FRewindDebuggerRuntime::Instance())
					{
						Runtime->StartRecording(Args);
					}
				};
			}),
		ECVF_Default
	));

	ConsoleObjects.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("RewindDebugger.StopRecording"),
		TEXT("Stops the current rewind debugger recording."),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				UE_AUTORTFM_ONCOMMIT(Args)
				{
					if (RewindDebugger::FRewindDebuggerRuntime* Runtime = RewindDebugger::FRewindDebuggerRuntime::Instance())
					{
						Runtime->StopRecording();
					}
				};
			}),
		ECVF_Default
	));

	IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, &AnimationExtension);

	FCoreDelegates::OnFEngineLoopInitComplete.AddLambda([]()
		{
			UE::RewindDebugger::FRewindDebuggerEngineEditorBridge::Get().Initialize();
		});
}

void FRewindDebuggerRuntimeModule::ShutdownModule()
{
	IConsoleManager& ConsoleManager = IConsoleManager::Get();
	for (IConsoleObject* ConsoleObject : ConsoleObjects)
	{
		if (ConsoleObject)
		{
			ConsoleManager.UnregisterConsoleObject(ConsoleObject);
		}
	}
	ConsoleObjects.Empty();

	RewindDebugger::FRewindDebuggerRuntime::Shutdown();

	IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, &AnimationExtension);

	// Calling Teardown only if the bridge was instantiated since we don't
	// want calling 'Get()' which will create the global instance during the shutdown.
	using namespace UE::RewindDebugger;
	if (FRewindDebuggerEngineEditorBridge::IsInstantiated())
	{
		FRewindDebuggerEngineEditorBridge::Get().TearDown();
	}
}

IMPLEMENT_MODULE(FRewindDebuggerRuntimeModule, RewindDebuggerRuntime);

#else

IMPLEMENT_MODULE(FDefaultModuleImpl, RewindDebuggerRuntime);

#endif // WITH_TRACE_BASED_DEBUGGERS