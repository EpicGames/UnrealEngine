// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "RewindDebuggerAnimationRuntime.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
