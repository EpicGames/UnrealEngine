// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TRACE_BASED_DEBUGGERS

#include "Delegates/IDelegateInstance.h"
#include "EngineEditorBridge.h"
#include "RewindDebuggerRuntime/RewindDebuggerRuntime.h"

#define UE_API REWINDDEBUGGERRUNTIME_API

namespace UE::RewindDebugger
{

/**
 * Object that bridges the RewindDebugger runtime module and the Engine/Editor.
 * This layer is not strictly necessary for RewindDebugger since its runtime currently has access to the engine module, but we use it
 * to share a common stack of layers with other trace-based debuggers.
 */
class FRewindDebuggerEngineEditorBridge : public TraceBasedDebuggers::FEngineEditorBridge
{
public:
	FRewindDebuggerEngineEditorBridge();

	~FRewindDebuggerEngineEditorBridge() = default;

	UE_API static FRewindDebuggerEngineEditorBridge& Get();

	/** @return Whether the static instance has been created (i.e., Get() was called and teardown is required) */
	UE_API static bool IsInstantiated();

private:
	virtual void OnInitializeInternal() override;
	virtual void OnTearDownInternal() override;
	virtual void OnRecordingStartedInternal() override;
	virtual void OnRecordingStoppedInternal() override;
	virtual void BuildRecordingStatusInternal(TraceBasedDebuggers::FRecordingStatusMessage& OutStatusMessage) const override;

	bool AddOnScreenRecordingMessage(float DummyDeltaTime = 0.1f);
	void RemoveOnScreenRecordingMessage();
	void HandleRecordingStartFailed(const FText& InFailureReason) const;
	void HandlePIEStarted(UGameInstance* GameInstance);

	void HandleTraceConnectionDetailsUpdated() const;

	FDelegateHandle RecordingStartedHandle;
	FDelegateHandle RecordingStoppedHandle;
	FDelegateHandle RecordingStartFailedHandle;
	uint64 RecordingMessageKey = 0;

#if WITH_EDITOR
	FDelegateHandle PIEStartedHandle;
#endif

	FTSTicker::FDelegateHandle DeferredShowMessageOnScreenHandle;
	static bool bIsInstantiated;
};

} // UE::RewindDebugger

#else

#define UE_API REWINDDEBUGGERRUNTIME_API
namespace UE::RewindDebugger
{
class FRewindDebuggerEngineEditorBridge
{
public:
	static FRewindDebuggerEngineEditorBridge& Get();
};
} // UE::RewindDebugger
#endif

#undef UE_API
