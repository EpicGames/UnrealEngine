// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#if WITH_TRACE_BASED_DEBUGGERS

#include "TraceBasedDebuggerRuntime.h"

#define UE_API REWINDDEBUGGERRUNTIME_API

REWINDDEBUGGERRUNTIME_API DECLARE_LOG_CATEGORY_EXTERN(LogRewindDebuggerRuntime, Log, All);

namespace UE::RewindDebugger
{
static constexpr FGuid DebuggerGuid = FGuid(0xC4E4CF24, 0x57DB4959, 0x8EDBD741, 0x673D25EC);
} // UE::RewindDebugger


namespace RewindDebugger
{

	class FRewindDebuggerRuntime : public UE::TraceBasedDebuggers::FRuntimeModule
	{
	public:
		UE_API FRewindDebuggerRuntime();

		static UE_API void Initialize();
		static UE_API void Shutdown();
		static FRewindDebuggerRuntime* Instance()
		{
			return InternalInstance;
		}

		UE_API void StartRecording();

		//~ For StartRecording overloads taking an arguments list or a FStartRecordingCommandMessage struct
		using FRuntimeModule::StartRecording;

		FSimpleMulticastDelegate ClearRecording;
		UE::TraceBasedDebuggers::FRecordingStateChangedDelegate RecordingStarted;
		UE::TraceBasedDebuggers::FRecordingStartFailedDelegate RecordingStartFailed;
		UE::TraceBasedDebuggers::FRecordingStateChangedDelegate RecordingStopped;

	private:
		//~ Begin FRuntimeModule interface
		virtual void OnRecordingStartingInternal() override;
		virtual void OnRecordingStartedInternal() override;
		virtual void OnRecordingStartFailedInternal(const FText& FailureReason) override;
		virtual void OnRecordingStoppedInternal() override;
		virtual void OnEnableRequiredTraceChannelsInternal() override;
		//~ End FRuntimeModule interface

		static UE_API FRewindDebuggerRuntime* InternalInstance;
	};
}

#undef UE_API
#endif // WITH_TRACE_BASED_DEBUGGERS
