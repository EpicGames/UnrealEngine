// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerVLog.h"

#include "Common/ProviderLock.h"
#include "FindInBlueprintManager.h"
#include "IRewindDebugger.h"
#include "IVisualLoggerProvider.h"
#include "LogVisualizerSettings.h"
#include "ObjectTrace.h"
#include "RewindDebuggerVLogSettings.h"
#include "SceneView.h"
#include "ToolMenus.h"
#include "VisualLogEntryRenderer.h"
#include "VLogRenderingActor.h"
#include "Debug/DebugDrawService.h"
#include "Editor/EditorEngine.h"
#include "Engine/Canvas.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "TraceServices/Model/Frames.h"
#include "VisualLogger/VisualLoggerTraceDevice.h"
#include "Widgets/LogCategoryVerbositySelector.h"

#define LOCTEXT_NAMESPACE "RewindDebuggerVLog"

namespace UE::RewindDebugger::VLog
{
const FName MenuSectionName("Visual Logger");
const FName CategoriesMenuName("RewindDebugger.CategoriesMenu");

TAutoConsoleVariable<int32> CVarRewindDebuggerVLogUseActor(TEXT("a.RewindDebugger.VisualLogs.UseActor"), 0, TEXT("Use actor based debug renderer for visual logs"));
}

void FRewindDebuggerVLog::OnShowDebugInfo(UCanvas* Canvas, APlayerController* Player)
{
	ScreenTextY = 60;
	if (IRewindDebugger* RewindDebugger = IRewindDebugger::Instance())
	{
		if (RewindDebugger->IsPIESimulating())
		{
			// make sure this is the primary view, when we are playing in PIE, so we don't clear ImmediateRenderQueue when this has been called on some other editor view.
			if (Canvas->SceneView->ViewActor)
			{
				for (FVisualLogEntry& Entry : ImmediateRenderQueue)
				{
					RenderLogEntry(Entry, Canvas);
				}

				ImmediateRenderQueue.SetNum(0);
			}
		}
		else
		{
			ObjectsVisited.Empty();
			if (const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession())
			{
				const double CurrentTraceTime = RewindDebugger->CurrentTraceTime();

				bool bFrameFound;
				TraceServices::FFrame CurrentFrame;
				{
					TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session);
					const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*Session);
					bFrameFound = FrameProvider.GetFrameFromTime(TraceFrameType_Game, CurrentTraceTime, CurrentFrame);
				}

				if (bFrameFound)
				{
					if (const IVisualLoggerProvider* VisualLoggerProvider = Session->ReadProvider<IVisualLoggerProvider>("VisualLoggerProvider"))
					{
						TraceServices::FProviderReadScopeLock ProviderReadScope(*VisualLoggerProvider);
						AddLogEntries(RewindDebugger->GetDebuggedObjects(), CurrentFrame.StartTime, CurrentFrame.EndTime, VisualLoggerProvider, Canvas);
					}
				}
			}
		}
	}
}

FRewindDebuggerVLog::~FRewindDebuggerVLog()
{
	UDebugDrawService::Unregister(DelegateHandle);

	if (UToolMenus* Menus = UToolMenus::Get())
	{
		if (UToolMenu* CategoriesMenu = Menus->FindMenu(UE::RewindDebugger::VLog::CategoriesMenuName))
		{
			CategoriesMenu->RemoveSection(UE::RewindDebugger::VLog::MenuSectionName);
		}
	}

	if (ToolMenuRegistrationHandle.IsValid())
	{
		UToolMenus::UnRegisterStartupCallback(ToolMenuRegistrationHandle);
	}
}

void FRewindDebuggerVLog::Initialize()
{
	ToolMenuRegistrationHandle = UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([this]()
		{
			LogCategoryFilterWidget = SNew(UE::RewindDebugger::SLogCategoryFilter);

			UToolMenu* CategoriesMenu = UToolMenus::Get()->ExtendMenu(UE::RewindDebugger::VLog::CategoriesMenuName);
			TWeakPtr<UE::RewindDebugger::SLogCategoryFilter> WeakFilter = LogCategoryFilterWidget;
			CategoriesMenu->AddDynamicSection("Visual Logger", FNewToolMenuDelegate::CreateLambda([WeakFilter](UToolMenu* InMenu)
				{
					if (const TSharedPtr<UE::RewindDebugger::SLogCategoryFilter> Filter = WeakFilter.Pin())
					{
						Filter->RefreshCategoryList();

						FToolMenuSection& Section = InMenu->AddSection("Visual Logger");
						Section.AddEntry(FToolMenuEntry::InitWidget(
							"VLog Category Filter",
							Filter.ToSharedRef(),
							FText::GetEmpty(),
							true,
							true,
							true));
					}
				})
			);
		})
	);

	FVisualLoggerTraceDevice& TraceDevice = FVisualLoggerTraceDevice::Get();
	TraceDevice.ImmediateRenderDelegate.BindRaw(this, &FRewindDebuggerVLog::ImmediateRender);

	DelegateHandle = UDebugDrawService::Register(TEXT("VirtualTextureResidency")/*TEXT("VisLog")*/, FDebugDrawDelegate::CreateRaw(this, &FRewindDebuggerVLog::OnShowDebugInfo));

	MonospaceFont = TStrongObjectPtr<UFont>(NewObject<UFont>(GetTransientPackage(), NAME_None, RF_Transient));
	MonospaceFont->FontCacheType = EFontCacheType::Runtime;
	MonospaceFont->RuntimeFontSource = ERuntimeFontSource::CoreStyleDefault;
	MonospaceFont->LegacyFontName = FName("Mono");
	MonospaceFont->LegacyFontSize = 9;
}

bool MatchCategoryFilters(const FName& CategoryName, ELogVerbosity::Type Verbosity)
{
	URewindDebuggerVLogSettings& Settings = URewindDebuggerVLogSettings::Get();
	return Settings.DisplayCategories.Contains(CategoryName) && Verbosity <= Settings.GetCategoryVerbosity(CategoryName);
}

void FRewindDebuggerVLog::RenderLogEntry(const FVisualLogEntry& Entry, UCanvas* Canvas)
{
	if (UE::RewindDebugger::VLog::CVarRewindDebuggerVLogUseActor.GetValueOnAnyThread())
	{
		// old actor based code path
		if (AVLogRenderingActor* RenderingActor = GetRenderingActor())
		{
			RenderingActor->AddLogEntry(Entry);
		}
	}
	else if (const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance())
	{
		UWorld* World = RewindDebugger->GetWorldToVisualize();
		FVisualLogEntryRenderer::RenderLogEntry(World, Entry, &MatchCategoryFilters, Canvas, GEngine->GetMediumFont(), MonospaceFont.Get(), ScreenTextY);
	}
}

void FRewindDebuggerVLog::ImmediateRender(const UObject* Object, const FVisualLogEntry& Entry)
{
#if OBJECT_TRACE_ENABLED
	uint64 ObjectId = FObjectTrace::GetObjectId(Object);
	if (DebuggedObjectIds.Contains(ObjectId))
	{
		ImmediateRenderQueue.Add(Entry);
	}
#endif
}

bool FRewindDebuggerVLog::IsCategoryActive(const FName& Category)
{
	URewindDebuggerVLogSettings& Settings = URewindDebuggerVLogSettings::Get();
	return Settings.DisplayCategories.Contains(Category);
}

void FRewindDebuggerVLog::ToggleCategory(const FName& Category)
{
	URewindDebuggerVLogSettings::Get().ToggleCategory(Category);

}

void FRewindDebuggerVLog::AddLogEntries(const TArray<TSharedPtr<FDebugObjectInfo>>& Components, float StartTime, float EndTime, const IVisualLoggerProvider* VisualLoggerProvider, UCanvas* Canvas)
{
	for (const TSharedPtr<FDebugObjectInfo>& ComponentInfo : Components)
	{
		const uint64 ObjectId = ComponentInfo->GetUObjectId();
		if (!ObjectsVisited.Contains(ObjectId))
		{
			ObjectsVisited.Add(ObjectId);
			VisualLoggerProvider->ReadVisualLogEntryTimeline(ObjectId, [this, StartTime, EndTime, Canvas](const IVisualLoggerProvider::VisualLogEntryTimeline& TimelineData)
				{
					TimelineData.EnumerateEvents(StartTime, EndTime, [this, StartTime, EndTime, Canvas](double InStartTime, double InEndTime, uint32 InDepth, const FVisualLogEntry& LogEntry)
						{
							if (InStartTime >= StartTime && InStartTime <= EndTime)
							{
								RenderLogEntry(LogEntry, Canvas);
							}
							return TraceServices::EEventEnumerate::Continue;
						});
				});
		}

		AddLogEntries(ComponentInfo->Children, StartTime, EndTime, VisualLoggerProvider, Canvas);
	}
}

AVLogRenderingActor* FRewindDebuggerVLog::GetRenderingActor()
{
	if (!VLogActor.IsValid())
	{
		UEditorEngine* EditorEngine = Cast<UEditorEngine>(GEngine);
		if (GIsEditor && EditorEngine && EditorEngine->PlayWorld)
		{
			FActorSpawnParameters SpawnParameters;
			SpawnParameters.ObjectFlags |= RF_Transient;
			VLogActor = EditorEngine->PlayWorld->SpawnActor<AVLogRenderingActor>(SpawnParameters);
		}
	}
	return VLogActor.Get();
}

void FRewindDebuggerVLog::Update(float DeltaTime, IRewindDebugger* RewindDebugger)
{
}

void FRewindDebuggerVLog::OnTrackListChanged(IRewindDebugger* RewindDebugger)
{
	const TArray<TSharedPtr<FDebugObjectInfo>>& DebuggedObjects = RewindDebugger->GetDebuggedObjects();
	DebuggedObjectIds.Reset(DebuggedObjects.Num());
	for (const TSharedPtr<FDebugObjectInfo>& Object : DebuggedObjects)
	{
		if (Object->Id.IsSet())
		{
			DebuggedObjectIds.Push(Object->GetUObjectId());
		}
	}
}

#undef LOCTEXT_NAMESPACE
