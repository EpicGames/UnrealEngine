// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IRewindDebuggerExtension.h"
// Needed for TStrongObjectPtr which requires complete type
#include "Engine/Font.h"

struct FDebugObjectInfo;
struct FVisualLogEntry;
class AVLogRenderingActor;
class IRewindDebugger;
class UCanvas;
class UToolMenu;

namespace UE::TraceBasedDebuggers
{
struct FRecordingControls;
}

namespace UE::RewindDebugger
{
struct FLogCategoryVerbosity;
struct FDebuggerRecordingControls;
class SLogCategoryFilter;
}

/** Rewind debugger extension for Visual Logger support */
class FRewindDebuggerVLog : public IRewindDebuggerExtension
{
public:
	void OnShowDebugInfo(UCanvas* Canvas, APlayerController* Player);
	virtual ~FRewindDebuggerVLog();

	virtual FString GetName() override
	{
		return TEXT("FRewindDebuggerVLog");
	}

	void Initialize();
	static void ToggleCategory(const FName& Category);
	static bool IsCategoryActive(const FName& Category);

protected:
	virtual void Update(float DeltaTime, IRewindDebugger* RewindDebugger) override;
	virtual void OnTrackListChanged(IRewindDebugger* RewindDebugger) override;

private:
	void AddLogEntries(const TArray<TSharedPtr<FDebugObjectInfo>>& Components, float StartTime, float EndTime, const class IVisualLoggerProvider* Provider, UCanvas* Canvas);
	void ImmediateRender(const UObject* Object, const FVisualLogEntry& Entry);
	void RenderLogEntry(const FVisualLogEntry& Entry, UCanvas* Canvas);

	AVLogRenderingActor* GetRenderingActor();

	TWeakObjectPtr<AVLogRenderingActor> VLogActor;

	TSet<uint64> ObjectsVisited;
	int32 ScreenTextY = 0;

	FDelegateHandle DelegateHandle;
	FDelegateHandle ToolMenuRegistrationHandle;
	TStrongObjectPtr<UFont> MonospaceFont;
	TSharedPtr<UE::RewindDebugger::SLogCategoryFilter> LogCategoryFilterWidget;

	TArray<uint64> DebuggedObjectIds;
	TArray<FVisualLogEntry> ImmediateRenderQueue;
};
