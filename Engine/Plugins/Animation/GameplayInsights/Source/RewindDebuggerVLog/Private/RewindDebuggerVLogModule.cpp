// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerVLogModule.h"

#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "VisualLogTrack.h"

#define LOCTEXT_NAMESPACE "RewindDebuggerVLogModule"

#ifndef ENABLE_REWINDDEBUGGER_VLOG_INTEGRATION
#define ENABLE_REWINDDEBUGGER_VLOG_INTEGRATION 1
#endif


#if ENABLE_REWINDDEBUGGER_VLOG_INTEGRATION
RewindDebugger::FVisualLogTrackCreator GVisualLogTrackCreator;
#endif

void FRewindDebuggerVLogModule::StartupModule()
{
#if ENABLE_REWINDDEBUGGER_VLOG_INTEGRATION
	IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, &RewindDebuggerVLogExtension);
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &VLogTraceModule);
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GVisualLogTrackCreator);

	RewindDebuggerVLogExtension.Initialize();
#endif
}

void FRewindDebuggerVLogModule::ShutdownModule()
{
#if ENABLE_REWINDDEBUGGER_VLOG_INTEGRATION
	IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, &RewindDebuggerVLogExtension);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &VLogTraceModule);
	IModularFeatures::Get().UnregisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GVisualLogTrackCreator);
#endif
}

IMPLEMENT_MODULE(FRewindDebuggerVLogModule, RewindDebuggerVLog);

#undef LOCTEXT_NAMESPACE
