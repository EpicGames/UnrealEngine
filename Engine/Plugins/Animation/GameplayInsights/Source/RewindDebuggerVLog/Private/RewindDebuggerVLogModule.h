// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "RewindDebuggerVLog.h"
#include "VLogTraceModule.h"

class SRewindDebugger;

class FRewindDebuggerVLogModule : public IModuleInterface
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	FRewindDebuggerVLog RewindDebuggerVLogExtension;
	FVLogTraceModule VLogTraceModule;
};
