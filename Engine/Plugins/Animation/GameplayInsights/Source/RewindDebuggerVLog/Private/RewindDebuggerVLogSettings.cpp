// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerVLogSettings.h"
#include "Misc/CoreDelegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RewindDebuggerVLogSettings)

URewindDebuggerVLogSettings::URewindDebuggerVLogSettings()
{
	FCoreDelegates::OnPreExit.AddLambda([]()
	{
		Get().SaveConfig();
	});
}

#if WITH_EDITOR

FText URewindDebuggerVLogSettings::GetSectionText() const
{
	return NSLOCTEXT("RewindDebuggerVLogSettings", "RewindDebuggerVLogSettingsName", "Rewind Debugger (Visual Log)");
}

FText URewindDebuggerVLogSettings::GetSectionDescription() const
{
	return NSLOCTEXT("RewindDebuggerVLogSettings", "RewindDebuggerVLogSettingsDescription", "Configure options for the visual log tracks in the Rewind Debugger.");
}

#endif

FName URewindDebuggerVLogSettings::GetCategoryName() const
{
	return FName(TEXT("Plugins"));
}

void URewindDebuggerVLogSettings::ToggleCategory(FName Category)
{
	if (DisplayCategories.Remove(Category) == 0)
	{
		DisplayCategories.Add(Category);
	}
	Modify();
	SaveConfig();
}

void URewindDebuggerVLogSettings::PostLoad()
{
	Super::PostLoad();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DisplayVerbosity != 0)
	{
		for (const FName& Category : DisplayCategories)
		{
			if (!DisplayCategoryVerbosities.Contains(Category))
			{
				DisplayCategoryVerbosities.Add(Category, DisplayVerbosity);
			}
		}
		DisplayVerbosity = 0;
		SaveConfig();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void URewindDebuggerVLogSettings::SetCategoryVerbosity(FName Category, ELogVerbosity::Type Verbosity)
{
	DisplayCategoryVerbosities.Add(Category, static_cast<uint8>(Verbosity));
	Modify();
	SaveConfig();
}

ELogVerbosity::Type URewindDebuggerVLogSettings::GetCategoryVerbosity(FName Category) const
{
	if (const uint8* Verbosity = DisplayCategoryVerbosities.Find(Category))
	{
		return static_cast<ELogVerbosity::Type>(*Verbosity);
	}

	return ELogVerbosity::Display;
}

URewindDebuggerVLogSettings& URewindDebuggerVLogSettings::Get()
{
	URewindDebuggerVLogSettings* MutableCDO = GetMutableDefault<URewindDebuggerVLogSettings>();
	check(MutableCDO != nullptr)
	
	return *MutableCDO;
}

