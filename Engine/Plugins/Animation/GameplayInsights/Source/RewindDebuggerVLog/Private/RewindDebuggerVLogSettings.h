// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "Logging/LogVerbosity.h"
#include "RewindDebuggerVLogSettings.generated.h"

/**
 * Settings for the Rewind Debugger Visual Logger integration.
 */
UCLASS(Config=EditorPerProjectUserSettings, meta=(DisplayName="Rewind Debugger - Visual Logging"))
class URewindDebuggerVLogSettings : public UDeveloperSettings
{
	GENERATED_BODY()
	
public:
	URewindDebuggerVLogSettings();

#if WITH_EDITOR
	virtual FText GetSectionText() const override;
	virtual FText GetSectionDescription() const override;
#endif

	virtual FName GetCategoryName() const override;
	virtual void PostLoad() override;
	void ToggleCategory(FName Category);

	UE_DEPRECATED(5.8, "Use DisplayCategoryVerbosities via Get/SetCategoryVerbosity instead.")
	UPROPERTY(Config, meta = (DeprecatedProperty, DeprecationMessage = "Use DisplayCategoryVerbosities instead."))
	uint8 DisplayVerbosity = ELogVerbosity::Display;

	// Display Visual Logger shapes in these categories
	UPROPERTY(EditAnywhere, Config, Category = VisualLogger)
	TSet<FName> DisplayCategories;

	// Per-category display verbosity. Categories without an entry default to ELogVerbosity::Display.
	UPROPERTY(EditAnywhere, Config, Category = VisualLogger)
	TMap<FName, uint8> DisplayCategoryVerbosities;

	void SetCategoryVerbosity(FName Category, ELogVerbosity::Type Verbosity);
	ELogVerbosity::Type GetCategoryVerbosity(FName Category) const;

	static URewindDebuggerVLogSettings & Get();
};
