// Copyright Epic Games, Inc. All Rights Reserved.

#include "VLogRenderingActor.h"
#include "RewindDebuggerVLogSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VLogRenderingActor)

AVLogRenderingActor::AVLogRenderingActor(const FObjectInitializer& ObjectInitializer)
{
}

AVLogRenderingActor::~AVLogRenderingActor()
{
}

void AVLogRenderingActor::Reset()
{
 	DebugShapes.Reset();
 	MarkComponentsRenderStateDirty();
}

void AVLogRenderingActor::AddLogEntry(const FVisualLogEntry& Entry)
{
	GetDebugShapes(Entry, false, DebugShapes);
 	MarkComponentsRenderStateDirty();
}

void AVLogRenderingActor::IterateDebugShapes(TFunction<void(const AVisualLoggerRenderingActorBase::FTimelineDebugShapes& Shapes)> Callback)
{
	Callback(DebugShapes);
 	DebugShapes.Reset();
}

bool AVLogRenderingActor::MatchCategoryFilters(const FName& CategoryName, ELogVerbosity::Type Verbosity) const
{
	URewindDebuggerVLogSettings& Settings = URewindDebuggerVLogSettings::Get();
	return Settings.DisplayCategories.Contains(CategoryName) && Verbosity <= Settings.GetCategoryVerbosity(CategoryName);
}

