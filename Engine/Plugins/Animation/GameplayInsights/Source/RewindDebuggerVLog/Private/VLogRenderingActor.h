// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "VisualLoggerRenderingActorBase.h"
#include "VLogRenderingActor.generated.h"

/**
*	Transient actor used to draw visual logger data on level
*/



UCLASS(config = Engine, NotBlueprintable, Transient, notplaceable, AdvancedClassDisplay)
class AVLogRenderingActor : public AVisualLoggerRenderingActorBase 
{
public:
	GENERATED_UCLASS_BODY()
	virtual ~AVLogRenderingActor();
	void Reset();
	void AddLogEntry(const FVisualLogEntry& LogEntry);

	virtual void IterateDebugShapes(TFunction<void(const FTimelineDebugShapes&) > Callback) override;
	virtual bool MatchCategoryFilters(const FName& CategoryName, ELogVerbosity::Type Verbosity) const override;
	
private:
	FTimelineDebugShapes DebugShapes;
};
