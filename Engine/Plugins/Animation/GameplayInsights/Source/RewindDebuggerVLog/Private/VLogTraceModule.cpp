// Copyright Epic Games, Inc. All Rights Reserved.

#include "VLogTraceModule.h"
#include "VisualLoggerProvider.h"
#include "VisualLoggerAnalyzer.h"

FName FVLogTraceModule::ModuleName("VisualLogger");

void FVLogTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("VisualLogger");
}

void FVLogTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedPtr<FVisualLoggerProvider> VisualLoggerProvider = MakeShared<FVisualLoggerProvider>(InSession);
	InSession.AddProvider(FVisualLoggerProvider::ProviderName, VisualLoggerProvider, VisualLoggerProvider);

	InSession.AddAnalyzer(new FVisualLoggerAnalyzer(InSession, *VisualLoggerProvider));
}

void FVLogTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(TEXT("VisualLogger"));
}

void FVLogTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{

}

