// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLogEntryRenderer.h"
#include "AI/Navigation/NavigationTypes.h"
#include "CoreMinimal.h"
#include "DebugRenderSceneProxy.h"
#include "DrawDebugHelpers.h"
#include "GeomTools.h"
#include "IndexTypes.h"
#include "Engine/Canvas.h"
#include "Engine/World.h"
#include "VisualLogger/VisualLogger.h"
#include "UObject/Package.h"

namespace
{
	static const int CircleSegments = 24;
	
	// utility wrapper for drawing shape descriptions
	inline void RenderDescription(UWorld* World, UCanvas* Canvas, UFont* Font, const FVisualLogShapeElement* ElementToDraw, const FVector& Position, const FColor& Color, int Index, int Count)
	{
		if (Canvas)
		{
			if (!ElementToDraw->Description.IsEmpty())
			{
				const FSceneView* View = Canvas->SceneView;
				if (View->GetCullingFrustum().IntersectPoint(Position))
				{
					const FVector3f ScreenLoc = UE::DebugDrawHelper::GetScaleAdjustedScreenLocation(Canvas, Position);
					Canvas->SetDrawColor(Color);
					const FString PrintString = Count == 1 ? ElementToDraw->Description : FString::Printf(TEXT("%s_%d"), *ElementToDraw->Description, Index);
					Canvas->Canvas->DrawShadowedString(ScreenLoc.X, ScreenLoc.Y, PrintString, Font, Color);
				}
			}
		}
	}

	// utility wrapper for circles
	inline void RenderCircle(UWorld* World, const FVector& Center, const FVector& UpAxis, float Radius, float Thickness, const FColor& Color, uint8 DepthPriority)
	{
		const FQuat Rotation = FQuat::FindBetweenNormals(FVector::UpVector, UpAxis);
		const FVector XAxis = Rotation.RotateVector(FVector::XAxisVector);
		const FVector YAxis = Rotation.RotateVector(FVector::YAxisVector);
				
		DrawCircle(World, Center, XAxis, YAxis, Color, Radius, CircleSegments, false, -1, DepthPriority, Thickness);
	}
		
	static bool IsPolygonWindingCorrect(const TArray<FVector>& Verts)
	{
		// this will work only for convex polys, but we're assuming that all logged polygons are convex in the first place
		if (Verts.Num() >= 3)
		{
			const FVector SurfaceNormal = FVector::CrossProduct(Verts[1] - Verts[0], Verts[2] - Verts[0]);
			const double TestDot = FVector::DotProduct(SurfaceNormal, FVector::UpVector);
			return TestDot > 0.;
		}

		return false;
	}
	static void GetPolygonMesh(const FVisualLogShapeElement* ElementToDraw, TArray<FVector>& Vertices, TArray<int32> Indices, const FVector& VertexOffset = FVector::ZeroVector)
	{
		FClipSMPolygon InPoly(ElementToDraw->Points.Num());
		InPoly.FaceNormal = FVector::UpVector;

		const bool bHasCorrectWinding = IsPolygonWindingCorrect(ElementToDraw->Points);
		if (bHasCorrectWinding)
		{
			for (int32 Index = 0; Index < ElementToDraw->Points.Num(); Index++)
			{
				FClipSMVertex v1;
				v1.Pos = (FVector3f)ElementToDraw->Points[Index];
				InPoly.Vertices.Add(v1);
			}
		}
		else
		{
			for (int32 Index = ElementToDraw->Points.Num() - 1; Index >= 0; Index--)
			{
				FClipSMVertex v1;
				v1.Pos = (FVector3f)ElementToDraw->Points[Index];
				InPoly.Vertices.Add(v1);
			}
		}

		TArray<FClipSMTriangle> OutTris;
	
		const bool bTriangulated = FGeomTools::TriangulatePoly(OutTris, InPoly, false);
		if (bTriangulated)
		{
			int32 LastIndex = 0;

			FGeomTools::RemoveRedundantTriangles(OutTris);
			for (const auto& CurrentTri : OutTris)
			{
				// todo: this float to double conversion is not great for doing world space debug rendering
				Vertices.Add(FVector(CurrentTri.Vertices[0].Pos) + VertexOffset);
				Vertices.Add(FVector(CurrentTri.Vertices[1].Pos) + VertexOffset);
				Vertices.Add(FVector(CurrentTri.Vertices[2].Pos) + VertexOffset);
				Indices.Add(LastIndex++);
				Indices.Add(LastIndex++);
				Indices.Add(LastIndex++);
			}
		}
	}
}


void FVisualLogEntryRenderer::RenderLogEntry(class UWorld* World, const FVisualLogEntry& Entry, TFunctionRef<bool (const FName&, ELogVerbosity::Type)> MatchCategoryFilters,
	UCanvas* Canvas, UFont* Font, UFont* MonospaceFont, int32& ScreenTextY)
{
	const float InverseDPI = 1.f / Canvas->GetDPIScale();
	
	for(const FVisualLogLine& LogLine : Entry.LogLines)
	{
		if (MatchCategoryFilters(LogLine.Category, LogLine.Verbosity))
		{
			ScreenTextY += Canvas->Canvas->DrawShadowedString(20, ScreenTextY, LogLine.Line, (LogLine.bMonospace && MonospaceFont) ? MonospaceFont : Font, LogLine.Color) * InverseDPI;
		}
	}
	
	const FVisualLogShapeElement* ElementToDraw = Entry.ElementsToDraw.GetData();
	const int32 ElementsCount = Entry.ElementsToDraw.Num();

	for (int32 ElementIndex = 0; ElementIndex < ElementsCount; ++ElementIndex, ++ElementToDraw)
	{
		if (!MatchCategoryFilters(ElementToDraw->Category, ElementToDraw->Verbosity))
		{
			continue;
		}

		uint8 DepthPriority = SDPG_Foreground;

		const FVector NavOffset(0., 0., 15.);
		const FVector CorridorOffset = NavOffset * 1.25f;
		const FColor Color = ElementToDraw->GetFColor();
		const EVisualLoggerShapeElement ElementType = ElementToDraw->GetType();
		// The wire elements force the draw type : 
		// const FDebugRenderSceneProxy::EDrawType DrawTypeOverride = (
		// 	(ElementType == EVisualLoggerShapeElement::WireBox) 
		// 	|| (ElementType == EVisualLoggerShapeElement::WireSphere)
		// 	|| (ElementType == EVisualLoggerShapeElement::WireCapsule)
		// 	|| (ElementType == EVisualLoggerShapeElement::WireCone)
		// 	|| (ElementType == EVisualLoggerShapeElement::WireCylinder)) ? FDebugRenderSceneProxy::EDrawType::WireMesh : FDebugRenderSceneProxy::EDrawType::Invalid;

		switch (ElementType)
		{
		case EVisualLoggerShapeElement::SinglePoint:
		case EVisualLoggerShapeElement::Sphere:
		case EVisualLoggerShapeElement::WireSphere:
		{
			const float Radius = float(ElementToDraw->Radius);
			const bool bDrawLabel = (ElementToDraw->Description.IsEmpty() == false);
			const int32 NumPoints = ElementToDraw->Points.Num();
		
			for (int32 Index = 0; Index < NumPoints; ++Index)
			{
				const FVector& Point = ElementToDraw->Points[Index];
				
				if (ElementType == EVisualLoggerShapeElement::WireSphere)
				{
					// for "wireframe" draw some circles that are on the sphere
					RenderCircle(World, Point, FVector(1,0,0), Radius, 1.5, Color, DepthPriority);
					RenderCircle(World, Point, FVector(0,1,0), Radius, 1.5, Color, DepthPriority);
					RenderCircle(World, Point, FVector(0,0,1), Radius, 1.5, Color, DepthPriority);
				}
				else
				{
					if (Radius < UE_SMALL_NUMBER || ElementType == EVisualLoggerShapeElement::SinglePoint)
					{
						DrawDebugPoint(World, Point, 16, Color, false, -1, DepthPriority);
					}
					else
					{
						DrawDebugSphere(World, Point, Radius, CircleSegments, Color, false, -1, DepthPriority);
					}
				}

				RenderDescription(World, Canvas, Font, ElementToDraw, Point, Color, Index, NumPoints);
			}
		}
		break;
		case EVisualLoggerShapeElement::Polygon:
		{
			FDebugRenderSceneProxy::FMesh TestMesh;
			TArray<FVector> Vertices;
			TArray<int32> Indices;
			GetPolygonMesh(ElementToDraw, Vertices, Indices, CorridorOffset);
			DrawDebugMesh(World, Vertices, Indices, Color, false, -1, DepthPriority);
		}
		break;
		case EVisualLoggerShapeElement::Mesh:
		{
			struct FHeaderData
			{
				int32 VerticesNum, FacesNum;
				FHeaderData(const FVector& InVector) : VerticesNum(static_cast<int32>(InVector.X)), FacesNum(static_cast<int32>(InVector.Y)) {}
			};
			const FHeaderData HeaderData(ElementToDraw->Points[0]);
		
			TArray<FVector> Vertices;
			TArray<int32> Indices;
			int32 StartIndex = 1;
			int32 EndIndex = StartIndex + HeaderData.VerticesNum;
			for (int32 VIdx = StartIndex; VIdx < EndIndex; VIdx++)
			{
				Vertices.Add(ElementToDraw->Points[VIdx]);
			}
		
			StartIndex = EndIndex;
			EndIndex = StartIndex + HeaderData.FacesNum;
			for (int32 VIdx = StartIndex; VIdx < EndIndex; VIdx++)
			{
				const FVector &CurrentFace = ElementToDraw->Points[VIdx];
				Indices.Add(static_cast<int32>(CurrentFace.X));
				Indices.Add(static_cast<int32>(CurrentFace.Y));
				Indices.Add(static_cast<int32>(CurrentFace.Z));
			}
			DrawDebugMesh(World, Vertices, Indices, Color, false, -1, DepthPriority);
		}
			break;
		case EVisualLoggerShapeElement::Segment:
		{
			const float Thickness = float(ElementToDraw->Thickness);
			const bool bDrawLabel = (ElementToDraw->Description.IsEmpty() == false);
			const FVector* Location = ElementToDraw->Points.GetData();
			const int32 NumPoints = ElementToDraw->Points.Num();

			for (int32 Index = 0; Index + 1 < NumPoints; Index += 2, Location += 2)
			{
				DrawDebugLine(World, Location[0], Location[1], Color, false, -1, DepthPriority, Thickness);
				RenderDescription(World, Canvas, Font, ElementToDraw, (Location[0] + Location[1])/2, Color, Index/2, NumPoints/2);
			}
		}
			break;
		case EVisualLoggerShapeElement::Path:
		{
			const float Thickness = float(ElementToDraw->Thickness);
			FVector Location = ElementToDraw->Points[0];
			for (int32 Index = 1; Index < ElementToDraw->Points.Num(); ++Index)
			{
				const FVector CurrentLocation = ElementToDraw->Points[Index];
				DrawDebugLine(World, Location, CurrentLocation, Color, false, -1, DepthPriority, Thickness);
				Location = CurrentLocation;
			}
		}
			break;
		case EVisualLoggerShapeElement::Box:
		case EVisualLoggerShapeElement::WireBox:
		{
			const float Thickness = float(ElementToDraw->Thickness);
			const bool bDrawLabel = (ElementToDraw->Description.IsEmpty() == false);
			const FVector* BoxExtent = ElementToDraw->Points.GetData();
			const int32 NumPoints = ElementToDraw->Points.Num();
		
			FTransform Transform(ElementToDraw->TransformationMatrix);
			for (int32 Index = 0; Index + 1 < NumPoints; Index += 2, BoxExtent += 2)
			{
				const FBox Box = FBox(*BoxExtent, *(BoxExtent + 1));
				// todo: wireframe, 
				DrawDebugSolidBox(World, Box, Color, Transform, false, -1.f, DepthPriority);
				RenderDescription(World, Canvas, Font, ElementToDraw, Transform.TransformPosition(Box.GetCenter()), Color, Index/2, NumPoints/2);
			}
		}
			break;
		case EVisualLoggerShapeElement::Cone:
		case EVisualLoggerShapeElement::WireCone:
		{
		 	const float Thickness = float(ElementToDraw->Thickness);
		 	const bool bDrawLabel = ElementToDraw->Description.IsEmpty() == false;
		 	for (int32 Index = 0; Index + 2 < ElementToDraw->Points.Num(); Index += 3)
		 	{
		 		const FVector Origin = ElementToDraw->Points[Index];
				const FVector Direction = ElementToDraw->Points[Index + 1].GetSafeNormal();
				const FVector Angles = ElementToDraw->Points[Index + 2];
				const double Length = Angles.X;
		
				FVector YAxis, ZAxis;
				Direction.FindBestAxisVectors(YAxis, ZAxis);

				DrawDebugCone(World, Origin, Direction, Length, Angles.Y, Angles.Z, CircleSegments, Color, false, -1.f, DepthPriority, Thickness);
				RenderDescription(World, Canvas, Font, ElementToDraw, Origin, Color, Index/3, ElementToDraw->Points.Num()/3);
			}
		}
			break;
		case EVisualLoggerShapeElement::Cylinder:
		case EVisualLoggerShapeElement::WireCylinder:
		{
			const float Thickness = float(ElementToDraw->Thickness);
			const bool bDrawLabel = ElementToDraw->Description.IsEmpty() == false;
			for (int32 Index = 0; Index + 2 < ElementToDraw->Points.Num(); Index += 3)
			{
				const FVector Start = ElementToDraw->Points[Index];
				const FVector End = ElementToDraw->Points[Index + 1];
				const FVector OtherData = ElementToDraw->Points[Index + 2];
				const FVector Center = 0.5 * (Start + End);

				DrawDebugCylinder(World, Start, End, static_cast<float>(OtherData.X), CircleSegments, Color, false, -1.f, DepthPriority, Thickness);
				RenderDescription(World, Canvas, Font, ElementToDraw, Center, Color, Index/3, ElementToDraw->Points.Num()/3);
			}
		}
			break;
		case EVisualLoggerShapeElement::Capsule:
		case EVisualLoggerShapeElement::WireCapsule:
		{
			const float Thickness = float(ElementToDraw->Thickness);
			const bool bDrawLabel = ElementToDraw->Description.IsEmpty() == false;
			for (int32 Index = 0; Index + 2 < ElementToDraw->Points.Num(); Index += 3)
			{
				const FVector Base = ElementToDraw->Points[Index + 0];
				const FVector FirstData = ElementToDraw->Points[Index + 1];
				const FVector SecondData = ElementToDraw->Points[Index + 2];
				const float HalfHeight = static_cast<float>(FirstData.X);
				const float Radius = static_cast<float>(FirstData.Y);
				const FQuat Rotation = FQuat(FirstData.Z, SecondData.X, SecondData.Y, SecondData.Z);

				const FVector Center = Base + Rotation.RotateVector(FVector(0, 0, HalfHeight));
				DrawDebugCapsule(World, Center, HalfHeight, Radius, Rotation, Color, false, -1.f, DepthPriority, Thickness);
				RenderDescription(World, Canvas, Font, ElementToDraw, Center, Color, Index/3, ElementToDraw->Points.Num()/3);
			}
		}
			break;
		case EVisualLoggerShapeElement::NavAreaMesh:
		{
			if (ElementToDraw->Points.Num() == 0)
			{
				continue;
			}
		
			struct FHeaderData
			{
				float MinZ, MaxZ;
				FHeaderData(const FVector& InVector) : MinZ(FloatCastChecked<float>(InVector.X, UE::LWC::DefaultFloatPrecision)), MaxZ(FloatCastChecked<float>(InVector.Y, UE::LWC::DefaultFloatPrecision)) {}
			};
			const FHeaderData HeaderData(ElementToDraw->Points[0]);
		
			TArray<FVector> AreaMeshPoints = ElementToDraw->Points;
			AreaMeshPoints.RemoveAt(0, EAllowShrinking::No);
			AreaMeshPoints.Add(ElementToDraw->Points[1]);
			TNavStatArray<FVector> Faces;
			int32 CurrentIndex = 0;
			TArray<FVector> Vertices;
			TArray<int> Indices;
			Vertices.Reserve((AreaMeshPoints.Num() - 1) * 6);
			Indices.Reserve((AreaMeshPoints.Num() - 1) * 6);
		
			for (int32 PointIndex = 0; PointIndex < AreaMeshPoints.Num() - 1; PointIndex++)
			{
				FVector Point = AreaMeshPoints[PointIndex];
				FVector NextPoint = AreaMeshPoints[PointIndex + 1];
		
				
				// LWC_TODO These won't work well with very large coordinates!
				FVector P1(Point.X, Point.Y, HeaderData.MinZ);
				FVector P2(Point.X, Point.Y, HeaderData.MaxZ);
				FVector P3(NextPoint.X, NextPoint.Y, HeaderData.MinZ);
				FVector P4(NextPoint.X, NextPoint.Y, HeaderData.MaxZ);
		
				Vertices.Add(P1); 
				Vertices.Add(P2); 
				Vertices.Add(P3);
		
				Indices.Add(CurrentIndex++);
				Indices.Add(CurrentIndex++);
				Indices.Add(CurrentIndex++);
		
				Vertices.Add(P3); 
				Vertices.Add(P2); 
				Vertices.Add(P4);
		
				Indices.Add(CurrentIndex++);
				Indices.Add(CurrentIndex++);
				Indices.Add(CurrentIndex++);
			}
				
			DrawDebugMesh(World, Vertices, Indices, Color, false, -1, DepthPriority);
		
			for (int32 VIdx = 0; VIdx < AreaMeshPoints.Num(); VIdx++)
			{
				DrawDebugLine(World,
					AreaMeshPoints[VIdx] + FVector(0., 0., HeaderData.MaxZ),
					AreaMeshPoints[(VIdx + 1) % AreaMeshPoints.Num()] + FVector(0., 0., HeaderData.MaxZ),
					Color, false, -1, DepthPriority, 1.5);
			}
		
		}
			break;
		case EVisualLoggerShapeElement::Arrow:
		{
			const bool bDrawLabel = (ElementToDraw->Description.IsEmpty() == false);
			const FVector* Location = ElementToDraw->Points.GetData();
			const int32 NumPoints = ElementToDraw->Points.Num();
		
			for (int32 Index = 0; Index + 1 < NumPoints; Index += 2, Location += 2)
			{
				DrawDebugDirectionalArrow(World, Location[0], Location[1], 40, Color, false, -1, DepthPriority, 1.5);
				RenderDescription(World, Canvas, Font, ElementToDraw, (Location[0] + Location[1]) / 2, Color, Index/2, ElementToDraw->Points.Num()/2);
			}
		}
		break;
		
		case EVisualLoggerShapeElement::Circle:
		case EVisualLoggerShapeElement::WireCircle:
		{
			const bool bDrawLabel = (ElementToDraw->Description.IsEmpty() == false);
				
			const int32 NumPoints = ElementToDraw->Points.Num();
		
			for (int32 Index = 0; Index + 2 < NumPoints; Index += 3)
			{
				const FVector Center = ElementToDraw->Points[Index + 0];
				const FVector UpAxis = ElementToDraw->Points[Index + 1];
				const double Radius = ElementToDraw->Points[Index + 2].X;
				const float Thickness = float(ElementToDraw->Thickness);

				RenderCircle(World, Center, UpAxis, Radius, Thickness, Color, DepthPriority);
				RenderDescription(World, Canvas, Font, ElementToDraw, Center, Color, Index/3, ElementToDraw->Points.Num()/3);
			}
		}
			break;
		case EVisualLoggerShapeElement::CoordinateSystem:
		{
			const float Thickness = float(ElementToDraw->Thickness);
			const bool bDrawLabel = ElementToDraw->Description.IsEmpty() == false;
			const int32 NumPoints = ElementToDraw->Points.Num();

			for (int32 Index = 0; Index + 2 < NumPoints; Index += 3)
			{
				const FVector AxisLoc = ElementToDraw->Points[Index];
				const FRotator AxisRot = ElementToDraw->Points[Index + 1].ToOrientationRotator();
				const float Scale = ElementToDraw->Points[Index + 2].X;

				DrawDebugCoordinateSystem(World, AxisLoc, AxisRot, Scale, /*bPersistentLines*/false, /*LifeTime*/-1, DepthPriority, Thickness);
				RenderDescription(World, Canvas, Font, ElementToDraw, AxisLoc, Color, Index/3, ElementToDraw->Points.Num()/3);
			}
		}

			break;
		case EVisualLoggerShapeElement::Invalid:
			UE_LOGF(LogVisual, Warning, "Invalid element type");
			break;
		
		default:
			UE_LOGF(LogVisual, Warning, "Unhandled element type: %d", int(ElementToDraw->GetType()));
		}
	}
}