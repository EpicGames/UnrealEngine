// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class FVisualLogEntryRenderer
{
public:
	static void RenderLogEntry(class UWorld* World, const struct FVisualLogEntry& Entry, TFunctionRef<bool(const FName&, ELogVerbosity::Type)> MatchCategoryFilters,
		class UCanvas* Canvas, class UFont* Font, class UFont* MonospaceFont, int32& ScreenTextY);
};
