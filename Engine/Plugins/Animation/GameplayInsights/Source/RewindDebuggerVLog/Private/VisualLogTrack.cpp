// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLogTrack.h"

#include "Common/ProviderLock.h"
#include "IGameplayProvider.h"
#include "IRewindDebugger.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "VisualLoggerProvider.h"
#include "SVisualLoggerTableRow.h"
#include "TraceServices/Model/Frames.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "VisualLogTrack"

namespace RewindDebugger
{

static TArray<TPair<FName, FText>> CategoryNameAsTextMappings;

FText& GetCategoryNameAsText(FName InCategory)
{
	TPair<FName, FText>* CachedEntry = CategoryNameAsTextMappings.FindByPredicate([InCategory](const TPair<FName, FText>& Pair)
		{
			return Pair.Key == InCategory;
		});

	if (CachedEntry == nullptr)
	{
		CachedEntry = &CategoryNameAsTextMappings.Add_GetRef({ InCategory, FText::FromName(InCategory) });
	}

	return CachedEntry->Value;
}

static const FLazyName VisualLogName("Visual Logging");

//----------------------------------------------------------------------//
// FVisualLogCategoryTrack
//----------------------------------------------------------------------//
FVisualLogCategoryTrack::FVisualLogCategoryTrack(uint64 InObjectId, const FName& InCategory) :
	ObjectId(InObjectId),
	Category(InCategory)
{
	TrackName = GetCategoryNameAsText(Category);
	EventData = MakeShared<SEventTimelineView::FTimelineEventData>();
	Icon = FSlateIcon("EditorStyle", "Sequencer.Tracks.Event", "Sequencer.Tracks.Event");

	SAssignNew(ListView, SListView<TSharedPtr<FLogEntryItem>>)
		.ListItemsSource(&LogEntries)
		.SelectionMode(ESelectionMode::None)
		.OnGenerateRow_Lambda([](TSharedPtr<FLogEntryItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
			{
				return SNew(SVisualLoggerTableRow, OwnerTable, Item);
			})
		.HeaderRow
		(
			SNew(SHeaderRow)
			+ SHeaderRow::Column(SVisualLoggerTableRow::ColumnId_VerbosityLabel)
				.FillWidth(0.1f)
				.DefaultLabel(LOCTEXT("VLoggerVerbosityLabel", "Verbosity"))
			+ SHeaderRow::Column(SVisualLoggerTableRow::ColumnId_LogLabel)
				.FillWidth(0.8f)
				.DefaultLabel(LOCTEXT("VLoggerLogLabel", "Log"))
		);
}

TSharedPtr<SEventTimelineView::FTimelineEventData> FVisualLogCategoryTrack::GetEventData() const
{
	if (!EventData.IsValid())
	{
		EventData = MakeShared<SEventTimelineView::FTimelineEventData>();
	}

	EventUpdateRequested++;

	return EventData;
}

TSharedPtr<SWidget> FVisualLogCategoryTrack::GetDetailsViewInternal()
{
	return ListView;
}

bool FVisualLogCategoryTrack::UpdateInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();

	TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	double StartTime = TraceTimeRange.GetLowerBoundValue();
	double EndTime = TraceTimeRange.GetUpperBoundValue();

	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();

	if (const FVisualLoggerProvider* VisLogProvider = AnalysisSession->ReadProvider<FVisualLoggerProvider>(FVisualLoggerProvider::ProviderName))
	{
		if (EventUpdateRequested > 10)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FVisualLogTrack::UpdateEventPointsInternal);
			EventUpdateRequested = 0;

			EventData->EventPoints.SetNum(0, EAllowShrinking::No);
			EventData->EventWindows.SetNum(0);

			TraceServices::FProviderReadScopeLock ProviderReadScope(*VisLogProvider);

			VisLogProvider->ReadVisualLogEntryTimeline(ObjectId, [this, StartTime, EndTime, VisLogProvider, AnalysisSession](const FVisualLoggerProvider::VisualLogEntryTimeline& InTimeline)
				{
					InTimeline.EnumerateEvents(StartTime, EndTime, [this, StartTime, EndTime, VisLogProvider, AnalysisSession](double InStartTime, double InEndTime, uint32 InDepth, const FVisualLogEntry& InMessage)
						{
							for (const FVisualLogShapeElement& Element : InMessage.ElementsToDraw)
							{
								if (Element.Category == Category)
								{
									EventData->EventPoints.Add({ InMessage.TimeStamp,GetCategoryNameAsText(Element.Category), FText::FromString(Element.Description),Element.GetFColor() });
								}
							}
							for (const FVisualLogLine& Line : InMessage.LogLines)
							{
								if (Line.Category == Category)
								{
									// No Description from log lines.
									EventData->EventPoints.Add({ InMessage.TimeStamp, GetCategoryNameAsText(Line.Category), FText::GetEmpty(), Line.Color });
								}
							}
							return TraceServices::EEventEnumerate::Continue;
						});
				});
		}


		double CurrentScrubTime = IRewindDebugger::Instance()->CurrentTraceTime();
		if (PreviousScrubTime != CurrentScrubTime)
		{
			PreviousScrubTime = CurrentScrubTime;
			LogEntries.Reset();

			bool bFrameFound;
			TraceServices::FFrame MarkerFrame;
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
				const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
				bFrameFound = FrameProvider.GetFrameFromTime(TraceFrameType_Game, CurrentScrubTime, MarkerFrame);
			}

			if (bFrameFound)
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*VisLogProvider);
				VisLogProvider->ReadVisualLogEntryTimeline(ObjectId,
					[this, &Entries = LogEntries, &MarkerFrame, EndTime, VisLogProvider, AnalysisSession](const FVisualLoggerProvider::VisualLogEntryTimeline& InTimeline)
					{
						InTimeline.EnumerateEvents(MarkerFrame.StartTime, MarkerFrame.EndTime,
							[this, &Entries, VisLogProvider, AnalysisSession](double InStartTime, double InEndTime, uint32 InDepth, const FVisualLogEntry& InMessage)
							{
								for (const FVisualLogShapeElement& Element : InMessage.ElementsToDraw)
								{
									if (Element.Category == Category
										&& !Element.Description.IsEmpty())
									{
										Entries.Add(MakeShared<FLogEntryItem>(
											FLogEntryItem
											{
												.Category = Element.Category.ToString(),
												.CategoryColor = Element.GetFColor(),
												.Verbosity = Element.Verbosity,
												.Line = Element.Description
											}));
									}
								}
								for (const FVisualLogLine& Line : InMessage.LogLines)
								{
									if (Line.Category == Category)
									{
										Entries.Add(MakeShared<FLogEntryItem>(
											FLogEntryItem
											{
												.Category = Line.Category.ToString(),
												.CategoryColor = Line.Color,
												.Verbosity = Line.Verbosity,
												.Line = Line.Line,
												.UserData = Line.UserData,
												.TagName = Line.TagName
											}));
									}
								}
								return TraceServices::EEventEnumerate::Continue;
							});
					});
			}
			ListView->RebuildList();
		}
	}

	constexpr bool bChanged = false;
	return bChanged;
}

TSharedPtr<SWidget> FVisualLogCategoryTrack::GetTimelineViewInternal()
{
	return SNew(SEventTimelineView)
		.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
		.EventData_Raw(this, &FVisualLogCategoryTrack::GetEventData);
}

FText FVisualLogCategoryTrack::GetStepCommandTooltipInternal(const RewindDebugger::EStepMode StepMode) const
{
	switch (StepMode)
	{
	case RewindDebugger::EStepMode::Forward:
		return LOCTEXT("StepForwardTooltip", "Step to next event in Logging track");
	case RewindDebugger::EStepMode::Backward:
		return LOCTEXT("StepBackwardTooltip", "Step to previous event in Logging track");
	}

	return {};
}

TOptional<double> FVisualLogCategoryTrack::GetStepFrameTimeInternal(const EStepMode StepMode, const FScrubTimeInformation& CurrentScrubTime) const
{
	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	if (RewindDebugger == nullptr)
	{
		return {};
	}

	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	if (AnalysisSession == nullptr)
	{
		return {};
	}

	const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider");
	if (GameplayProvider == nullptr)
	{
		return {};
	}

	const FVisualLoggerProvider* VisLogProvider = AnalysisSession->ReadProvider<FVisualLoggerProvider>(FVisualLoggerProvider::ProviderName);
	if (VisLogProvider == nullptr)
	{
		return {};
	}

	const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
	TraceServices::FProviderReadScopeLock GameProviderReadScope(*GameplayProvider);

	// Legacy Index that is always 0 now
	constexpr int32 RecordingIndex = 0;
	const IGameplayProvider::RecordingInfoTimeline* Recording = GameplayProvider->GetRecordingInfo(RecordingIndex);
	if (Recording == nullptr)
	{
		return {};
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
	TraceServices::FFrame Frame;
	if (FramesProvider.GetFrameFromTime(TraceFrameType_Game, CurrentScrubTime.ProfileTime, Frame))
	{
		TraceServices::FProviderReadScopeLock VisLogProviderReadScope(*VisLogProvider);

		bool bEventFound = false;

		// Set the frame index increment based on the step mode
		const int32 FrameInc = (StepMode == EStepMode::Backward) ? -1 : 1;

		// Set the index of our first candidate
		// Note that we don't care about decrementing pass value 0 since GetFrame will simply fail
		uint64 CandidateFrameIndex = Frame.Index + FrameInc;

		while (const TraceServices::FFrame* CandidateFrame = FramesProvider.GetFrame(TraceFrameType_Game, CandidateFrameIndex))
		{
			// Inspect the VisLog timeline to see if there are events for that frame
			VisLogProvider->ReadVisualLogEntryTimeline(ObjectId,
				[&CandidateFrame, &bEventFound, Category = Category](const FVisualLoggerProvider::VisualLogEntryTimeline& InTimeline)
				{
					InTimeline.EnumerateEvents(CandidateFrame->StartTime, CandidateFrame->EndTime,
						[&bEventFound, Category](double InStartTime, double InEndTime, uint32 InDepth, const FVisualLogEntry& InMessage)
						{
							for (const FVisualLogShapeElement& Element : InMessage.ElementsToDraw)
							{
								if (Element.Category == Category)
								{
									bEventFound = true;
									break;
								}
							}
							for (const FVisualLogLine& Line : InMessage.LogLines)
							{
								if (Line.Category == Category)
								{
									bEventFound = true;
								}
							}
							return bEventFound ? TraceServices::EEventEnumerate::Stop : TraceServices::EEventEnumerate::Continue;
						});
				});

			// Event was found
			if (bEventFound)
			{
				const FRecordingInfoMessage* MatchingRecordingInfo = nullptr;
				Recording->EnumerateEvents(CandidateFrame->StartTime, CandidateFrame->EndTime,
					[&MatchingRecordingInfo](double InStartTime, double InEndTime, uint32 InDepth, const FRecordingInfoMessage& InMessage)
					{
						MatchingRecordingInfo = &InMessage;
						return TraceServices::EEventEnumerate::Stop;
					});

				if (!ensureMsgf(MatchingRecordingInfo, TEXT("We expect to always have recording info for any frame with events")))
				{
					// Fallback on default behavior
					return {};
				}

				return MatchingRecordingInfo->ElapsedTime;
			}

			// Try prev/next frame
			CandidateFrameIndex += FrameInc;
		}
	}

	// Everything was properly analyzed and there is simply no more events before/after so preserve the current time
	return CurrentScrubTime.ElapsedTime;
}

//----------------------------------------------------------------------//
// FVisualLogTrack
//----------------------------------------------------------------------//

FVisualLogTrack::FVisualLogTrack(const uint64 InObjectId) :
	ObjectId(InObjectId)
{
	Icon = FSlateIcon("EditorStyle", "Sequencer.Tracks.Event", "Sequencer.Tracks.Event");
}

bool FVisualLogTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVisualLogTrack::UpdateInternal);
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();

	TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	double StartTime = TraceTimeRange.GetLowerBoundValue();
	double EndTime = TraceTimeRange.GetUpperBoundValue();

	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FVisualLoggerProvider* VisLogProvider = AnalysisSession->ReadProvider<FVisualLoggerProvider>(FVisualLoggerProvider::ProviderName);

	bool bChanged = false;

	if (VisLogProvider)
	{
		TArray<FName> UniqueTrackIds;

		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*VisLogProvider);

			VisLogProvider->ReadVisualLogEntryTimeline(ObjectId, [this, StartTime, EndTime, VisLogProvider, AnalysisSession, &UniqueTrackIds](const FVisualLoggerProvider::VisualLogEntryTimeline& InTimeline)
				{
					InTimeline.EnumerateEvents(StartTime, EndTime, [this, StartTime, EndTime, VisLogProvider, AnalysisSession, &UniqueTrackIds](double InStartTime, double InEndTime, uint32 InDepth, const FVisualLogEntry& InMessage)
						{
							if (InMessage.LogLines.Num() > 0)
							{
								for (const FVisualLogLine& Line : InMessage.LogLines)
								{
									UniqueTrackIds.AddUnique(Line.Category);
								}
							}
							for (const FVisualLogShapeElement& Element : InMessage.ElementsToDraw)
							{
								UniqueTrackIds.AddUnique(Element.Category);
							}
							return TraceServices::EEventEnumerate::Continue;
						});
				});
		}

		UniqueTrackIds.StableSort([](const FName& A, const FName& B) { return A.ToString() < B.ToString(); });
		const int32 TrackCount = UniqueTrackIds.Num();

		if (Children.Num() != TrackCount)
		{
			UE_LOGF(LogRewindDebugger, Verbose, "List changed because track count (%d) != current number of children (%d)", TrackCount, Children.Num());
			bChanged = true;
		}

		Children.SetNum(UniqueTrackIds.Num());
		for (int i = 0; i < TrackCount; i++)
		{
			if (!Children[i].IsValid() || Children[i].Get()->GetName() != UniqueTrackIds[i])
			{
				Children[i] = MakeShared<FVisualLogCategoryTrack>(ObjectId, UniqueTrackIds[i]);
				bChanged = true;
			}

			if (Children[i]->Update())
			{
				bChanged = true;
			}
		}
	}

	return bChanged;
}

TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> FVisualLogTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(Children.GetData()), Children.Num());
}

TSharedPtr<SWidget> FVisualLogTrack::GetDetailsViewInternal()
{
	return nullptr;
}

//----------------------------------------------------------------------//
// FVisualLogTrackCreator
//----------------------------------------------------------------------//
FName FVisualLogTrackCreator::GetTargetTypeNameInternal() const
{
	return UObject::StaticClass()->GetFName();
}

FName FVisualLogTrackCreator::GetNameInternal() const
{
	return VisualLogName;
}

void FVisualLogTrackCreator::GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const
{
	Types.Add({VisualLogName, LOCTEXT("Visual Logging", "Visual Logging")});
}

TSharedPtr<FRewindDebuggerTrack> FVisualLogTrackCreator::CreateTrackInternal(const FObjectId& InObjectId) const
{
	return MakeShared<FVisualLogTrack>(InObjectId.GetMainId());
}

bool FVisualLogTrackCreator::HasDebugInfoInternal(const FObjectId& InObjectId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVisualLogTrack::HasDebugInfoInternal);
	const TraceServices::IAnalysisSession* AnalysisSession = IRewindDebugger::Instance()->GetAnalysisSession();

	bool bHasData = false;
	if (const FVisualLoggerProvider* VLogProvider = AnalysisSession->ReadProvider<FVisualLoggerProvider>(FVisualLoggerProvider::ProviderName))
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*VLogProvider);
		VLogProvider->ReadVisualLogEntryTimeline(InObjectId.GetMainId(), [&bHasData](const FVisualLoggerProvider::VisualLogEntryTimeline& InTimeline)
			{
				bHasData = true;
			});
	}
	return bHasData;
}

} // RewindDebugger

#undef LOCTEXT_NAMESPACE