// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IRewindDebuggerTrackCreator.h"
#include "RewindDebuggerTrack.h"
#include "SEventTimelineView.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Views/SListView.h"

struct FLogEntryItem;

namespace RewindDebugger
{

/**
 * Tracks representing all log lines of a given category 
 */
class FVisualLogCategoryTrack : public FRewindDebuggerTrack
{
public:
	FVisualLogCategoryTrack(uint64 InObjectId, const FName& Category);
	FName GetCategory() const
	{
		return Category;
	}

private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
	virtual FText GetStepCommandTooltipInternal(EStepMode) const override;
	virtual TOptional<double> GetStepFrameTimeInternal(EStepMode, const FScrubTimeInformation&) const override;

	virtual FSlateIcon GetIconInternal() override
	{
		return Icon;
	}

	virtual FText GetDisplayNameInternal() const override
	{
		return TrackName;
	}

	virtual uint64 GetObjectIdInternal() const override
	{
		return ObjectId;
	}

	virtual FName GetNameInternal() const override
	{
		return Category;
	}

	TSharedPtr<SEventTimelineView::FTimelineEventData> GetEventData() const;
	
	mutable TSharedPtr<SEventTimelineView::FTimelineEventData> EventData;
	mutable int EventUpdateRequested = 0;
	
	uint64 ObjectId;
	FName Category;
	FSlateIcon Icon;
	FText TrackName;
	TSharedPtr<SListView<TSharedPtr<FLogEntryItem>>> ListView;
	TArray<TSharedPtr<FLogEntryItem>> LogEntries;

	double PreviousScrubTime = 0.0f;
	
	enum class EFilterType
	{
		VisualLog,
		VisualLogState,
		SyncMarker,
	};
};

/**
 * Parent track for category tracks
 */
class FVisualLogTrack : public FRewindDebuggerTrack
{
public:
	explicit FVisualLogTrack(uint64 InObjectId);

private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;

	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FName GetNameInternal() const override { return "Visual Logging"; }
	virtual FText GetDisplayNameInternal() const override { return NSLOCTEXT("RewindDebugger", "VisualLoggerTrackName", "Visual Logging"); }
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;

	FSlateIcon Icon;
	uint64 ObjectId;

	TArray<TSharedPtr<FVisualLogCategoryTrack>> Children;
};

#if WITH_ENGINE
/**
 * Track creator for VisualLogger recorded entries
 */
class FVisualLogTrackCreator : public IRewindDebuggerTrackCreator
{
	virtual FName GetTargetTypeNameInternal() const override;
	virtual FName GetNameInternal() const override;
	virtual void GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<FRewindDebuggerTrack> CreateTrackInternal(const FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const FObjectId& InObjectId) const override;
};
#endif // WITH_ENGINE

}
