// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLoggerAnalyzer.h"

#include "Common/ProviderLock.h"
#include "HAL/LowLevelMemTracker.h"
#include "Serialization/MemoryReader.h"
#include "VisualLoggerProvider.h"

FVisualLoggerAnalyzer::FVisualLoggerAnalyzer(TraceServices::IAnalysisSession& InSession, FVisualLoggerProvider& InProvider)
	: Session(InSession)
	, Provider(InProvider)
{
}

void FVisualLoggerAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Builder.RouteEvent(RouteId_VisualLogEntry, "VisualLogger", "VisualLogEntry");
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Builder.RouteEvent(RouteId_VisualLogEntryHeader, "VisualLogger", "VisualLogEntryHeader");
	Builder.RouteEvent(RouteId_VisualLogEntryChunk, "VisualLogger", "VisualLogEntryChunk");
}

bool FVisualLoggerAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FVisualLoggerAnalyzer"));

	TraceServices::FProviderEditScopeLock ProviderEditScope(Provider);

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
		case RouteId_VisualLogEntryHeader:
		{
			FVLogEntry& LogEntry = Provider.AddLogEntry();
			LogEntry.EntryID = EventData.GetValue<uint32>("ID");
			const uint64 Cycle = EventData.GetValue<uint64>("Cycle");
			LogEntry.RecordingTime = Context.EventTime.AsSeconds(Cycle);
			LogEntry.RecordingWorldTime = EventData.GetValue<double>("RecordingTime");
			LogEntry.OwnerID = EventData.GetValue<uint64>("OwnerID");
			LogEntry.ChunkNum = EventData.GetValue<uint16>("TotalChunkNum");
			LogEntry.Size = EventData.GetValue<uint32>("Size");
			LogEntry.Data.Reserve(LogEntry.Size);

			break;
		}

		case RouteId_VisualLogEntryChunk:
		{
			const uint32 ID = EventData.GetValue<uint32>("ID");
			const uint16 ChunkNum = EventData.GetValue<uint16>("ChunkNum");
			const uint16 Size = EventData.GetValue<uint16>("Size");
			TArrayView<const uint8> Data = EventData.GetArrayView<uint8>("Data");

			Provider.AddLogEntryChunk(ID, ChunkNum, Size, Data);

			break;
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		case RouteId_VisualLogEntry:
		{
			uint64 OwnerId = EventData.GetValue<uint64>("OwnerId");
			uint64 Cycle = EventData.GetValue<uint64>("Cycle");
			double RecordingTime = EventData.GetValue<double>("RecordingTime");
			TArrayView<const uint8> SerializedData = EventData.GetArrayView<uint8>("LogEntry");
			FMemoryReaderView Archive(SerializedData);
			FVisualLogEntry Entry;
			Archive << Entry;
			Entry.TimeStamp = RecordingTime;
			Provider.AppendVisualLogEntry(OwnerId, Context.EventTime.AsSeconds(Cycle), Entry);
			break;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return true;
}
