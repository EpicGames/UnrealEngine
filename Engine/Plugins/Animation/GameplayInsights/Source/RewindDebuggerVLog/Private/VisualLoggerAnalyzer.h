// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

class FVisualLoggerProvider;
namespace TraceServices { class IAnalysisSession; }

class FVisualLoggerAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FVisualLoggerAnalyzer(TraceServices::IAnalysisSession& InSession, FVisualLoggerProvider& InProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override {}
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_VisualLogEntry UE_DEPRECATED(5.8, "Used only to read old recordings; now using header + chunk to handle large events"),
		RouteId_VisualLogEntryHeader,
		RouteId_VisualLogEntryChunk,
	};

	TraceServices::IAnalysisSession& Session;
	FVisualLoggerProvider& Provider;
};
