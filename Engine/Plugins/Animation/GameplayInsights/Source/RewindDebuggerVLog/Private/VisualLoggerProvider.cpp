// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLoggerProvider.h"
#include "Serialization/MemoryReader.h"

namespace TraceServices
{
thread_local FProviderLock::FThreadLocalState GVisualLoggerProviderLockState;
}

FName FVisualLoggerProvider::ProviderName("VisualLoggerProvider");

#define LOCTEXT_NAMESPACE "VisualLoggerProvider"

FVisualLoggerProvider::FVisualLoggerProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
{
}

bool FVisualLoggerProvider::ReadVisualLogEntryTimeline(uint64 InObjectId, TFunctionRef<void(const VisualLogEntryTimeline&)> Callback) const
{
	ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToLogEntryTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(LogEntryTimelines.Num()))
		{
			Callback(*LogEntryTimelines[*IndexPtr]);
			return true;
		}
	}

	return false;
}

void FVisualLoggerProvider::EnumerateCategories(TFunctionRef<void(const FName&)> Callback) const
{
	ReadAccessCheck();

	for(const FName& Category : Categories)
	{
		Callback(Category);
	}
}

void  FVisualLoggerProvider::AppendVisualLogEntry(uint64 InObjectId, double InTime, const FVisualLogEntry& Entry)
{
	EditAccessCheck();

	TSharedPtr<TraceServices::TPointTimeline<FVisualLogEntry, FVLogTimelineSettings>> Timeline;
	uint32* IndexPtr = ObjectIdToLogEntryTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		Timeline = LogEntryTimelines[*IndexPtr];
	}
	else
	{
		Timeline = MakeShared<TraceServices::TPointTimeline<FVisualLogEntry, FVLogTimelineSettings>>(Session.GetLinearAllocator());
		ObjectIdToLogEntryTimelines.Add(InObjectId, LogEntryTimelines.Num());
		LogEntryTimelines.Add(Timeline.ToSharedRef());
	}

	Timeline->AppendEvent(InTime, Entry);

	for (const FVisualLogLine& Line : Entry.LogLines)
	{
		Categories.AddUnique(Line.Category);
	}

	for (const FVisualLogShapeElement& Shape : Entry.ElementsToDraw)
	{
		Categories.AddUnique(Shape.Category);
	}
}

FVLogEntry& FVisualLoggerProvider::AddLogEntry()
{
	EditAccessCheck();

	return LogEntries.AddDefaulted_GetRef();
}

void FVisualLoggerProvider::AddLogEntryChunk(const uint32 ID, const uint32 ChunkNum, const uint16 Size, const TArrayView<const uint8>& ChunkData)
{
	EditAccessCheck();

	const int32 Index = LogEntries.IndexOfByPredicate([ID](const FVLogEntry& Entry) { return Entry.EntryID == ID; });
	if (Index == INDEX_NONE)
	{
		return;
	}

	FVLogEntry& LogEntry = LogEntries[Index];
	constexpr uint32 ChunkSize = TNumericLimits<uint16>::Max();
	check(LogEntry.Data.Num() == ChunkNum * ChunkSize);

	LogEntry.Data.Append(ChunkData.GetData(), Size);

	if (LogEntry.Size == LogEntry.Data.Num())
	{
		FMemoryReaderView Archive(LogEntry.Data);
		FVisualLogEntry Entry;
		Archive << Entry;
		Entry.TimeStamp = LogEntry.RecordingWorldTime;
		AppendVisualLogEntry(LogEntry.OwnerID, LogEntry.RecordingTime, Entry);
		LogEntries.RemoveAtSwap(Index);
	}
}

#undef LOCTEXT_NAMESPACE
