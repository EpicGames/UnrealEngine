// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IVisualLoggerProvider.h"
#include "Common/ProviderLock.h"
#include "Model/PointTimeline.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "VisualLogger/VisualLoggerTypes.h"

namespace TraceServices
{
class IAnalysisSession;
extern thread_local FProviderLock::FThreadLocalState GVisualLoggerProviderLockState;
}

struct FVLogTimelineSettings
{
	enum
	{
		EventsPerPage = 4096
	};
};

struct FVLogEntry
{
	uint32 EntryID = INDEX_NONE;
	uint32 Size = 0;
	uint64 OwnerID = INDEX_NONE;
	double RecordingTime = 0;
	double RecordingWorldTime = 0;
	TArray<uint8> Data;
	uint16 ChunkNum = 0;
};

class FVisualLoggerProvider : public IVisualLoggerProvider
{
public:
	static FName ProviderName;

	explicit FVisualLoggerProvider(TraceServices::IAnalysisSession& InSession);

	//~ IProvider (read lock)
	virtual void BeginRead() const override
	{
		Lock.BeginRead(TraceServices::GVisualLoggerProviderLockState);
	}
	virtual void EndRead() const override
	{
		Lock.EndRead(TraceServices::GVisualLoggerProviderLockState);
	}
	virtual void ReadAccessCheck() const override
	{
		Lock.ReadAccessCheck(TraceServices::GVisualLoggerProviderLockState);
	}

	//~ IEditableProvider (write lock)
	virtual void BeginEdit() const override
	{
		Lock.BeginWrite(TraceServices::GVisualLoggerProviderLockState);
	}
	virtual void EndEdit() const override
	{
		Lock.EndWrite(TraceServices::GVisualLoggerProviderLockState);
	}
	virtual void EditAccessCheck() const override
	{
		Lock.WriteAccessCheck(TraceServices::GVisualLoggerProviderLockState);
	}

	//~ IVisualLoggerProvider interface
	virtual bool ReadVisualLogEntryTimeline(uint64 InObjectId, TFunctionRef<void(const VisualLogEntryTimeline&)> Callback) const override;
	virtual void EnumerateCategories(TFunctionRef<void(const FName&)> Callback) const override;

	/** Add an object event message */
	void AppendVisualLogEntry(uint64 InObjectId, double InTime, const FVisualLogEntry& Entry);

	FVLogEntry& AddLogEntry();
	void AddLogEntryChunk(uint32 ID, uint32 ChunkNum, uint16 Size, const TArrayView<const uint8>& ChunkData);

private:
	mutable TraceServices::FProviderLock Lock;
	TraceServices::IAnalysisSession& Session;

	TArray<FVLogEntry> LogEntries;
	TMap<uint64, uint32> ObjectIdToLogEntryTimelines;
	TArray<TSharedRef<TraceServices::TPointTimeline<FVisualLogEntry, FVLogTimelineSettings>>> LogEntryTimelines;
	TArray<FName> Categories;
};
