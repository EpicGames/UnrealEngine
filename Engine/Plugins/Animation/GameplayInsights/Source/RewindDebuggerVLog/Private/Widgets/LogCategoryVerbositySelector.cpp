// Copyright Epic Games, Inc. All Rights Reserved.

#include "LogCategoryVerbositySelector.h"

#include "Common/ProviderLock.h"
#include "Components/VerticalBox.h"
#include "IRewindDebugger.h"
#include "IVisualLoggerProvider.h"
#include "RemoteSessionsManager.h"
#include "RewindDebuggerEngineEditorBridge.h"
#include "RewindDebuggerVLogRuntimeTypes.h"
#include "RewindDebuggerVLogSettings.h"
#include "SessionInfo.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "RewindDebuggerVLog"

namespace UE::RewindDebugger
{

namespace LogCategoryFilterLayout
{
	constexpr float RecordingColumnWidth = 155.f;
	constexpr float DisplayColumnWidth = 179.f;
	constexpr float ScrollbarAllowance = 18.f;
}

namespace ColumnIds
{
	static const FName Category("Category");
	static const FName Recording("Recording");
	static const FName Display("Display");
}


FVLogExtensionSessionData* GetVLogExtensionData()
{
#if WITH_TRACE_BASED_DEBUGGERS
	using namespace UE::TraceBasedDebuggers;
	if (const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance())
	{
		const TSharedPtr<FRemoteSessionsManager> RemoteSessionsManager = FRewindDebuggerEngineEditorBridge::Get().GetSessionsManager();
		const TSharedPtr<FSessionInfo> SessionInfo = RewindDebugger->GetSelectedDebugSessionInfo();
		if (SessionInfo && RemoteSessionsManager)
		{
			FVLogExtensionSessionData* DebuggerData = SessionInfo->GetDebuggerData<FVLogExtensionSessionData>();

			// Create if missing
			if (DebuggerData == nullptr)
			{
				SessionInfo->SetDebuggerData(FVLogExtensionSessionData{});
				DebuggerData = SessionInfo->GetDebuggerData<FVLogExtensionSessionData>();
			}
			return DebuggerData;
		}
	}
#endif // WITH_TRACE_BASED_DEBUGGERS

	return nullptr;
}

FText GetVerbosityAsText(const ELogVerbosity::Type InVerbosity)
{
	if (InVerbosity == UnknownVerbosity)
	{
		return LOCTEXT("UnknownVerbosity", "(unknown)");
	}

	// We reuse NoLogging to mean "category off". The logging system itself never uses it as a verbosity.
	if (InVerbosity == ELogVerbosity::NoLogging)
	{
		return LOCTEXT("NoneVerbosity", "None");
	}

	return FText::FromString(ToString(InVerbosity));
}

FSlateFontInfo GetVerbosityFont(const ELogVerbosity::Type InVerbosity)
{
	if (InVerbosity == UnknownVerbosity)
	{
		return FAppStyle::GetFontStyle("NormalFontItalic");
	}

	return FAppStyle::GetFontStyle("BoldFont");
}

FSlateColor GetVerbosityColor(const ELogVerbosity::Type InVerbosity)
{
	// Using same colors as Unreal Insights (see FTimeMarkerTrackBuilder::GetColorByVerbosity)
	static FLinearColor Colors[] =
	{
		FLinearColor(0.0f, 0.0f, 0.0f, 1.0f), // NoLogging
		FLinearColor(1.0f, 0.0f, 0.0f, 1.0f), // Fatal
		FLinearColor(1.0f, 0.1f, 0.1f, 1.0f), // Error
		FLinearColor(0.7f, 0.5f, 0.0f, 1.0f), // Warning
		FLinearColor(0.0f, 0.7f, 0.0f, 1.0f), // Display
		FLinearColor(0.0f, 0.7f, 1.0f, 1.0f), // Log
		FLinearColor(0.7f, 0.7f, 0.7f, 1.0f), // Verbose
		FLinearColor(1.0f, 1.0f, 1.0f, 1.0f), // VeryVerbose
	};

	// Special case that represents verbosity levels waiting for the response message
	// or for categories that couldn't be found by the remote process
	static FLinearColor UnknownVerbosityColor(0.3f, 0.3f, 0.3f, 1.0f);
	if (InVerbosity == UnknownVerbosity)
	{
		return UnknownVerbosityColor;
	}

	static_assert(sizeof(Colors) / sizeof(FLinearColor) == static_cast<int>(ELogVerbosity::Type::NumVerbosity), "ELogVerbosity::Type has changed!?");
	return Colors[InVerbosity & ELogVerbosity::VerbosityMask];
}

void SLogCategoryFilter::Construct(const FArguments& InArgs)
{
	// Fatal is omitted: the remote's "Log" command collapses NoLogging → Fatal, so we use Fatal as the "off" signal internally.
	VerbosityOptions =
	{
		MakeShared<ELogVerbosity::Type>(ELogVerbosity::Error),
		MakeShared<ELogVerbosity::Type>(ELogVerbosity::Warning),
		MakeShared<ELogVerbosity::Type>(ELogVerbosity::Display),
		MakeShared<ELogVerbosity::Type>(ELogVerbosity::Log),
		MakeShared<ELogVerbosity::Type>(ELogVerbosity::Verbose),
		MakeShared<ELogVerbosity::Type>(ELogVerbosity::VeryVerbose),
	};

	ChildSlot
	[
		BuildContent()
	];
}

void SLogCategoryFilter::OnSearchTextChanged(const FText& InText)
{
	SearchText = InText;
	FilteredCategories.Empty(AllCategories.Num());

	if (SearchText.IsEmpty())
	{
		FilteredCategories = AllCategories;
	}
	else
	{
		for (const TSharedPtr<FLogCategoryVerbosity>& Item : AllCategories)
		{
			if (Item->CategoryName.ToString().Contains(SearchText.ToString()))
			{
				FilteredCategories.Add(Item);
			}
		}
	}

	if (ListView)
	{
		ListView->RequestListRefresh();
	}
}

void SLogCategoryFilter::MeasureCategoryColumnWidth()
{
	constexpr float CategoryCellPadding = 40.f;
	constexpr float MinCategoryWidth = 120.f;

	const FSlateFontInfo NormalFont = FAppStyle::GetFontStyle("NormalFont");
	float MaxWidth = MinCategoryWidth;

	// Get max category name width.
	if (const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance())
	{
		if (const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession())
		{
			if (const IVisualLoggerProvider* VisualLoggerProvider = Session->ReadProvider<IVisualLoggerProvider>("VisualLoggerProvider"))
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*VisualLoggerProvider);

				const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
				VisualLoggerProvider->EnumerateCategories([&MaxWidth, &FontMeasure, &NormalFont](const FName& Category)
				{
					const FVector2D TextSize(FontMeasure->Measure(Category.ToString(), NormalFont));
					MaxWidth = FMath::Max(MaxWidth, static_cast<float>(TextSize.X));
				});
			}
		}
	}

	// Never shrink.
	CategoryColumnWidth = FMath::Max(CategoryColumnWidth, MaxWidth + CategoryCellPadding);
}

FOptionalSize SLogCategoryFilter::GetTotalWidth() const
{
	using namespace LogCategoryFilterLayout;
	const float RecordingWidth = bUsingVerbosityFilterWhenRecording ? RecordingColumnWidth : 0.f;
	return FOptionalSize(CategoryColumnWidth + RecordingWidth + DisplayColumnWidth + ScrollbarAllowance);
}

TSharedRef<SWidget> SLogCategoryFilter::BuildContent()
{
	MeasureCategoryColumnWidth();

	HeaderRowStyle = FAppStyle::Get().GetWidgetStyle<FHeaderRowStyle>("TableView.Header");
	HeaderRowStyle.BackgroundBrush = HeaderRowStyle.ColumnStyle.NormalBrush;
	HeaderRowStyle.SplitterHandleSize = 0.f;
	HeaderRowStyle.HorizontalSeparatorBrush = FSlateNoResource();
	HeaderRowStyle.HorizontalSeparatorThickness = 0.f;

	RowStyle = FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ComboBox.Row");
	RowStyle.SetEvenRowBackgroundBrush(HeaderRowStyle.ColumnStyle.NormalBrush);
	RowStyle.SetOddRowBackgroundBrush(HeaderRowStyle.ColumnStyle.NormalBrush);

	BuildHeaderRow();
	BuildSwitcher();

	RefreshCategoryList();
	HeaderRow->SetShowGeneratedColumn(ColumnIds::Recording, bUsingVerbosityFilterWhenRecording);

	return SNew(SBox)
	.WidthOverride(MakeAttributeSP(this, &SLogCategoryFilter::GetTotalWidth))
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(SearchBox, SSearchBox)
			.OnTextChanged(this, &SLogCategoryFilter::OnSearchTextChanged)
			.InitialText(SearchText)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SImage).Image(FAppStyle::GetBrush("Menu.Separator"))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			Switcher.ToSharedRef()
		]
	];
}

void SLogCategoryFilter::BuildHeaderRow()
{
	using namespace LogCategoryFilterLayout;

	constexpr float ColumnGap = 16.f;
	constexpr float RecordingDisplayGap = 32.f;
	constexpr float DisplayRightPad = 8.f;

	const FSlateFontInfo BoldFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);

	SAssignNew(HeaderRow, SHeaderRow)
	.Style(&HeaderRowStyle)

	+ SHeaderRow::Column(ColumnIds::Category)
	.FixedWidth(CategoryColumnWidth)
	.HeaderContentPadding(FMargin(6.f, 5.f))
	[
		SNew(STextBlock)
		.Text(LOCTEXT("CategoryColumnLabel", "Category"))
		.Font(BoldFont)
	]

	+ SHeaderRow::Column(ColumnIds::Recording)
	.FixedWidth(RecordingColumnWidth)
	.HeaderContentPadding(FMargin(6.f + ColumnGap, 5.f, 6.f, 5.f))
	[
		SNew(SHorizontalBox)
		.ToolTipText(LOCTEXT("RecordingColumnTooltip", "Minimum severity level to record for each category."))
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
		[
			SNew(SBox)
			.WidthOverride(16.f)
			.HeightOverride(16.f)
			.Padding(2.f)
			[
				SNew(SImage)
				.Image(FSlateStyleRegistry::FindSlateStyle("RewindDebuggerStyle")->GetBrush("RewindDebugger.Filter.Recording"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("RecordingColumnLabel", "Recording"))
			.Font(BoldFont)
		]
	]

	+ SHeaderRow::Column(ColumnIds::Display)
	.FixedWidth(DisplayColumnWidth)
	.HeaderContentPadding(FMargin(6.f + RecordingDisplayGap, 5.f, 6.f + DisplayRightPad, 5.f))
	[
		SNew(SHorizontalBox)
		.ToolTipText(LOCTEXT("DisplayColumnTooltip", "Minimum severity level to show in the viewport when scrubbing the timeline."))
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
		[
			SNew(SBox)
			.WidthOverride(16.f)
			.HeightOverride(16.f)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Visible"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("DisplayColumnLabel", "Display"))
			.TextStyle(FAppStyle::Get(), "Graph.Node.NodeTitle")
			.Font(BoldFont)
		]
	];
}

void SLogCategoryFilter::BuildSwitcher()
{
	// Create switcher widget first since RefreshCategoryList will set the active slot
	SAssignNew(Switcher, SWidgetSwitcher)

	// Slot 0 - Waiting state: throbber while fetching verbosity levels from the remote process.
	+ SWidgetSwitcher::Slot()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SCircularThrobber)
		]
	]

	// Slot 1 - Ready state: recording filter checkbox + category list.
	+ SWidgetSwitcher::Slot()
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(4.f, 3.f))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.Padding(FMargin(2.f, 0.f))
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("VerbosityFilteringTooltip", "Whether logging functions based on LogCategories should record an entry based on the category active verbosity level."))
				.IsChecked_Lambda([this]()
					{
						return bUsingVerbosityFilterWhenRecording ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
					{
						SetUseVerbosityLevelFiltering(InCheckBoxState == ECheckBoxState::Checked);
					})
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FilteringEnabled", "Use recording filters"))
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBox)
			.MaxDesiredHeight(250.f)
			[
				SAssignNew(ListView, SListView<TSharedPtr<FLogCategoryVerbosity>>)
				.ListItemsSource(&FilteredCategories)
				.HeaderRow(HeaderRow.ToSharedRef())
				.OnGenerateRow(this, &SLogCategoryFilter::OnGenerateRow)
				.ScrollbarVisibility(EVisibility::Visible)
				.SelectionMode(ESelectionMode::None)
			]
		]
	]

	// Slot 2 - No-connection state: shown when no remote session is established.
	+ SWidgetSwitcher::Slot()
	[
		SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(FMargin(10.0f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("NoRemoteSessionConnected", "No remote session connected.\nConnect to a remote session to manage verbosity filters."))
			.AutoWrapText(true)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		]
	];
}

static TSharedRef<SWidget> MakeVerbosityCell(
	const TArray<TSharedPtr<ELogVerbosity::Type>>& VerbosityOptions,
	const TFunction<ELogVerbosity::Type()>& GetVerbosity,
	const TFunction<void(ELogVerbosity::Type)>& SetVerbosity,
	const TFunction<bool()>& IsComboEnabled,
	const TFunction<bool()>& IsCheckboxEnabled)
{
	return SNew(SHorizontalBox)
	.Clipping(EWidgetClipping::ClipToBounds)

	// Enable/Disable checkbox
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	.Padding(FMargin(2, 0, 2, 0))
	[
		SNew(SCheckBox)
		.IsEnabled(IsCheckboxEnabled)
		.IsChecked_Lambda([GetVerbosity]()
			{
				const ELogVerbosity::Type Verbosity = GetVerbosity();
				return (Verbosity != ELogVerbosity::NoLogging && Verbosity != UnknownVerbosity) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		)
		.OnCheckStateChanged_Lambda([GetVerbosity, SetVerbosity](ECheckBoxState InState)
			{
				if (InState == ECheckBoxState::Checked)
				{
					const ELogVerbosity::Type Current = GetVerbosity();
					// Treat UnknownVerbosity the same as NoLogging: default to Warning when enabling
					const bool bCurrentIsOff = (Current == ELogVerbosity::NoLogging || Current == UnknownVerbosity);
					SetVerbosity(bCurrentIsOff ? ELogVerbosity::Warning : Current);
				}
				else
				{
					SetVerbosity(ELogVerbosity::NoLogging);
				}
			}
		)
	]

	// Category log picker
	+ SHorizontalBox::Slot()
	.FillWidth(1.f)
	[
		SNew(SComboBox<TSharedPtr<ELogVerbosity::Type>>)
		.IsEnabled(IsComboEnabled)
		.OptionsSource(&VerbosityOptions)
		.OnGenerateWidget_Lambda([](TSharedPtr<ELogVerbosity::Type> InOption)
			{
				return SNew(STextBlock)
				.Text(GetVerbosityAsText(*InOption))
				.ColorAndOpacity(GetVerbosityColor(*InOption))
				.Font(GetVerbosityFont(*InOption));
			}
		)
		.OnSelectionChanged_Lambda([SetVerbosity, GetVerbosity](TSharedPtr<ELogVerbosity::Type> InNewValue, ESelectInfo::Type)
			{
				if (InNewValue && GetVerbosity() != ELogVerbosity::NoLogging)
				{
					SetVerbosity(*InNewValue);
				}
			}
		)
		[
			SNew(STextBlock)
			.Text_Lambda([GetVerbosity]()
				{
					const ELogVerbosity::Type Verbosity = GetVerbosity();
					return Verbosity == ELogVerbosity::NoLogging ? LOCTEXT("NoneVerbositySelected", "None") : GetVerbosityAsText(Verbosity);
				}
			)
			.ColorAndOpacity_Lambda([GetVerbosity]()
				{
					const ELogVerbosity::Type Verbosity = GetVerbosity();
					return Verbosity == ELogVerbosity::NoLogging ? FSlateColor(FLinearColor(0.3f, 0.3f, 0.3f, 1.f)) : GetVerbosityColor(Verbosity);
				}
			)
			.Font_Lambda([GetVerbosity]()
				{
					const ELogVerbosity::Type Verbosity = GetVerbosity();
					return Verbosity == ELogVerbosity::NoLogging ? FAppStyle::GetFontStyle("NormalFont") : GetVerbosityFont(Verbosity);
				}
			)
		]
	];
}

class SLogCategoryFilterRow : public SMultiColumnTableRow<TSharedPtr<FLogCategoryVerbosity>>
{
public:
	SLATE_BEGIN_ARGS(SLogCategoryFilterRow) {}
		SLATE_ARGUMENT(TSharedPtr<FLogCategoryVerbosity>, Item)
		SLATE_ARGUMENT(TArray<TSharedPtr<ELogVerbosity::Type>>, VerbosityOptions)
		SLATE_ARGUMENT(TFunction<void(FName, ELogVerbosity::Type)>, OnSetRecordingVerbosity)
		SLATE_STYLE_ARGUMENT(FTableRowStyle, RowStyle)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		Item = InArgs._Item;
		VerbosityOptions = InArgs._VerbosityOptions;
		OnSetRecordingVerbosity = InArgs._OnSetRecordingVerbosity;

		SMultiColumnTableRow::Construct(
			SMultiColumnTableRow::FArguments()
				.Padding(FMargin(5, 2))
				.Style(InArgs._RowStyle),
			InOwnerTable);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnId) override
	{
		const FName CategoryName = Item->CategoryName;

		if (InColumnId == ColumnIds::Category)
		{
			return SNew(SBox)
			.Padding(FMargin(8.f, 0.f, 0.f, 0.f))
			[
				SNew(STextBlock)
				.Text(FText::FromName(CategoryName))
				.ToolTipText(FText::FromName(CategoryName))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Font(FAppStyle::GetFontStyle("NormalFont"))
			];
		}

		if (InColumnId == ColumnIds::Recording)
		{
			auto GetRecordingVerbosity = [Item = Item]()
			{
				return Item->GetVerbosity();
			};

			auto SetRecordingVerbosity = [Item = Item, OnSet = OnSetRecordingVerbosity](ELogVerbosity::Type InVerbosity)
			{
				// No optimistic local write — the row's verbosity is only updated once the server response arrives (via the Sync callback).
				if (OnSet)
				{
					OnSet(Item->CategoryName, InVerbosity);
				}
			};

			auto CategoryRecordingEnabled = [Item = Item]()
			{
				const ELogVerbosity::Type Verbosity = Item->GetVerbosity();
				return Verbosity != ELogVerbosity::NoLogging && Verbosity != UnknownVerbosity;
			};

			auto IsCategoryKnown = [Item = Item]()
			{
				return Item->GetVerbosity() != UnknownVerbosity;
			};

			return SNew(SBox)
			.Padding(FMargin(16.f, 0.f, 0.f, 0.f))
			[
				MakeVerbosityCell(VerbosityOptions, GetRecordingVerbosity, SetRecordingVerbosity, CategoryRecordingEnabled, IsCategoryKnown)
			];
		}

		// ColumnIds::Display
		{
			auto GetDisplayVerbosity = [CategoryName]()
				{
					URewindDebuggerVLogSettings& Settings = URewindDebuggerVLogSettings::Get();
					return Settings.DisplayCategories.Contains(CategoryName)
						? Settings.GetCategoryVerbosity(CategoryName)
						: ELogVerbosity::NoLogging;
				};

			auto SetDisplayVerbosity = [CategoryName](ELogVerbosity::Type InVerbosity)
				{
					URewindDebuggerVLogSettings& Settings = URewindDebuggerVLogSettings::Get();
					if (InVerbosity == ELogVerbosity::NoLogging)
					{
						Settings.DisplayCategories.Remove(CategoryName);
						Settings.Modify();
						Settings.SaveConfig();
					}
					else
					{
						if (!Settings.DisplayCategories.Contains(CategoryName))
						{
							Settings.ToggleCategory(CategoryName);
						}
						Settings.SetCategoryVerbosity(CategoryName, InVerbosity);
					}
				};

			auto CategoryDisplayEnabled = [CategoryName]()
				{
					return URewindDebuggerVLogSettings::Get().DisplayCategories.Contains(CategoryName);
				};

			auto IsDisplayCheckboxEnabled = []() { return true; };

			return SNew(SBox)
			.Padding(FMargin(32.f, 0.f, 8.f, 0.f))
			[
				MakeVerbosityCell(VerbosityOptions, GetDisplayVerbosity, SetDisplayVerbosity, CategoryDisplayEnabled, IsDisplayCheckboxEnabled)
			];
		}
	}

private:
	TSharedPtr<FLogCategoryVerbosity> Item;
	TArray<TSharedPtr<ELogVerbosity::Type>> VerbosityOptions;
	TFunction<void(FName, ELogVerbosity::Type)> OnSetRecordingVerbosity;
};

TSharedRef<ITableRow> SLogCategoryFilter::OnGenerateRow(TSharedPtr<FLogCategoryVerbosity> InItem, const TSharedRef<STableViewBase>& InTable)
{
	return SNew(SLogCategoryFilterRow, InTable)
		.Item(InItem)
		.VerbosityOptions(VerbosityOptions)
		.RowStyle(&RowStyle)
		.OnSetRecordingVerbosity([this](FName InName, ELogVerbosity::Type InVerbosity)
		{
			SetCategoryRecordingVerbosity(InName, InVerbosity);
		});
}

void SLogCategoryFilter::RefreshCategoryList()
{
	AllCategories.Empty();

	auto InsertFunc = [&AllCategories = AllCategories](const FName& LogCategory)
		{
			int32 InsertIndex = 0;
			for (InsertIndex = AllCategories.Num() - 1; InsertIndex >= 0; --InsertIndex)
			{
				TSharedPtr<FLogCategoryVerbosity> CheckCategory = AllCategories[InsertIndex];
				// No duplicates
				if (!CheckCategory.IsValid() || CheckCategory->CategoryName == LogCategory)
				{
					return;
				}

				if (CheckCategory->CategoryName.Compare(LogCategory) < 0)
				{
					break;
				}
			}
			AllCategories.Insert(MakeShareable(new FLogCategoryVerbosity{ LogCategory, UnknownVerbosity }), InsertIndex + 1);
		};

	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	if (const TraceServices::IAnalysisSession* Session = RewindDebugger ? RewindDebugger->GetAnalysisSession() : nullptr)
	{
		if (const IVisualLoggerProvider* VisualLoggerProvider = Session->ReadProvider<IVisualLoggerProvider>("VisualLoggerProvider"))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*VisualLoggerProvider);
			VisualLoggerProvider->EnumerateCategories([InsertFunc](const FName& Category)
				{
					InsertFunc(Category);
				});
		}
	}

	// Re-measure in case newly discovered categories are wider than the initial build-time measurement.
	const float PreviousWidth = CategoryColumnWidth;
	MeasureCategoryColumnWidth();
	if (HeaderRow && CategoryColumnWidth != PreviousWidth)
	{
		HeaderRow->SetColumnWidth(ColumnIds::Category, CategoryColumnWidth);
	}

	// Fetch verbosity levels from the remote process
	SyncVerbosityLevels();

	// Reapply the current search text
	OnSearchTextChanged(SearchText);
}

void SLogCategoryFilter::SyncVerbosityLevels()
{
#if WITH_TRACE_BASED_DEBUGGERS

	int32 SwitcherStateIndex = SwitcherSlotIndexForReadyState;
	ON_SCOPE_EXIT
	{
		if (Switcher)
		{
			Switcher->SetActiveWidgetIndex(SwitcherStateIndex);
		}
	};

	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();

	using namespace UE::TraceBasedDebuggers;
	const TSharedPtr<FRemoteSessionsManager> RemoteSessionsManager = FRewindDebuggerEngineEditorBridge::Get().GetSessionsManager();
	const TSharedPtr<FSessionInfo> SessionInfo = RewindDebugger ? RewindDebugger->GetSelectedDebugSessionInfo() : nullptr;

	if (!SessionInfo.IsValid() || !RemoteSessionsManager.IsValid())
	{
		// No live session: recording verbosity controls are unavailable, but display verbosity still works.
		// Show the category rows as long as there's analysis data to display; fall back to the no-connection placeholder only when there is truly nothing to show.
		const bool bHasDisplayableCategories = RewindDebugger && (RewindDebugger->GetAnalysisSession() != nullptr) && !AllCategories.IsEmpty();
		SetRecordingColumnVisibility(false);
		SwitcherStateIndex = bHasDisplayableCategories ? SwitcherSlotIndexForReadyState : SwitcherSlotIndexForNoConnectionState;
		return;
	}

	// Nothing to sync if the list of known categories is empty
	if (AllCategories.IsEmpty())
	{
		return;
	}

	FVLogExtensionSessionData* DebuggerData = GetVLogExtensionData();
	if (DebuggerData == nullptr)
	{
		return;
	}

	// Build query message to get the verbosity levels of all discovered log categories
	FLogCategoryStatusQueryMessage Query;
	Query.Categories.Reserve(AllCategories.Num());
	for (const TSharedPtr<FLogCategoryVerbosity>& Item : AllCategories)
	{
		Query.Categories.Push(*Item);

		FLogCategoryVerbosity* CategoryState = DebuggerData->LogCategoriesStatesByName.Find(Item->CategoryName);
		if (CategoryState == nullptr)
		{
			CategoryState = &DebuggerData->LogCategoriesStatesByName.Add(Item->CategoryName);
			CategoryState->CategoryName = Item->CategoryName;
			CategoryState->Verbosity = UnknownVerbosity;
		}
	}

	// Mark session data as pending response
	DebuggerData->bPendingRefresh = true;

	// Hide the categories and display feedback for the waiting state
	SwitcherStateIndex = SwitcherSlotIndexForWaitingState;

	// Register callback when the response is received
	DebuggerData->OnDataUpdated.BindSPLambda(this, [this]()
		{
			// Update our list from the updated session data
			if (FVLogExtensionSessionData* DebuggerData = GetVLogExtensionData())
			{
				for (const TSharedPtr<FLogCategoryVerbosity>& Item : AllCategories)
				{
					if (const FLogCategoryVerbosity* State = DebuggerData->LogCategoriesStatesByName.Find(Item->CategoryName))
					{
						Item->Verbosity = State->Verbosity;
						continue;
					}

					ensureMsgf(false, TEXT("The response message should contain the verbosity levels for all our current known categories."));
				}
				bUsingVerbosityFilterWhenRecording = DebuggerData->bUsingVerbosityFilterWhenRecording;
			}

			SetRecordingColumnVisibility(bUsingVerbosityFilterWhenRecording);

			// We can now display the refreshed categories
			if (Switcher)
			{
				Switcher->SetActiveWidgetIndex(SwitcherSlotIndexForReadyState);
			}

			// Force row repaint after the switcher flip so mutated Item data is visible.
			if (ListView)
			{
				ListView->RequestListRefresh();
			}
		});

	// Send the query
	RemoteSessionsManager->SendCommand(SessionInfo->Address, Query);
#endif // WITH_TRACE_BASED_DEBUGGERS
}

void SLogCategoryFilter::SetUseVerbosityLevelFiltering(const bool bEnable)
{
#if WITH_TRACE_BASED_DEBUGGERS
	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	if (RewindDebugger == nullptr)
	{
		return;
	}

	using namespace UE::TraceBasedDebuggers;
	const TSharedPtr<FRemoteSessionsManager> RemoteSessionsManager = FRewindDebuggerEngineEditorBridge::Get().GetSessionsManager();
	const TSharedPtr<FSessionInfo> SessionInfo = RewindDebugger->GetSelectedDebugSessionInfo();
	if (!SessionInfo.IsValid() || !RemoteSessionsManager.IsValid())
	{
		// Can't send request without a valid sessions managers and an active session
		return;
	}

	FVLogExtensionSessionData* DebuggerData = GetVLogExtensionData();
	if (DebuggerData == nullptr)
	{
		return;
	}

	// Mark session data as pending response
	DebuggerData->bPendingRefresh = true;

	// Hide the categories and display feedback for the waiting state
	if (Switcher)
	{
		Switcher->SetActiveWidgetIndex(SwitcherSlotIndexForWaitingState);
	}

	// Register callback when the response is received
	DebuggerData->OnDataUpdated.BindSPLambda(this, [this]()
		{
			// Update our local flag from the updated session data
			if (FVLogExtensionSessionData* DebuggerData = GetVLogExtensionData())
			{
				bUsingVerbosityFilterWhenRecording = DebuggerData->bUsingVerbosityFilterWhenRecording;
			}

			SetRecordingColumnVisibility(bUsingVerbosityFilterWhenRecording);

			// We can now display the refreshed state
			if (Switcher)
			{
				Switcher->SetActiveWidgetIndex(SwitcherSlotIndexForReadyState);
			}
		});

	// Send command to enable/disable the filtering
	RemoteSessionsManager->SendCommand(SessionInfo->Address, FVerbosityFilteringStateChangeCommandMessage{ .bEnableFiltering = bEnable });
#endif // WITH_TRACE_BASED_DEBUGGERS
}

void SLogCategoryFilter::SetRecordingColumnVisibility(const bool bVisible)
{
	if (HeaderRow)
	{
		HeaderRow->SetShowGeneratedColumn(ColumnIds::Recording, bVisible);
	}

	if (ListView)
	{
		ListView->RequestListRefresh();
	}
}

void SLogCategoryFilter::SetCategoryRecordingVerbosity(const FName InCategoryName, const ELogVerbosity::Type InVerbosity) const
{
#if WITH_TRACE_BASED_DEBUGGERS
	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	if (RewindDebugger == nullptr)
	{
		return;
	}

	using namespace UE::TraceBasedDebuggers;
	const TSharedPtr<FRemoteSessionsManager> RemoteSessionsManager = FRewindDebuggerEngineEditorBridge::Get().GetSessionsManager();
	const TSharedPtr<FSessionInfo> SessionInfo = RewindDebugger->GetSelectedDebugSessionInfo();

	if (!SessionInfo.IsValid() || !RemoteSessionsManager.IsValid())
	{
		// Can't send request without a valid sessions managers and an active session
		return;
	}

	FVLogExtensionSessionData* DebuggerData = GetVLogExtensionData();
	if (DebuggerData == nullptr)
	{
		return;
	}

	FLogCategoryVerbosity* CategoryState = DebuggerData->LogCategoriesStatesByName.Find(InCategoryName);
	if (CategoryState == nullptr)
	{
		CategoryState = &DebuggerData->LogCategoriesStatesByName.Add(InCategoryName);
		CategoryState->CategoryName = InCategoryName;
	}

	CategoryState->Verbosity = UnknownVerbosity;

	// Mark session data as pending response
	DebuggerData->bPendingRefresh = true;

	// Hide the categories and display feedback for the waiting state
	if (Switcher)
	{
		Switcher->SetActiveWidgetIndex(SwitcherSlotIndexForWaitingState);
	}

	// No custom OnDataUpdated binding here — the SyncVerbosityLevels callback installed on the last RefreshCategoryList
	// remains bound and will faithfully apply whatever the server echoes back (including UnknownVerbosity for
	// categories the remote did not find). This keeps the UI server-authoritative instead of masking not-found
	// results with the user's requested value.

	// Send command to change the verbosity level
	RemoteSessionsManager->SendCommand(SessionInfo->Address, FLogCategoryStateChangeCommandMessage
		{
			FLogCategoryVerbosity{.CategoryName = CategoryState->CategoryName, .Verbosity = InVerbosity}
		});

#endif // WITH_TRACE_BASED_DEBUGGERS
}

ELogVerbosity::Type SLogCategoryFilter::GetCategoryRecordingVerbosity(const FName InCategoryName) const
{
	if (FVLogExtensionSessionData* DebuggerData = GetVLogExtensionData())
	{
		if (const FLogCategoryVerbosity* CategoryState = DebuggerData->LogCategoriesStatesByName.Find(InCategoryName))
		{
			return CategoryState->GetVerbosity();
		}
	}

	return UnknownVerbosity;
}

bool SLogCategoryFilter::CanChangeCategoryVerbosity(const FName InCategoryName) const
{
	// Prevent users from changing again the verbosity while we wait for the confirmation from the remote process.
	if (const FVLogExtensionSessionData* DebuggerData = GetVLogExtensionData())
	{
		if (DebuggerData->bPendingRefresh)
		{
			return false;
		}

		return true;
	}

	return false;
}

} // UE::RewindDebugger
#undef LOCTEXT_NAMESPACE
