// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Templates/SharedPointerFwd.h"
#include "Internationalization/Text.h"
#include "Styling/SlateTypes.h"
#include "Types/SlateStructs.h"

class ITableRow;
class SWidgetSwitcher;
class SSearchBox;
class SHeaderRow;
class STableViewBase;
template <typename ItemType> class SListView;
namespace ELogVerbosity { enum Type : uint8; }

namespace UE::RewindDebugger
{
struct FLogCategoryVerbosity;

class SLogCategoryFilter : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLogCategoryFilter) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	void RefreshCategoryList();

private:

	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FLogCategoryVerbosity> InItem, const TSharedRef<STableViewBase>& InTable);

	TSharedRef<SWidget> BuildContent();
	void BuildHeaderRow();
	void BuildSwitcher();
	void OnSearchTextChanged(const FText& InText);
	void MeasureCategoryColumnWidth();
	FOptionalSize GetTotalWidth() const;

	void SetUseVerbosityLevelFiltering(bool bEnable);
	void SetRecordingColumnVisibility(bool bVisible);

	void SyncVerbosityLevels();
	bool CanChangeCategoryVerbosity(FName InCategoryName) const;
	void SetCategoryRecordingVerbosity(FName InCategoryName, ELogVerbosity::Type InVerbosity) const;
	ELogVerbosity::Type GetCategoryRecordingVerbosity(FName InCategoryName) const;

	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<SListView<TSharedPtr<FLogCategoryVerbosity>>> ListView;
	TSharedPtr<SWidgetSwitcher> Switcher;
	TSharedPtr<SHeaderRow> HeaderRow;
	FHeaderRowStyle HeaderRowStyle;
	FTableRowStyle RowStyle;

	TArray<TSharedPtr<FLogCategoryVerbosity>> AllCategories;
	TArray<TSharedPtr<FLogCategoryVerbosity>> FilteredCategories;
	TArray<TSharedPtr<ELogVerbosity::Type>> VerbosityOptions;
	FText SearchText;

	static constexpr float DefaultCategoryColumnWidth = 120.f;
	float CategoryColumnWidth = DefaultCategoryColumnWidth;

	bool bUsingVerbosityFilterWhenRecording = false;
	static constexpr int32 SwitcherSlotIndexForWaitingState = 0;
	static constexpr int32 SwitcherSlotIndexForReadyState = 1;
	static constexpr int32 SwitcherSlotIndexForNoConnectionState = 2;
};

} // UE::RewindDebugger

