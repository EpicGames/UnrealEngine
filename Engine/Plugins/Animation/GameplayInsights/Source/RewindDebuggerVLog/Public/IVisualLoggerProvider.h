// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "TraceServices/Containers/Timelines.h"
#include "TraceServices/Model/AnalysisSession.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "VisualLogger/VisualLoggerTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

struct FVisualLogEntry;

class IVisualLoggerProvider
	: public TraceServices::IProvider
	, public TraceServices::IEditableProvider
{
public:
	typedef TraceServices::ITimeline<FVisualLogEntry> VisualLogEntryTimeline;

	virtual bool ReadVisualLogEntryTimeline(uint64 InObjectId, TFunctionRef<void(const VisualLogEntryTimeline&)> Callback) const = 0;
	virtual void EnumerateCategories(TFunctionRef<void(const FName&)> Callback) const = 0;
};

