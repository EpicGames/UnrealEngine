// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class RewindDebuggerVLog : ModuleRules
	{
		public RewindDebuggerVLog(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"GameplayInsights",
				"InputCore",
				"LogVisualizer",
				"RewindDebuggerInterface",
				"RewindDebuggerRuntime",
				"RewindDebuggerVLogRuntime",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"TraceAnalysis",
				"TraceBasedDebuggers",
				"TraceServices",
				"UnrealEd"
			});
		}
	}
}

