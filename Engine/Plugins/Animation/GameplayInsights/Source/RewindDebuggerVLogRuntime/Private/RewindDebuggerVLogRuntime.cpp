// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerVLogRuntime.h"

#if UE_DEBUG_RECORDING_ENABLED
#include "Engine/Engine.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Misc/App.h"
#include "Misc/StringOutputDevice.h"
#include "RemoteSessionsManager.h"
#include "RewindDebuggerVLogRuntimeTypes.h"
#include "SessionInfo.h"
#include "VisualLogger/VisualLogger.h"
#include "VisualLogger/VisualLoggerSettings.h"

namespace UE::RewindDebugger
{

using namespace TraceBasedDebuggers;

FVLogExtensionSessionData* GetSessionData(const TSharedPtr<FRemoteSessionsManager>& InSessionManager, const FGuid InSessionId)
{
	if (InSessionManager)
	{
		if (const TSharedPtr<FSessionInfo> SessionInfoPtr = InSessionManager->GetSessionInfo(InSessionId).Pin())
		{
			return SessionInfoPtr->GetDebuggerData<FVLogExtensionSessionData>();
		}
	}
	return nullptr;
}

void FRewindDebuggerVLogRuntime::RecordingStarted()
{
	// start recording visual logger data
	bWasRecording = FVisualLogger::Get().IsRecording();
	FVisualLogger::Get().SetIsRecordingToTrace(true);
}

void FRewindDebuggerVLogRuntime::RecordingStopped()
{
	// stop recording visual logger data
	FVisualLogger::Get().SetIsRecordingToTrace(false);
	if (!bWasRecording)
	{
		FVisualLogger::Get().SetIsRecording(false);
	}
}

void FRewindDebuggerVLogRuntime::RegisterMessageHandlers(
	const TSharedPtr<FRemoteSessionsManager>& InSessionsManager, FMessageEndpointBuilder& InEndPointBuilder)
{
	using namespace UE::TraceBasedDebuggers;

	InEndPointBuilder
		.Handling<FVerbosityFilteringStateChangeCommandMessage>(
			[WeakSessionsManager = InSessionsManager.ToWeakPtr()](const FVerbosityFilteringStateChangeCommandMessage& InMessage, const TSharedRef<IMessageContext>&)
			{
				UE_AUTORTFM_ONCOMMIT(InMessage, WeakSessionsManager)
				{
					const TSharedPtr<FRemoteSessionsManager> SessionsManager = WeakSessionsManager.Pin();
					if (!SessionsManager.IsValid())
					{
						return;
					}

					FVerbosityFilteringStateChangeResponseMessage ResponseMessage;
					ResponseMessage.InstanceID = FApp::GetInstanceId();

					UVisualLoggerSettings* Settings = GetMutableDefault<UVisualLoggerSettings>();
					Settings->SetUseVerbosityFilterWhenRecording(InMessage.bEnableFiltering);
					ResponseMessage.bFilteringEnabled = Settings->bUseVerbosityFilterWhenRecording;

					SessionsManager->PublishMessage(ResponseMessage);
				};
			})
		.Handling<FLogCategoryStateChangeCommandMessage>(
			[WeakSessionsManager = InSessionsManager.ToWeakPtr()](const FLogCategoryStateChangeCommandMessage& InMessage, const TSharedRef<IMessageContext>&)
			{
				UE_AUTORTFM_ONCOMMIT(InMessage, WeakSessionsManager)
				{
					const TSharedPtr<FRemoteSessionsManager> SessionsManager = WeakSessionsManager.Pin();
					if (!SessionsManager.IsValid())
					{
						return;
					}

					FLogCategoryStateChangeResponseMessage ResponseMessage;
					ResponseMessage.InstanceID = FApp::GetInstanceId();
					ResponseMessage.NewState.CategoryName = InMessage.NewState.CategoryName;
					// Default to UnknownVerbosity so the editor side knows the category was not found.
					// The editor callback distinguishes "found and set to X" from "not found" by checking
					// for UnknownVerbosity in the response.
					ResponseMessage.NewState.Verbosity = UnknownVerbosity;

					// Make sure the category exists
					FStringOutputDevice OutputDevice;
					GEngine->Exec(/*World*/nullptr, TEXT("Log list"), OutputDevice);

					TArray<FString> AllCategories;
					OutputDevice.ParseIntoArrayWS(AllCategories);

					const int32 CategoryIndex = AllCategories.IndexOfByKey(InMessage.NewState.CategoryName);

					// If it exists we can then execute the command and send confirmation of the new verbosity level
					if (CategoryIndex != INDEX_NONE)
					{
						// "Log" command parser does not accept "NoLogging"; map it to "None".
						const ELogVerbosity::Type RequestedVerbosity = InMessage.NewState.GetVerbosity();
						const TCHAR* VerbosityArg = (RequestedVerbosity == ELogVerbosity::NoLogging)
							? TEXT("None")
							: ToString(RequestedVerbosity);
						const FString Command = FString::Printf(TEXT("Log %s %s")
							, *InMessage.NewState.CategoryName.ToString()
							, VerbosityArg);
						GEngine->Exec(/*World*/nullptr, *Command);

						ResponseMessage.NewState.Verbosity = InMessage.NewState.Verbosity;
					}

					SessionsManager->PublishMessage(ResponseMessage);
				};
			})
		.Handling<FLogCategoryStatusQueryMessage>(
			[WeakSessionsManager = InSessionsManager.ToWeakPtr()](const FLogCategoryStatusQueryMessage& InMessage, const TSharedRef<IMessageContext>&)
			{
				UE_AUTORTFM_ONCOMMIT(InMessage, WeakSessionsManager)
				{
					if (const TSharedPtr<FRemoteSessionsManager> SessionsManager = WeakSessionsManager.Pin())
					{
						FStringOutputDevice OutputDevice;
						GEngine->Exec(/*World*/nullptr, TEXT("Log list"), OutputDevice);

						FLogCategoryStatusQueryResponseMessage ResponseMessage;
						ResponseMessage.InstanceID = FApp::GetInstanceId();
						ResponseMessage.Categories = InMessage.Categories;
						ResponseMessage.bUsingVerbosityFilterWhenRecording = FVisualLogger::bUseVerbosityFilterWhenRecording;
						TArray<FString> AllCategories;
						OutputDevice.ParseIntoArrayWS(AllCategories);
						for (FLogCategoryVerbosity& CategoryInfo : ResponseMessage.Categories)
						{
							CategoryInfo.Verbosity = UnknownVerbosity;
							const int32 CategoryIndex = AllCategories.IndexOfByKey(CategoryInfo.CategoryName);
							if (CategoryIndex != INDEX_NONE)
							{
								const int32 VerbosityIndex = CategoryIndex + 1;
								if (AllCategories.IsValidIndex(VerbosityIndex))
								{
									// "None" collapses to Fatal on the remote; fold it back so the UI round-trips cleanly.
									const ELogVerbosity::Type Verbosity = ParseLogVerbosityFromString(AllCategories[VerbosityIndex]);
									CategoryInfo.Verbosity = (Verbosity == ELogVerbosity::Fatal) ? ELogVerbosity::NoLogging : Verbosity;
								}
							}
						}

						SessionsManager->PublishMessage(ResponseMessage);
					}
				};
			});

	if (InSessionsManager && InSessionsManager->IsController())
	{
		InEndPointBuilder
			.Handling<FVerbosityFilteringStateChangeResponseMessage>(
				[WeakSessionsManager = InSessionsManager.ToWeakPtr()](const FVerbosityFilteringStateChangeResponseMessage& InMessage, const TSharedRef<IMessageContext>&)
				{
					if (FVLogExtensionSessionData* DebuggerData = GetSessionData(WeakSessionsManager.Pin(), InMessage.InstanceID))
					{
						DebuggerData->bUsingVerbosityFilterWhenRecording = InMessage.bFilteringEnabled;
						DebuggerData->NotifyDataUpdated();
					}
				})
			.Handling<FLogCategoryStateChangeResponseMessage>(
				[WeakSessionsManager = InSessionsManager.ToWeakPtr()](const FLogCategoryStateChangeResponseMessage& InMessage, const TSharedRef<IMessageContext>&)
				{
					if (FVLogExtensionSessionData* DebuggerData = GetSessionData(WeakSessionsManager.Pin(), InMessage.InstanceID))
					{
						if (FLogCategoryVerbosity* FoundState = DebuggerData->LogCategoriesStatesByName.Find(InMessage.NewState.CategoryName))
						{
							*FoundState = InMessage.NewState;
						}

						DebuggerData->NotifyDataUpdated();
					}
				})
			.Handling<FLogCategoryStatusQueryResponseMessage>(
				[WeakSessionsManager = InSessionsManager.ToWeakPtr()](const FLogCategoryStatusQueryResponseMessage& InMessage, const TSharedRef<IMessageContext>&)
				{
					if (FVLogExtensionSessionData* DebuggerData = GetSessionData(WeakSessionsManager.Pin(), InMessage.InstanceID))
					{
						for (const FLogCategoryVerbosity& Category : InMessage.Categories)
						{
							if (FLogCategoryVerbosity* FoundState = DebuggerData->LogCategoriesStatesByName.Find(Category.CategoryName))
							{
								*FoundState = Category;
							}
						}
						DebuggerData->bUsingVerbosityFilterWhenRecording = InMessage.bUsingVerbosityFilterWhenRecording;
						DebuggerData->NotifyDataUpdated();
					}
				});
	}
}

void FRewindDebuggerVLogRuntime::RegisterMessageTypes(
	const TSharedPtr<FRemoteSessionsManager>& InSessionsManager, FMessageEndpoint& InMessageEndpoint)
{
	if (InSessionsManager)
	{
		InSessionsManager->RegisterExternalSupportedMessageType<FVerbosityFilteringStateChangeCommandMessage>();
		InSessionsManager->RegisterExternalSupportedMessageType<FVerbosityFilteringStateChangeResponseMessage>();
		InSessionsManager->RegisterExternalSupportedMessageType<FLogCategoryStateChangeCommandMessage>();
		InSessionsManager->RegisterExternalSupportedMessageType<FLogCategoryStateChangeResponseMessage>();
		InSessionsManager->RegisterExternalSupportedMessageType<FLogCategoryStatusQueryMessage>();
		InSessionsManager->RegisterExternalSupportedMessageType<FLogCategoryStatusQueryResponseMessage>();

		if (InSessionsManager->IsController())
		{
			InMessageEndpoint.Subscribe<FVerbosityFilteringStateChangeResponseMessage>();
			InMessageEndpoint.Subscribe<FLogCategoryStateChangeResponseMessage>();
			InMessageEndpoint.Subscribe<FLogCategoryStatusQueryResponseMessage>();
		}
	}

	InMessageEndpoint.Subscribe<FVerbosityFilteringStateChangeCommandMessage>();
	InMessageEndpoint.Subscribe<FLogCategoryStateChangeCommandMessage>();
	InMessageEndpoint.Subscribe<FLogCategoryStatusQueryMessage>();
}

} // UE::RewindDebugger

#endif // UE_DEBUG_RECORDING_ENABLED