// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VisualLogger/VisualLoggerDefines.h"

#if UE_DEBUG_RECORDING_ENABLED
#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"

struct FMessageEndpointBuilder;
class FMessageEndpoint;

namespace UE::RewindDebugger
{
class FRewindDebuggerVLogRuntime : public IRewindDebuggerRuntimeExtension
{
public:
	virtual void RecordingStarted() override;
	virtual void RecordingStopped() override;
	virtual void RegisterMessageHandlers(const TSharedPtr<TraceBasedDebuggers::FRemoteSessionsManager>&, FMessageEndpointBuilder&) override;
	virtual void RegisterMessageTypes(const TSharedPtr<TraceBasedDebuggers::FRemoteSessionsManager>&, FMessageEndpoint&) override;

private:
	bool bWasRecording = false;
};

}
#endif // UE_DEBUG_RECORDING_ENABLED
