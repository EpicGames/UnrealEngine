// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerVLogRuntimeModule.h"
#include "Modules/ModuleManager.h"

#if UE_DEBUG_RECORDING_ENABLED
#include "Features/IModularFeatures.h"

void FRewindDebuggerVLogRuntimeModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, &VLogExtension);
}

void FRewindDebuggerVLogRuntimeModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, &VLogExtension);
}
#endif // UE_DEBUG_RECORDING_ENABLED

IMPLEMENT_MODULE(FRewindDebuggerVLogRuntimeModule, RewindDebuggerVLogRuntime);
