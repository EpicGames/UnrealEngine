// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VisualLogger/VisualLoggerDefines.h"
#include "Modules/ModuleInterface.h"

#if UE_DEBUG_RECORDING_ENABLED
#include "RewindDebuggerVLogRuntime.h"
#endif // UE_DEBUG_RECORDING_ENABLED

class FRewindDebuggerVLogRuntimeModule : public IModuleInterface
{
#if UE_DEBUG_RECORDING_ENABLED
public:
	//~ IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	UE::RewindDebugger::FRewindDebuggerVLogRuntime VLogExtension;
#endif // UE_DEBUG_RECORDING_ENABLED
};
