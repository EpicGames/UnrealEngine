// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogVerbosity.h"
#include "Misc/Guid.h"
#include "RewindDebuggerVLogRuntimeTypes.generated.h"

namespace UE::RewindDebugger
{

/**
 * Specific value for category verbosity levels waiting for their status provided by the remote process
 * or for categories not found by the remote process (edge case for now, but will be useful when allowing users to type manually a category)
 */
constexpr ELogVerbosity::Type UnknownVerbosity = ELogVerbosity::NumVerbosity;

USTRUCT()
struct FLogCategoryVerbosity
{
	GENERATED_BODY()

	UPROPERTY()
	FName CategoryName;

	UPROPERTY()
	uint8 Verbosity = 0;

	ELogVerbosity::Type GetVerbosity() const
	{
		return static_cast<ELogVerbosity::Type>(Verbosity);
	}
};

USTRUCT()
struct FVerbosityFilteringStateChangeCommandMessage
{
	GENERATED_BODY()

	UPROPERTY()
	bool bEnableFiltering = false;
};

USTRUCT()
struct FVerbosityFilteringStateChangeResponseMessage
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid InstanceID = FGuid();

	UPROPERTY()
	bool bFilteringEnabled = false;
};

USTRUCT()
struct FLogCategoryStateChangeCommandMessage
{
	GENERATED_BODY()

	UPROPERTY()
	FLogCategoryVerbosity NewState;
};

USTRUCT()
struct FLogCategoryStateChangeResponseMessage
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid InstanceID = FGuid();

	UPROPERTY()
	FLogCategoryVerbosity NewState;
};

USTRUCT()
struct FLogCategoryStatusQueryMessage
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FLogCategoryVerbosity> Categories;
};

USTRUCT()
struct FLogCategoryStatusQueryResponseMessage
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid InstanceID = FGuid();

	UPROPERTY()
	bool bUsingVerbosityFilterWhenRecording = false;

	UPROPERTY()
	TArray<FLogCategoryVerbosity> Categories;
};

/**
 * Structure specific to RewindDebugger VLog extension to store the states of the log categories in the session info struct.
 * The struct is stored as a DebuggerSpecificSessionData inside UE::TraceBasedDebuggers::FSessionInfo
 * and get be retrieved using GetDebuggerData<UE::RewindDebugger::FVLogExtensionSessionData>().
 */
USTRUCT()
struct FVLogExtensionSessionData
{
	GENERATED_BODY()

	TMap<FName, FLogCategoryVerbosity> LogCategoriesStatesByName;

	FSimpleDelegate OnDataUpdated;
	bool bUsingVerbosityFilterWhenRecording = false;
	bool bPendingRefresh = false;

	void NotifyDataUpdated()
	{
		bPendingRefresh = false;
		OnDataUpdated.ExecuteIfBound();
	}
};

}
