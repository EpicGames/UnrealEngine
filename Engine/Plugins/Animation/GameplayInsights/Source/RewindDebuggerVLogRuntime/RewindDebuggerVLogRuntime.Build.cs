// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class RewindDebuggerVLogRuntime : ModuleRules
	{
		public RewindDebuggerVLogRuntime(ReadOnlyTargetRules Target) : base(Target)
		{
			bAllowUETypesInNamespaces = true;

			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"RewindDebuggerRuntimeInterface",
				"TraceBasedDebuggers",
				"TraceLog"
			});
		}
	}
}

